[/Script/PixelStreaming.PixelStreamingSettings]
DefaultCursorClassName=/PixelStreaming/DefaultCursor.DefaultCursor_C
TextEditBeamCursorClassName=/PixelStreaming/TextEditBeamCursor.TextEditBeamCursor_C
HiddenCursorClassName=/PixelStreaming/HiddenCursor.HiddenCursor_C
bMouseAlwaysAttached=true

[CoreRedirects]
+ClassRedirects=(OldName="PixelStreamerInputComponent", NewName="/Script/PixelStreaming.PixelStreamingInput")
+ClassRedirects=(OldName="PixelStreamingInputComponent",NewName="/Script/PixelStreaming.PixelStreamingInput")
+PropertyRedirects=(OldName="PixelStreamerInputComponent.OnPixelStreamingInputEvent",NewName="PixelStreamerInputComponent.OnInputEvent")