{
	"FileVersion": 3,
	"Version": 1.0,
	"VersionName": "1.0",
	"FriendlyName": "Pixel Streaming",
	"Description": "Streaming of Unreal Engine audio and rendering to WebRTC-compatible media players such as a web browsers.",
	"Category": "Graphics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Platforms/PixelStreaming/index.html",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PixelStreaming",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ],
			"TargetDenyList": [ ]
		},
		{
			"Name": "PixelStreamingServers",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ],
			"TargetDenyList": [ ]
		},
		{
            "Name": "PixelStreamingEditor",
            "Type": "UncookedOnly",
            "LoadingPhase": "PreDefault",
            "PlatformAllowList": [ "Win64", "Linux", "Mac" ],
            "TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PixelStreamingBlueprint",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PixelStreamingBlueprintEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PixelStreamingHMD",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PixelStreamingInput",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "PixelCapture",
			"Enabled": true
		},
		{
			"Name": "WmfMedia",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "AVCodecsCore",
			"Enabled": true,
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ]
		},
		{
			"Name": "NVCodecs",
			"Enabled": true,
            "PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "AMFCodecs",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		},
        {
			"Name": "VTCodecs",
			"Enabled": true,
			"PlatformAllowList": [ "Mac" ]
		},
		{
			"Name": "WebSocketNetworking",
			"Enabled": true
		},
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		}
	]
}
