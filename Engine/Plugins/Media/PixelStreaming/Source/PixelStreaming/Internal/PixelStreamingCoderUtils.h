// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/VideoDecoder.h"
#if PLATFORM_WINDOWS
	#include "DynamicRHI.h"
	#include "Video/Resources/D3D/VideoResourceD3D.h"
	#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#elif PLATFORM_LINUX
	#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#elif PLATFORM_APPLE
	#include "Video/Resources/Metal/VideoResourceMetal.h"
#endif

namespace UE::PixelStreaming
{
	/**
	 * As windows supports many RHIs and many codecs, we need to check at runtime if the current codec and RHI is compatible.
	 * 
	 * To remove nested switch-cases, this function is templated to take a video encoder config of the target codec. eg:
	 * 
	 * IsEncoderSupported<FVideoEncoderConfigH264>();
	 * OR
	 * IsEncoderSupported<FVideoEncoderConfigAV1>();
	 * OR
	 * ...
	*/
    template <typename TCodec>
	PIXELSTREAMING_API inline bool IsEncoderSupported()
	{
		TSharedRef<FAVInstance> const Instance = MakeShared<FAVInstance>();
#if PLATFORM_WINDOWS
		const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
		switch (RHIType)
		{
			case ERHIInterfaceType::D3D11:
				return FVideoEncoder::IsSupported<FVideoResourceD3D11, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::D3D12:
				return FVideoEncoder::IsSupported<FVideoResourceD3D12, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::Vulkan:
				return FVideoEncoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			default:
				return false;
		}
#elif PLATFORM_LINUX
		return FVideoEncoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#elif PLATFORM_APPLE
		return FVideoEncoder::IsSupported<FVideoResourceMetal, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#else
		return false;		
#endif
	}

	/**
	 * As windows supports many RHIs and many codecs, we need to check at runtime if the current codec and RHI is compatible.
	 * 
	 * To remove nested switch-cases, this function is templated to take a video decoder config of the target codec. eg:
	 * 
	 * IsDecoderSupported<FVideoDecoderConfigVP8>();
	 * OR
	 * IsDecoderSupported<FVideoDecoderConfigAV1>();
	 * OR
	 * ...
	*/
	template <typename TCodec>
	PIXELSTREAMING_API inline bool IsDecoderSupported()
	{
		TSharedRef<FAVInstance> const Instance = MakeShared<FAVInstance>();
#if PLATFORM_WINDOWS
		const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
		switch (RHIType)
		{
			case ERHIInterfaceType::D3D11:
				return FVideoDecoder::IsSupported<FVideoResourceD3D11, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::D3D12:
				return FVideoDecoder::IsSupported<FVideoResourceD3D12, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::Vulkan:
				return FVideoDecoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			default:
				return false;
		}
#elif PLATFORM_LINUX
		return FVideoDecoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#elif PLATFORM_APPLE
		return FVideoDecoder::IsSupported<FVideoResourceMetal, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#else
		return false;		
#endif
	}
}