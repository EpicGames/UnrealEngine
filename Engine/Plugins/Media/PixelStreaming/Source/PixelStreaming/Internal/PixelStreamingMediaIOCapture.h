// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StripedMap.h"
#include "MediaCapture.h"
#include "PixelStreamingVideoInput.h"
#include "Slate/SceneViewport.h"

#include "PixelStreamingMediaIOCapture.generated.h"

UCLASS(BlueprintType)
class PIXELSTREAMING_API UPixelStreamingMediaIOCapture : public UMediaCapture
{
	GENERATED_BODY()

	//~ Begin UMediaCapture interface
public:
	/**
	 * GPU copy methods
	 */
	virtual void OnRHIResourceCaptured_RenderingThread(
		FRHICommandListImmediate&							   RHICmdList,
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		FTextureRHIRef										   InTexture) override;

	virtual void OnRHIResourceCaptured_AnyThread(
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		FTextureRHIRef										   InTexture) override;

	/**
	 * CPU readback methods
	 */
	virtual void OnFrameCaptured_RenderingThread(
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		void*												   InBuffer,
		int32												   Width,
		int32												   Height,
		int32												   BytesPerRow) override;

	/**
	 * Custom conversion operation for Mac
	 */
	virtual void OnCustomCapture_RenderingThread(
		FRDGBuilder&										   GraphBuilder,
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		FRDGTextureRef										   InSourceTexture,
		FRDGTextureRef										   OutputTexture,
		const FRHICopyTextureInfo&							   CopyInfo,
		FVector2D											   CropU,
		FVector2D											   CropV) override;

	virtual bool InitializeCapture() override;
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool ShouldCaptureRHIResource() const { return bDoGPUCopy; }
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;
	virtual bool SupportsAnyThreadCapture() const override;
	// We override the texture flags because on Mac we want the texture to have the CPU_Readback flag
	virtual ETextureCreateFlags GetOutputTextureFlags() const override;
	virtual void				WaitForGPU(FRHITexture* InRHITexture) override;
	//~ End UMediaCapture interface

	TSharedPtr<FSceneViewport>			GetViewport() const { return SceneViewport.Pin(); }
	virtual void						ViewportResized(FViewport* Viewport, uint32 ResizeCode);
	bool								WasViewportResized() const { return bViewportResized; }
	void								SetVideoInput(TSharedPtr<FPixelStreamingVideoInput> InVideoInput) { VideoInput = InVideoInput; }
	TWeakPtr<FPixelStreamingVideoInput> GetVideoInput() { return VideoInput; }

	DECLARE_MULTICAST_DELEGATE(FOnCaptureViewportInitialized);
	FOnCaptureViewportInitialized OnCaptureViewportInitialized;

private:
	void HandleCapturedFrame(FTextureRHIRef InTexture);
	void UpdateCaptureResolution(int32 Width, int32 Height);

private:
	TWeakPtr<FSceneViewport>			SceneViewport;
	TWeakPtr<FPixelStreamingVideoInput> VideoInput;

	/* We track whether the viewport has been resized since we created this capturer as resize means restart capturer. */
	bool bViewportResized = false;

	/* Tracks the captured resolution, we use this to determine if resize events contain a different resolution than what we previously captured. */
	TUniquePtr<FIntPoint> CaptureResolution;

	/* Whether we want the UMediaCapture to read back to frame into cpu memory or not. */
	bool bDoGPUCopy = true;

	// Thread safe map of fences used to wait on to ensure the media capture conversion is complete
	TStripedMap<32, uintptr_t, FGPUFenceRHIRef> Fences;
};
