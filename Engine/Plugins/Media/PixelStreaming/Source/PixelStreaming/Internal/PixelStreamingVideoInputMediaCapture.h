// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Misc/Optional.h"
#include "MediaOutput.h"
#include "PixelStreamingVideoInputRHI.h"
#include "PixelStreamingMediaIOCapture.h"
#include "PixelStreamingVideoInputMediaCapture.generated.h"

UCLASS(BlueprintType)
class UPixelStreamingMediaIOOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	virtual FIntPoint GetRequestedSize() const override { return UMediaOutput::RequestCaptureSourceSize; }
	virtual EPixelFormat GetRequestedPixelFormat() const override { return EPixelFormat::PF_B8G8R8A8; }
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override { return EMediaCaptureConversionOperation::CUSTOM; }

private:
	UPROPERTY(Transient)
	TObjectPtr<UPixelStreamingMediaIOCapture> MediaCapture = nullptr;
};

/*
 * Use this if you want to send VCam output as video input.
 */
class PIXELSTREAMING_API FPixelStreamingVideoInputMediaCapture : public FPixelStreamingVideoInputRHI, public TSharedFromThis<FPixelStreamingVideoInputMediaCapture>
{
public:

	/**
	 * @brief Creates a MediaIO capture of the active viewport and starts capturing as soon as possible.
	 * @return A video input backed by the created MediaIO capture that sends frames from the active viewport.
	*/
	static TSharedPtr<FPixelStreamingVideoInputMediaCapture> CreateActiveViewportCapture();

	/**
	 * @brief Creates a video input where the user can specify their own MediaIO output and capture.
	 * This method does not does not configure or start capturing, this is left to the user.
	 * Use this constructor if you know how to configure the MediaIOCapture yourself or don't want to capture the active viewport.
	 * @param MediaCapture The custom MediaIOCapture that will pass its captured frames as video input.
	 * @return A video input backed by the passed in MediaIO capture.
	*/
	static TSharedPtr<FPixelStreamingVideoInputMediaCapture> Create(TObjectPtr<UPixelStreamingMediaIOCapture> MediaCapture);

	FPixelStreamingVideoInputMediaCapture();
	FPixelStreamingVideoInputMediaCapture(TObjectPtr<UPixelStreamingMediaIOCapture> MediaCapture);
	virtual ~FPixelStreamingVideoInputMediaCapture();

	virtual FString ToString() override;

protected:

	virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, FIntPoint Resolution) override;
	void StartActiveViewportCapture();
	void LateStartActiveViewportCapture();

private:
	void CreateDefaultActiveViewportMediaCapture();
	void OnCaptureActiveViewportStateChanged();

private:
	TObjectPtr<UPixelStreamingMediaIOCapture> MediaCapture = nullptr;

	TOptional<FDelegateHandle> OnFrameEndDelegateHandle;
};
