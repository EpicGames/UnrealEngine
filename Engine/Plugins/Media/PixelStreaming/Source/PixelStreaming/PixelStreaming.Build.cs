// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;

namespace UnrealBuildTool.Rules
{
    public class PixelStreaming : ModuleRules
    {
        string PixelStreamingProgramsOutputDirectory = "$(ProjectDir)/Samples/PixelStreaming/WebServers";
        const string PixelStreamingProgramsDirectory = "../../Engine/Plugins/Media/PixelStreaming/Resources/WebServers";

        private void AddFolder(string Folder)
        {
            string DirectoryToAdd = new DirectoryInfo(Path.Combine(PixelStreamingProgramsDirectory, Folder)).FullName;

            if (!Directory.Exists(DirectoryToAdd))
            {
                return;
            }

            List<string> DependenciesToAdd = new List<string>();
            DependenciesToAdd.AddRange(Directory.GetFiles(DirectoryToAdd, "*.*", SearchOption.AllDirectories));

            List<string> PathsToIgnore = new List<string>();
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/node_modules").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/implementations/typescript/node_modules").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/implementations/EpicGames/node_modules").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/implementations/react/node_modules").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/library/node_modules").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/ui-library/node_modules").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/logs").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/platform_scripts/cmd/node").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/platform_scripts/cmd/coturn").FullName);
            PathsToIgnore.Add(new DirectoryInfo(DirectoryToAdd + "/platform_scripts/bash/node").FullName);

            // Go through each file/folder in the WebServers and selectively choose which ones we wish to package.
            foreach (var DependencySource in DependenciesToAdd)
            {
                // We want to skip packaging the above directories as they are all generated by running scripts
                bool bSkip = false;
                foreach (var PathToIgnore in PathsToIgnore)
                {
                    if (DependencySource.StartsWith(PathToIgnore))
                    {
                        bSkip = true;
                        continue;
                    }
                }

                // If we are not skipping, we want to add this to our packaged output
                if (!bSkip)
                {
                    string RelativeSourcePath = Path.GetRelativePath(PixelStreamingProgramsDirectory, DependencySource);
                    RuntimeDependencies.Add(Path.Combine(PixelStreamingProgramsOutputDirectory, RelativeSourcePath), DependencySource, StagedFileType.SystemNonUFS);
                }
            }
        }

        public PixelStreaming(ReadOnlyTargetRules Target) : base(Target)
        {
			bRequiresPlatformSDK = true;
			
			IWYUSupport = IWYUSupport.None;

            // This is so for game projects using our public headers don't have to include extra modules they might not know about.
            PublicDependencyModuleNames.AddRange(new string[]
            {
                "ApplicationCore",
                "InputDevice",
                "WebRTC",
                "PixelCapture",
                "PixelStreamingInput",
                "MediaIOCore"
            });

            PrivateDependencyModuleNames.AddRange(new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "EngineSettings",
                "InputCore",
                "Json",
                "Renderer",
                "RenderCore",
                "RHI",
                "SignalProcessing",
                "Slate",
                "SlateCore",
                "AudioMixer",
                "WebRTC",
                "WebSockets",
                "Sockets",
                "MediaUtils",
                "DeveloperSettings",
                "AVCodecsCore",
                "AVCodecsCoreRHI",
                "PixelCaptureShaders",
                "PixelStreamingServers",
                "PixelStreamingHMD",
                "TraceLog",
                "HTTP",
                "NVML",
                "AudioPlatformConfiguration"
            });
            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.AddRange(new string[]
                {
                    "UnrealEd"
                });
            }

            PrivateDefinitions.Add("PIXELSTREAMING_DUMP_ENCODING=0");

            if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
            {
                PrivateDependencyModuleNames.Add("VulkanRHI");
                AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan", "CUDA");
            }

            PrivateDefinitions.Add("PIXELSTREAMING_DUMP_ENCODING=0");


            if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
            {
                PrivateDependencyModuleNames.Add("D3D11RHI");
                PrivateDependencyModuleNames.Add("D3D12RHI");

                AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");
            }

            if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
            {
                AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
            }

            // When we build a Game target we also package the servers with it as runtime dependencies
            if (Target.Type == TargetType.Game && Target.ProjectFile != null)
            {
                AddFolder("Frontend");
                AddFolder("SignallingWebServer");
                AddFolder("Matchmaker");
                AddFolder("SFU");
                AddFolder("Extras");
                AddFolder("Signalling");
                AddFolder("Common");
                AddFolder("SS_Test");

                RuntimeDependencies.Add("$(ProjectDir)/Samples/PixelStreaming/WebServers/get_ps_servers.bat", "$(PluginDir)/Resources/WebServers/get_ps_servers.bat", StagedFileType.NonUFS);
                RuntimeDependencies.Add("$(ProjectDir)/Samples/PixelStreaming/WebServers/get_ps_servers.sh", "$(PluginDir)/Resources/WebServers/get_ps_servers.sh", StagedFileType.NonUFS);
            }
        }
    }
}
