// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdaptedVideoTrackSource.h"

namespace UE::PixelStreaming
{
	FAdaptedVideoTrackSource::FAdaptedVideoTrackSource() = default;

	FAdaptedVideoTrackSource::FAdaptedVideoTrackSource(int required_alignment)
		: video_adapter_(required_alignment) {}

	FAdaptedVideoTrackSource::~FAdaptedVideoTrackSource() = default;

	bool FAdaptedVideoTrackSource::GetStats(Stats* stats)
	{
		webrtc::MutexLock lock(&stats_mutex_);

		if (!stats_)
		{
			return false;
		}

		*stats = *stats_;
		return true;
	}

	void FAdaptedVideoTrackSource::OnFrame(const webrtc::VideoFrame& frame)
	{
		rtc::scoped_refptr<webrtc::VideoFrameBuffer> buffer(
			frame.video_frame_buffer());
		/*  Note that this is a "best effort" approach to
			wants.rotation_applied; apply_rotation_ can change from false to
			true between the check of apply_rotation() and the call to
			broadcaster_.OnFrame(), in which case we generate a frame with
			pending rotation despite some sink with wants.rotation_applied ==
			true was just added. The VideoBroadcaster enforces
			synchronization for us in this case, by not passing the frame on
			to sinks which don't want it.
		*/
		if (apply_rotation() && frame.rotation() != webrtc::kVideoRotation_0 && buffer->type() == webrtc::VideoFrameBuffer::Type::kI420)
		{
			/* Apply pending rotation. */
			webrtc::VideoFrame rotated_frame(frame);
			rotated_frame.set_video_frame_buffer(
				webrtc::I420Buffer::Rotate(*buffer->GetI420(), frame.rotation()));
			rotated_frame.set_rotation(webrtc::kVideoRotation_0);
			broadcaster_.OnFrame(rotated_frame);
		}
		else
		{
			broadcaster_.OnFrame(frame);
		}
	}

	void FAdaptedVideoTrackSource::AddOrUpdateSink(
		rtc::VideoSinkInterface<webrtc::VideoFrame>* sink,
		const rtc::VideoSinkWants& wants)
	{
		broadcaster_.AddOrUpdateSink(sink, wants);
		OnSinkWantsChanged(broadcaster_.wants());
	}

	void FAdaptedVideoTrackSource::RemoveSink(
		rtc::VideoSinkInterface<webrtc::VideoFrame>* sink)
	{
		broadcaster_.RemoveSink(sink);
		OnSinkWantsChanged(broadcaster_.wants());
	}

	bool FAdaptedVideoTrackSource::apply_rotation()
	{
		return broadcaster_.wants().rotation_applied;
	}

	void FAdaptedVideoTrackSource::OnSinkWantsChanged(
		const rtc::VideoSinkWants& wants)
	{
		video_adapter_.OnSinkWants(wants);
	}

	bool FAdaptedVideoTrackSource::AdaptFrame(int width,
		int height,
		int64_t time_us,
		int* out_width,
		int* out_height,
		int* crop_width,
		int* crop_height,
		int* crop_x,
		int* crop_y)
	{
		{
			webrtc::MutexLock lock(&stats_mutex_);
			stats_ = Stats{ width, height };
		}

		if (!broadcaster_.frame_wanted())
		{
			return false;
		}

		if (!video_adapter_.AdaptFrameResolution(
				width, height, time_us * rtc::kNumNanosecsPerMicrosec, crop_width,
				crop_height, out_width, out_height))
		{
			broadcaster_.OnDiscardedFrame();
			// VideoAdapter dropped the frame.
			return false;
		}

		*crop_x = (width - *crop_width) / 2;
		*crop_y = (height - *crop_height) / 2;
		return true;
	}

} // namespace UE::PixelStreaming
