// Copyright Epic Games, Inc. All Rights Reserved.

/*
 *  Copyright (c) 2016 The WebRTC project authors. All Rights Reserved.
 *
 *  Use of this source code is governed by a BSD-style license
 *  that can be found in the LICENSE file in the root of the source
 *  tree. An additional intellectual property rights grant can be found
 *  in the file PATENTS.  All contributing project authors may
 *  be found in the AUTHORS file in the root of the source tree.
 */

#pragma once

#include <stdint.h>
#include "WebRTCIncludes.h"

/*
 * This code is nearly identical to the version found inside the WebRTC source file:
 * media/base/adapted_video_track_source.h
 */

namespace UE::PixelStreaming
{
	// Base class for sources which needs video adaptation, e.g., video
	// capture sources. Sinks must be added and removed on one and only
	// one thread, while AdaptFrame and OnFrame may be called on any
	// thread.
	class FAdaptedVideoTrackSource : public webrtc::Notifier<webrtc::VideoTrackSourceInterface>
	{
	public:
		FAdaptedVideoTrackSource();
		~FAdaptedVideoTrackSource() override;

		// Implements rtc::VideoSourceInterface.
		void AddOrUpdateSink(rtc::VideoSinkInterface<webrtc::VideoFrame>* sink,
			const rtc::VideoSinkWants& wants) override;
		void RemoveSink(rtc::VideoSinkInterface<webrtc::VideoFrame>* sink) override;

		virtual void AddRef() const override { RefCount.IncRef(); }
		virtual rtc::RefCountReleaseStatus Release() const override
		{
			const rtc::RefCountReleaseStatus Status = RefCount.DecRef();
			if (Status == rtc::RefCountReleaseStatus::kDroppedLastRef)
			{
				delete this;
			}
			return Status;
		}

	protected:
		// Allows derived classes to initialize |video_adapter_| with a custom
		// alignment.
		explicit FAdaptedVideoTrackSource(int required_alignment);
		// Checks the apply_rotation() flag. If the frame needs rotation, and it is a
		// plain memory frame, it is rotated. Subclasses producing native frames must
		// handle apply_rotation() themselves.
		void OnFrame(const webrtc::VideoFrame& frame);

		// Reports the appropriate frame size after adaptation. Returns true
		// if a frame is wanted. Returns false if there are no interested
		// sinks, or if the VideoAdapter decides to drop the frame.
		bool AdaptFrame(int width,
			int height,
			int64_t time_us,
			int* out_width,
			int* out_height,
			int* crop_width,
			int* crop_height,
			int* crop_x,
			int* crop_y);

		// Returns the current value of the apply_rotation flag, derived
		// from the VideoSinkWants of registered sinks. The value is derived
		// from sinks' wants, in AddOrUpdateSink and RemoveSink. Beware that
		// when using this method from a different thread, the value may
		// become stale before it is used.
		bool apply_rotation();

		cricket::VideoAdapter* video_adapter() { return &video_adapter_; }

	private:
		mutable webrtc::webrtc_impl::RefCounter RefCount{ 0 };

		// Part of VideoTrackSourceInterface.
		bool GetStats(Stats* stats) override;

		void OnSinkWantsChanged(const rtc::VideoSinkWants& wants);

		// Encoded sinks not implemented for AdaptedVideoTrackSource.
		bool SupportsEncodedOutput() const override { return false; }
		void GenerateKeyFrame() override {}
		void AddEncodedSink(rtc::VideoSinkInterface<webrtc::RecordableEncodedFrame>* sink) override {}
		void RemoveEncodedSink(rtc::VideoSinkInterface<webrtc::RecordableEncodedFrame>* sink) override {}

		cricket::VideoAdapter video_adapter_;
		webrtc::Mutex stats_mutex_;
		absl::optional<Stats> stats_ RTC_GUARDED_BY(stats_mutex_);

		rtc::VideoBroadcaster broadcaster_;
	};

} // namespace UE::PixelStreaming
