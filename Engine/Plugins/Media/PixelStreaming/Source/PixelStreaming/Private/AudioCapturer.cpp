// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCapturer.h"
#include "AudioMixerDevice.h"
#include "SampleBuffer.h"
#include "PixelStreamingTrace.h"
#include "Engine/GameEngine.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreamingAudioCapturer, Log, All);
DEFINE_LOG_CATEGORY(LogPixelStreamingAudioCapturer);

// These are copied from webrtc internals
#define CHECKinitialized_() \
	{                       \
		if (!bInitialized)  \
		{                   \
			return -1;      \
		};                  \
	}
#define CHECKinitialized__BOOL() \
	{                            \
		if (!bInitialized)       \
		{                        \
			return false;        \
		};                       \
	}

namespace UE::PixelStreaming
{
	constexpr int FAudioCapturer::SampleRate;
	constexpr int FAudioCapturer::NumChannels;

	FAudioCapturerListener::FAudioCapturerListener(FAudioCapturer& InParent)
		: Parent(&InParent)
	{
	}

	void FAudioCapturerListener::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 InNumChannels, const int32 InSampleRate, double AudioClock)
	{
		check(Parent);
		Parent->OnNewSubmixBuffer(OwningSubmix, AudioData, NumSamples, InNumChannels, InSampleRate, AudioClock);
	}

	FAudioCapturer::FAudioCapturer()
		: Listener(MakeShared<FAudioCapturerListener>(*this))
	{
	}

	void FAudioCapturer::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 InNumChannels, const int32 InSampleRate, double AudioClock)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming FAudioCapturer::OnNewSubmixBuffer", PixelStreamingChannel);
		if (!(bInitialized && bRecordingInitialized))
		{
			return;
		}

		// Only 48000hz supported for now
		if (InSampleRate != SampleRate)
		{
			// Only report the problem once
			if (!bFormatChecked)
			{
				bFormatChecked = true;
				UE_LOGF(LogPixelStreamingAudioCapturer, Error, "Audio samplerate needs to be 48000hz");
			}
			return;
		}

		UE_LOGF(LogPixelStreamingAudioCapturer, VeryVerbose, "captured %d samples, %dc, %dHz", NumSamples, NumChannels, SampleRate);

		Audio::TSampleBuffer<float> Buffer(AudioData, NumSamples, InNumChannels, SampleRate);
		// Mix to stereo if required, since PixelStreaming only accepts stereo at the moment
		if (Buffer.GetNumChannels() != NumChannels)
		{
			Buffer.MixBufferToChannels(NumChannels);
		}

		// Convert to signed PCM 16-bits
		PCM16.Reset(Buffer.GetNumSamples());
		PCM16.AddZeroed(Buffer.GetNumSamples());
		const float* Ptr = reinterpret_cast<const float*>(Buffer.GetData());
		for (int16& S : PCM16)
		{
			int32 N = *Ptr >= 0 ? *Ptr * int32(MAX_int16) : *Ptr * (int32(MAX_int16) + 1);
			S = static_cast<int16>(FMath::Clamp(N, int32(MIN_int16), int32(MAX_int16)));
			Ptr++;
		}

		RecordingBuffer.Append(reinterpret_cast<const uint8*>(PCM16.GetData()), PCM16.Num() * sizeof(PCM16[0]));
		int BytesPer10Ms = (SampleRate * NumChannels * static_cast<int>(sizeof(uint16))) / 100;

		// Feed in 10ms chunks
		while (RecordingBuffer.Num() >= BytesPer10Ms)
		{
			{
				FScopeLock Lock(&DeviceBufferCS);
				if (DeviceBuffer)
				{
					DeviceBuffer->SetRecordedBuffer(RecordingBuffer.GetData(), BytesPer10Ms / (sizeof(uint16) * NumChannels));
					DeviceBuffer->DeliverRecordedData();
					UE_LOGF(LogPixelStreamingAudioCapturer, VeryVerbose, "passed %d bytes", BytesPer10Ms);
				}
			}

			RecordingBuffer.RemoveAt(0, BytesPer10Ms, EAllowShrinking::No);
		}
	}

	int32 FAudioCapturer::ActiveAudioLayer(AudioLayer* audioLayer) const
	{
		*audioLayer = AudioDeviceModule::kDummyAudio;
		return 0;
	}

	int32 FAudioCapturer::RegisterAudioCallback(webrtc::AudioTransport* audioCallback)
	{
		FScopeLock Lock(&DeviceBufferCS);
		DeviceBuffer->RegisterAudioCallback(audioCallback);

		return 0;
	}

	int32 FAudioCapturer::Init()
	{
		if (bInitialized)
		{
			return 0;
		}

		{
			FScopeLock Lock(&DeviceBufferCS);

			m_taskQueueFactory = webrtc::CreateDefaultTaskQueueFactory();
			DeviceBuffer = MakeUnique<webrtc::AudioDeviceBuffer>(m_taskQueueFactory.get());
		}

		FAudioDeviceHandle AudioDevice = GEngine ? GEngine->GetMainAudioDevice() : FAudioDeviceHandle();
		if (AudioDevice)
		{
			AudioDevice->RegisterSubmixBufferListener(Listener->AsShared(), AudioDevice->GetMainSubmixObject());
		}

		bInitialized = true;
		UE_LOGF(LogPixelStreamingAudioCapturer, Verbose, "Init");
		return 0;
	}

	int32 FAudioCapturer::Terminate()
	{
		if (!bInitialized)
		{
			return 0;
		}

		// UE-218577: GetMainAudioDevice may have changed since Init was called. Introduce variable (not straightforward due to multi-thread destruction of FAudioCapturer, see ticket).
		FAudioDeviceHandle AudioDevice = GEngine ? GEngine->GetMainAudioDevice() : FAudioDeviceHandle();
		if (AudioDevice)
		{
			AudioDevice->UnregisterSubmixBufferListener(Listener->AsShared(), AudioDevice->GetMainSubmixObject());
		}
		
		bInitialized = false;
		{
			FScopeLock Lock(&DeviceBufferCS);
			DeviceBuffer.Reset();
		}

		UE_LOGF(LogPixelStreamingAudioCapturer, Verbose, "Terminate");
		return 0;
	}

	bool FAudioCapturer::Initialized() const
	{
		return bInitialized;
	}

	int16 FAudioCapturer::PlayoutDevices()
	{
		CHECKinitialized_();
		return -1;
	}

	int16 FAudioCapturer::RecordingDevices()
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::PlayoutDeviceName(
		uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize])
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::RecordingDeviceName(
		uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize])
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::SetPlayoutDevice(uint16 index)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioCapturer::SetPlayoutDevice(WindowsDeviceType device)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioCapturer::SetRecordingDevice(uint16 index)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioCapturer::SetRecordingDevice(WindowsDeviceType device)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioCapturer::PlayoutIsAvailable(bool* available)
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::InitPlayout()
	{
		CHECKinitialized_();
		return -1;
	}

	bool FAudioCapturer::PlayoutIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return false;
	}

	int32 FAudioCapturer::RecordingIsAvailable(bool* available)
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::InitRecording()
	{
		CHECKinitialized_();

		{
			FScopeLock Lock(&DeviceBufferCS);
			// #Audio : Allow dynamic values for samplerate and/or channels ,
			// or receive those from UnrealEngine ?
			DeviceBuffer->SetRecordingSampleRate(SampleRate);
			DeviceBuffer->SetRecordingChannels(NumChannels);
		}

		bRecordingInitialized = true;
		return 0;
	}

	bool FAudioCapturer::RecordingIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return bRecordingInitialized == true;
	}

	int32 FAudioCapturer::StartPlayout()
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::StopPlayout()
	{
		CHECKinitialized_();
		return -1;
	}

	bool FAudioCapturer::Playing() const
	{
		CHECKinitialized__BOOL();
		return false;
	}

	int32 FAudioCapturer::StartRecording()
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::StopRecording()
	{
		CHECKinitialized_();
		return -1;
	}

	bool FAudioCapturer::Recording() const
	{
		CHECKinitialized__BOOL();
		return bRecordingInitialized;
	}

	int32 FAudioCapturer::InitSpeaker()
	{
		CHECKinitialized_();
		return -1;
	}

	bool FAudioCapturer::SpeakerIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return false;
	}

	int32 FAudioCapturer::InitMicrophone()
	{
		CHECKinitialized_();
		return 0;
	}

	bool FAudioCapturer::MicrophoneIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return true;
	}

	int32 FAudioCapturer::StereoPlayoutIsAvailable(bool* available) const
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::SetStereoPlayout(bool enable)
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::StereoPlayout(bool* enabled) const
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioCapturer::StereoRecordingIsAvailable(bool* available) const
	{
		CHECKinitialized_();
		*available = true;
		return 0;
	}

	int32 FAudioCapturer::SetStereoRecording(bool enable)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioCapturer::StereoRecording(bool* enabled) const
	{
		CHECKinitialized_();
		*enabled = true;
		return 0;
	}
} // namespace UE::PixelStreaming
