// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebRTCIncludes.h"
#include "HAL/CriticalSection.h"
#include "HAL/ThreadSafeBool.h"
#include "ISubmixBufferListener.h"

namespace UE::PixelStreaming 
{
	class FAudioCapturer;

	struct FAudioCapturerListener : public ISubmixBufferListener
	{
		FAudioCapturerListener(FAudioCapturer& InParent);

		// ISubmixBufferListener interface
		virtual void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock) override;

		FAudioCapturer* Parent = nullptr;
	};

	// This class is only used if -PixelStreamingWebRTCUseLegacyAudioDevice or related CVar are used.
	// This class will likely be removed in the future once the new ADM is confirmed stable.
	class FAudioCapturer : public webrtc::AudioDeviceModule
	{
	public:
		FAudioCapturer();

	private:
		friend struct FAudioCapturerListener;

		void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock);

		//
		// webrtc::AudioDeviceModule interface
		//
		int32 ActiveAudioLayer(AudioLayer* audioLayer) const override;
		int32 RegisterAudioCallback(webrtc::AudioTransport* audioCallback) override;

		// Main initialization and termination
		int32 Init() override;
		int32 Terminate() override;
		bool Initialized() const override;

		// Device enumeration
		int16 PlayoutDevices() override;
		int16 RecordingDevices() override;
		int32 PlayoutDeviceName(uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize]) override;
		int32 RecordingDeviceName(uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize]) override;

		// Device selection
		int32 SetPlayoutDevice(uint16 index) override;
		int32 SetPlayoutDevice(WindowsDeviceType device) override;
		int32 SetRecordingDevice(uint16 index) override;
		int32 SetRecordingDevice(WindowsDeviceType device) override;

		// Audio transport initialization
		int32 PlayoutIsAvailable(bool* available) override;
		int32 InitPlayout() override;
		bool PlayoutIsInitialized() const override;
		int32 RecordingIsAvailable(bool* available) override;
		int32 InitRecording() override;
		bool RecordingIsInitialized() const override;

		// Audio transport control
		virtual int32 StartPlayout() override;
		virtual int32 StopPlayout() override;
		virtual bool Playing() const override;
		virtual int32 StartRecording() override;
		virtual int32 StopRecording() override;
		virtual bool Recording() const override;

		// Audio mixer initialization
		virtual int32 InitSpeaker() override;
		virtual bool SpeakerIsInitialized() const override;
		virtual int32 InitMicrophone() override;
		virtual bool MicrophoneIsInitialized() const override;

		// Speaker volume controls
		virtual int32 SpeakerVolumeIsAvailable(bool* available) override
		{
			return -1;
		}
		virtual int32 SetSpeakerVolume(uint32 volume) override
		{
			return -1;
		}
		virtual int32 SpeakerVolume(uint32* volume) const override
		{
			return -1;
		}
		virtual int32 MaxSpeakerVolume(uint32* maxVolume) const override
		{
			return -1;
		}
		virtual int32 MinSpeakerVolume(uint32* minVolume) const override
		{
			return -1;
		}

		// Microphone volume controls
		virtual int32 MicrophoneVolumeIsAvailable(bool* available) override
		{
			return -1;
		}
		virtual int32 SetMicrophoneVolume(uint32 volume) override
		{
			return -1;
		}
		virtual int32 MicrophoneVolume(uint32* volume) const override
		{
			return -1;
		}
		virtual int32 MaxMicrophoneVolume(uint32* maxVolume) const override
		{
			return -1;
		}
		virtual int32 MinMicrophoneVolume(uint32* minVolume) const override
		{
			return -1;
		}

		// Speaker mute control
		virtual int32 SpeakerMuteIsAvailable(bool* available) override
		{
			return -1;
		}
		virtual int32 SetSpeakerMute(bool enable) override
		{
			return -1;
		}
		virtual int32 SpeakerMute(bool* enabled) const override
		{
			return -1;
		}

		// Microphone mute control
		virtual int32 MicrophoneMuteIsAvailable(bool* available) override
		{
			return -1;
		}
		virtual int32 SetMicrophoneMute(bool enable) override
		{
			return -1;
		}
		virtual int32 MicrophoneMute(bool* enabled) const override
		{
			return -1;
		}

		// Stereo support
		virtual int32 StereoPlayoutIsAvailable(bool* available) const override;
		virtual int32 SetStereoPlayout(bool enable) override;
		virtual int32 StereoPlayout(bool* enabled) const override;
		virtual int32 StereoRecordingIsAvailable(bool* available) const override;
		virtual int32 SetStereoRecording(bool enable) override;
		virtual int32 StereoRecording(bool* enabled) const override;

		// Playout delay
		virtual int32 PlayoutDelay(uint16* delayMS) const override
		{
			return -1;
		}

		// Only supported on Android.
		virtual bool BuiltInAECIsAvailable() const override
		{
			return false;
		}
		virtual bool BuiltInAGCIsAvailable() const override
		{
			return false;
		}
		virtual bool BuiltInNSIsAvailable() const override
		{
			return false;
		}

		// Enables the built-in audio effects. Only supported on Android.
		virtual int32 EnableBuiltInAEC(bool enable) override
		{
			return -1;
		}
		virtual int32 EnableBuiltInAGC(bool enable) override
		{
			return -1;
		}
		virtual int32 EnableBuiltInNS(bool enable) override
		{
			return -1;
		}

		FThreadSafeBool bInitialized = false;
		TArray<int16> PCM16;

		TUniquePtr<webrtc::AudioDeviceBuffer> DeviceBuffer;
		FCriticalSection DeviceBufferCS; // protects capturing audio and sending it to WebRTC from simultaneous termination

		TArray<uint8_t> RecordingBuffer;
		int RecordingBufferSize = 0;

		FThreadSafeBool bRecordingInitialized = false;
		static constexpr int SampleRate = 48000;
		static constexpr int NumChannels = 2;

		bool bFormatChecked = false;

		std::unique_ptr<webrtc::TaskQueueFactory> m_taskQueueFactory;
		TSharedPtr<FAudioCapturerListener, ESPMode::ThreadSafe> Listener;
	};
} // namespace UE::PixelStreaming