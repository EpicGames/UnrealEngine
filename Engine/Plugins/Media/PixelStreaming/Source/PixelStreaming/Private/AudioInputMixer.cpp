// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioInputMixer.h"
#include "Settings.h"
#include "SampleBuffer.h"
#include "PixelStreamingTrace.h"
#include "PixelStreamingPrivate.h"

namespace UE::PixelStreaming
{

	/* ---------------------------- FAudioInputMixer ---------------------------- */

	FAudioInputMixer::FAudioInputMixer()
		: Mixers(MakeShared<FMixers>())
		, bIsMixing(false)
	{
	}

	FAudioInputMixer::~FAudioInputMixer()
	{
		StopMixing();
	}

	TSharedPtr<FAudioInput> FAudioInputMixer::CreateInput()
	{
		float Gain = Settings::CVarPixelStreamingWebRTCAudioGain.GetValueOnAnyThread();
		return MakeShared<FAudioInput>(*this, Mixers->GetMaxBufferSize(), Gain);
	}

	void FAudioInputMixer::DisconnectInput(TSharedPtr<FAudioInput> AudioInput)
	{
		if (!AudioInput)
		{
			return;
		}
		if (!Mixers.IsValid())
		{
			return;
		}

		Mixers->Mixer.RemovePatch(AudioInput->GetPatchInput());
	}

	void FAudioInputMixer::RegisterAudioCallback(webrtc::AudioTransport* InAudioCallback)
	{
		Runnable = MakeUnique<FMixerRunnable>(InAudioCallback, Mixers);
	}

	void FAudioInputMixer::StartMixing()
	{
		if (!Runnable.IsValid())
		{
			return;
		}
		MixerThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(Runnable.Get(), TEXT("Pixel Streaming Audio Mixer")));
		bIsMixing = true;
	}

	void FAudioInputMixer::StopMixing()
	{
		if (MixerThread.IsValid() && bIsMixing)
		{
			MixerThread->Kill();
			bIsMixing = false;
		}
		if (Runnable.IsValid())
		{
			Runnable->Stop();
		}
	}

	/* ---------------------------- FMixerRunnable ---------------------------- */

	FMixerRunnable::FMixerRunnable(webrtc::AudioTransport* InAudioCallback, TSharedPtr<FMixers> InMixers)
		: AudioCallback(InAudioCallback)
		, Mixers(InMixers)
	{
		const int NMixingSamples = Mixers->GetMaxBufferSize();
		MixingBuffer.SetNumUninitialized(NMixingSamples);
	}

	bool FMixerRunnable::Init()
	{
		return true;
	}

	uint32 FMixerRunnable::Run()
	{
		bIsRunning = true;

		while (bIsRunning)
		{
			Tick();

			// Sleep every 1ms
			FPlatformProcess::Sleep(0.01f);
		}

		return 0;
	}

	void FMixerRunnable::Stop()
	{
		bIsRunning = false;
	}

	void FMixerRunnable::Exit()
	{
		bIsRunning = false;
	}

	void FMixerRunnable::Tick()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming FAudioInputMixer::FMixerRunnable::Tick", PixelStreamingChannel);
		if (!Mixers.IsValid())
		{
			return;
		}

		// 4 samples is the absolute minimum required for mixing
		if (MixingBuffer.Num() < 4)
		{
			return;
		}

		const uint32 TargetNumSamples = Mixers->SampleRate * Mixers->NumChannels * 0.01f; // 10ms of audio is what webrtc likes
		int32 NSamplesPopped = Mixers->Mixer.PopAudio(MixingBuffer.GetData(), TargetNumSamples, false /* bUseLatestAudio */);

		if (NSamplesPopped == 0)
		{
			return;
		}

		const size_t Frames = NSamplesPopped / Mixers->NumChannels;
		const size_t BytesPerFrame = Mixers->NumChannels * sizeof(int16_t);

		// WebRTC wants audio as PCM16, so we need to do a conversion from float to int16
		Audio::TSampleBuffer<int16> PCM16Buffer(MixingBuffer, Mixers->NumChannels, Mixers->SampleRate);

		AudioCallback->RecordedDataIsAvailable(
			PCM16Buffer.GetData(),
			Frames,
			BytesPerFrame,
			Mixers->NumChannels,
			Mixers->SampleRate,
			0,
			0,
			Mixers->VolumeLevel,
			false,
			Mixers->VolumeLevel);
	}

	/* ---------------------------- FAudioInput ---------------------------- */

	FAudioInput::FAudioInput(FAudioInputMixer& InMixer, int32 MaxSamples, float InGain)
	{
		PatchInput = InMixer.Mixers->Mixer.AddNewInput(MaxSamples, InGain);
		NumChannels = InMixer.Mixers->NumChannels;
		SampleRate = InMixer.Mixers->SampleRate;
	}
	
	void FAudioInput::PushAudio(const float* InBuffer, int32 NumSamples, int32 InNumChannels, int32 InSampleRate)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming FAudioInputMixer::PushAudio", PixelStreamingChannel);
		TArray<float> AudioBuffer;

		if(SampleRate != InSampleRate)
		{
			float SampleRateConversionRatio = static_cast<float>(SampleRate) / static_cast<float>(InSampleRate);
			UE_LOGF(LogPixelStreaming, VeryVerbose, "(FAudioInput) Sample rate conversion ratio is : %f.", SampleRateConversionRatio);
			Resampler.Init(Audio::EResamplingMethod::Linear, SampleRateConversionRatio, InNumChannels);

			int32 NumOriginalSamples = NumSamples / InNumChannels;
			int32 NumConvertedSamples = FMath::CeilToInt(NumSamples / InNumChannels * SampleRateConversionRatio);
			int32 OutputSamples = INDEX_NONE;
			AudioBuffer.AddZeroed(NumConvertedSamples * InNumChannels);

			// Perform the sample rate conversion
			int32 ErrorCode = Resampler.ProcessAudio(const_cast<float*>(InBuffer), NumOriginalSamples, false, AudioBuffer.GetData(), NumConvertedSamples, OutputSamples);
			verifyf(OutputSamples <= NumConvertedSamples, TEXT("OutputSamples > NumConvertedSamples"));
			if (ErrorCode != 0)
			{
				UE_LOGF(LogPixelStreaming, Warning, "(FAudioInput) Problem occured resampling audio data. Code: %d", ErrorCode);
				return;
			}
		}
		else
		{
			AudioBuffer.AddZeroed(NumSamples);
			FMemory::Memcpy(AudioBuffer.GetData(), InBuffer, AudioBuffer.Num() * sizeof(float));
		}


		// Mix to our target number of channels if the source does not already match.
		if (NumChannels != InNumChannels)
		{
			Audio::TSampleBuffer<float> Buffer(AudioBuffer.GetData(), AudioBuffer.Num(), InNumChannels, InSampleRate);
			Buffer.MixBufferToChannels(NumChannels);
			PatchInput.PushAudio(Buffer.GetData(), Buffer.GetNumSamples());
		}
		else
		{
			PatchInput.PushAudio(AudioBuffer.GetData(), AudioBuffer.Num());
		}
	}

} // namespace UE::PixelStreaming
