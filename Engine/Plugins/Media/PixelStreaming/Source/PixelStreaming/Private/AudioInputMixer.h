// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/MultithreadedPatching.h"
#include "AudioResampler.h"
#include "Containers/Array.h"
#include "IPixelStreamingAudioInput.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"

namespace webrtc
{
	class AudioTransport;
}

namespace UE::PixelStreaming
{

	struct FMixers
	{
		uint8 NumChannels = 2;
		uint32 SampleRate = 48000;
		float SampleSizeSeconds = 0.5f;
		uint32_t VolumeLevel = 14392; // This magic number is a default from WebRTC
		Audio::FPatchMixer Mixer;
		uint32 GetMaxBufferSize() const { return SampleRate * SampleSizeSeconds * NumChannels; }
	};

	/*
	 * Polls mixers for audio and passes it to the WebRTC callback on a fixed interval.
	 */
	class FMixerRunnable : public FRunnable, public FSingleThreadRunnable
	{
	public:
		FMixerRunnable(webrtc::AudioTransport* InAudioCallback, TSharedPtr<FMixers> InMixers);
		virtual ~FMixerRunnable() = default;

		// Begin FRunnable interface.
		virtual bool Init() override;
		virtual uint32 Run() override;
		virtual void Stop() override;
		virtual void Exit() override;
		virtual void Tick() override;
		virtual FSingleThreadRunnable* GetSingleThreadInterface() override { return this; };
		// End FRunnable interface

	private:
		bool bIsRunning;
		webrtc::AudioTransport* AudioCallback;
		TSharedPtr<FMixers> Mixers;
		Audio::VectorOps::FAlignedFloatBuffer MixingBuffer;
	};

	/*
	 * Allows pushing abitrary audio into Pixel Streaming, sample rate is assumed to be 48KHz.
	 * Note: All `PushAudio` methods should be called from the same single thread. Pushing from multiple threads is unsupported.
	 */
	class FAudioInput : public IPixelStreamingAudioInput
	{
	public:
		FAudioInput(class FAudioInputMixer& InMixer, int32 MaxSamples, float InGain);
		virtual ~FAudioInput() = default;

		/* Pushes audio, which must match the underlying sample rate/num channels of the mixer. */
		void PushAudio(const float* InBuffer, int32 NumSamples, int32 NumChannels, int32 SampleRate) override;

		const Audio::FPatchInput& GetPatchInput() const { return PatchInput; }

	private:
		Audio::FResampler Resampler;
		Audio::FPatchInput PatchInput;
		uint8 NumChannels;
		uint32 SampleRate;
	};

	/*
	 * Mixes audio from multiple inputs into stereo.
	 */
	class FAudioInputMixer
	{
	public:
		FAudioInputMixer();
		virtual ~FAudioInputMixer();
		TSharedPtr<FAudioInput> CreateInput();
		void DisconnectInput(TSharedPtr<FAudioInput> AudioInput);
		void RegisterAudioCallback(webrtc::AudioTransport* AudioCallback);
		void StartMixing();
		void StopMixing();

	private:
		TSharedPtr<FMixers> Mixers;
		bool bIsMixing;

		TUniquePtr<FRunnableThread> MixerThread;
		TUniquePtr<FMixerRunnable> Runnable;

		// So it can access the `Mixers` private member.
		friend class FAudioInput;
	};

} // namespace UE::PixelStreaming