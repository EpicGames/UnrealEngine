// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPlayoutRequester.h"
#include "PixelStreamingPrivate.h"
#include "Misc/ScopeLock.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "WebRTCIncludes.h"
#include "PixelStreamingTrace.h"

namespace UE::PixelStreaming
{
	FAudioPlayoutRequester::FAudioPlayoutRequester()
		: bIsPlayoutInitialised(false)
		, bIsPlaying(false)
		, SampleRate(48000) // webrtc will mix all sources into one big buffer with this sample rate
		, NumChannels(2)	// webrtc will mix all sources into one big buffer with this many channels
		, RequesterRunnable()
		, RequesterThread()
		, AudioCallback(nullptr)
		, PlayoutCriticalSection()
		, PlayoutBuffer()
	{
	}

	void FAudioPlayoutRequester::Uninitialise()
	{
		StopPlayout();
		AudioCallback = nullptr;
		bIsPlayoutInitialised = false;
		PlayoutBuffer.Empty();
	}

	void FAudioPlayoutRequester::InitPlayout() { bIsPlayoutInitialised = true; }

	void FAudioPlayoutRequester::StartPlayout()
	{
		if (PlayoutIsInitialized() && !Playing())
		{
			TFunction<void()> RequesterFunc = [this]() {
				FScopeLock Lock(&PlayoutCriticalSection);

				// Only request audio if audio callback is valid
				if (!AudioCallback)
				{
					return;
				}

				// Our intention is to request samples at some fixed interval (i.e 10ms)
				int32 NSamplesPerChannel =
					(SampleRate) / (1000 / FAudioPlayoutRequester::RequestIntervalMs);
				const size_t BytesPerFrame = NumChannels * sizeof(int16_t);

				// Ensure buffer has the total number of samples we need for all audio
				// frames
				PlayoutBuffer.Reserve(NSamplesPerChannel * NumChannels);

				size_t OutNSamples = 0;
				int64_t ElapsedTimeMs = -1;
				int64_t NtpTimeMs = -1;

				// Note this is mixed result of all audio sources, which in turn triggers
				// the sinks for each audio source to be called. For example, if you had 3
				// audio sources in the browser sending 16kHz mono they would all be mixed
				// down into the number of channels and sample rate specified below.
				// However, for listening to each audio source prior to mixing refer to
				// UE::PixelStreaming::FAudioSink.
				uint32_t Result = AudioCallback->NeedMorePlayData(
					NSamplesPerChannel, BytesPerFrame, NumChannels, SampleRate,
					PlayoutBuffer.GetData(), OutNSamples, &ElapsedTimeMs, &NtpTimeMs);

				if (Result != 0)
				{
					UE_LOGF(LogPixelStreaming, Error, "NeedMorePlayData return non-zero result indicating an error");
				}
			};

			RequesterThread.Reset(nullptr);
			RequesterRunnable =
				MakeUnique<FAudioPlayoutRequester::Runnable>(RequesterFunc);
			RequesterThread.Reset(FRunnableThread::Create(
				RequesterRunnable.Get(),
				TEXT("Pixel Streaming WebRTC Audio Requester")));
			bIsPlaying = true;
		}
	}

	void FAudioPlayoutRequester::StopPlayout()
	{
		if (PlayoutIsInitialized() && Playing())
		{
			RequesterRunnable->Stop();
			bIsPlaying = false;
		}
	}

	bool FAudioPlayoutRequester::Playing() const { return bIsPlaying; }

	bool FAudioPlayoutRequester::PlayoutIsInitialized() const
	{
		return bIsPlayoutInitialised;
	}

	void FAudioPlayoutRequester::RegisterAudioCallback(
		webrtc::AudioTransport* AudioCb)
	{
		FScopeLock Lock(&PlayoutCriticalSection);
		AudioCallback = AudioCb;
	}

	//--FAudioRequester::Runnable--

	FAudioPlayoutRequester::Runnable::Runnable(TFunction<void()> RequestPlayoutFunc)
		: bIsRunning(false)
		, LastAudioRequestTimeMs(0)
		, RequestPlayoutFunc(RequestPlayoutFunc)
	{
	}

	bool FAudioPlayoutRequester::Runnable::Init()
	{
		return RequestPlayoutFunc != nullptr;
	}

	uint32 FAudioPlayoutRequester::Runnable::Run()
	{
		LastAudioRequestTimeMs = rtc::TimeMillis();
		bIsRunning = true;

		// Request audio in a loop until this boolean is toggled off.
		while (bIsRunning)
		{
			int64_t Now = rtc::TimeMillis();
			int64_t DeltaMs = Now - LastAudioRequestTimeMs;

			// Check if the 10ms delta has elapsed, if it has not, then sleep the
			// remaining
			if (DeltaMs < FAudioPlayoutRequester::RequestIntervalMs)
			{
				int64_t SleepTimeMs = FAudioPlayoutRequester::RequestIntervalMs - DeltaMs;
				float SleepTimeSecs = (float)SleepTimeMs / 1000.0f;
				FPlatformProcess::Sleep(SleepTimeSecs);
				continue;
			}

			// Update request time to now seeing as the 10ms delta has elapsed
			LastAudioRequestTimeMs = Now;

			{
				// Actually request playout
				TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming FAudioPlayoutRequester::Run RequestPlayoutFunc", PixelStreamingChannel);
				RequestPlayoutFunc();
			}
		}

		return 0;
	}

	void FAudioPlayoutRequester::Runnable::Stop() { bIsRunning = false; }

	void FAudioPlayoutRequester::Runnable::Exit()
	{
		bIsRunning = false;
		RequestPlayoutFunc = nullptr;
	}
} // namespace UE::PixelStreaming
