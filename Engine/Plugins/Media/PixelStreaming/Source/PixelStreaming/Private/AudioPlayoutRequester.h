// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/CriticalSection.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

namespace webrtc
{
	class AudioTransport;
}

namespace UE::PixelStreaming
{
	// Requests audio from WebRTC at a regular interval (10ms)
	// This is required so that WebRTC audio sinks actually have
	// some audio data for their sinks. Without this WebRTC assumes
	// there is no demand for audio and does not populate the sinks.
	class FAudioPlayoutRequester
	{
	public:
		class Runnable : public FRunnable
		{
		public:
			Runnable(TFunction<void()> RequestPlayoutFunc);
			virtual ~Runnable() = default;

			// Begin FRunnable interface.
			virtual bool Init() override;
			virtual uint32 Run() override;
			virtual void Stop() override;
			virtual void Exit() override;
			// End FRunnable interface

		private:
			bool bIsRunning;
			int64_t LastAudioRequestTimeMs;
			TFunction<void()> RequestPlayoutFunc;
		};

		FAudioPlayoutRequester();
		virtual ~FAudioPlayoutRequester() = default;

		virtual void InitPlayout();
		virtual void StartPlayout();
		virtual void StopPlayout();
		virtual bool Playing() const;
		virtual bool PlayoutIsInitialized() const;
		virtual void Uninitialise();
		virtual void RegisterAudioCallback(webrtc::AudioTransport* AudioCallback);

	public:
		static int16_t const RequestIntervalMs = 10;

	private:
		FThreadSafeBool bIsPlayoutInitialised;
		FThreadSafeBool bIsPlaying;
		uint32 SampleRate;
		uint8 NumChannels;
		TUniquePtr<FAudioPlayoutRequester::Runnable> RequesterRunnable;
		TUniquePtr<FRunnableThread> RequesterThread;
		webrtc::AudioTransport* AudioCallback;
		FCriticalSection PlayoutCriticalSection;
		TArray<int16_t> PlayoutBuffer;
	};
} // namespace UE::PixelStreaming
