// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSink.h"
#include "PixelStreamingTrace.h"

namespace UE::PixelStreaming
{
	FAudioSink::~FAudioSink()
	{
		for (IPixelStreamingAudioConsumer* AudioConsumer : AudioConsumers)
		{
			AudioConsumer->OnConsumerRemoved();
		}
	}

	void FAudioSink::OnData(const void* audio_data, int bits_per_sample, int sample_rate, size_t number_of_channels, size_t number_of_frames, absl::optional<int64_t> absolute_capture_timestamp_ms)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming FAudioSink::OnData", PixelStreamingChannel);
		// This data is populated from the internals of WebRTC, basically each audio track sent from the browser has its RTP audio source received and decoded.
		// The sample rate and number of channels here has absolutely no relationship with PixelStreamingAudioDeviceModule.
		// The sample rate and number of channels here is determined adaptively by WebRTC's NetEQ class that selects sample rate/number of channels
		// based on network conditions and other factors.

		if (!HasAudioConsumers())
		{
			return;
		}

		// Iterate audio consumers and pass this data to their buffers
		for (IPixelStreamingAudioConsumer* AudioConsumer : AudioConsumers)
		{
			AudioConsumer->ConsumeRawPCM(static_cast<const int16_t*>(audio_data), sample_rate, number_of_channels, number_of_frames);
		}

		return;
	}

	void FAudioSink::AddAudioConsumer(IPixelStreamingAudioConsumer* AudioConsumer)
	{
		bool bAlreadyInSet = false;
		AudioConsumers.Add(AudioConsumer, &bAlreadyInSet);
		if (!bAlreadyInSet)
		{
			AudioConsumer->OnConsumerAdded();
		}
	}

	void FAudioSink::RemoveAudioConsumer(IPixelStreamingAudioConsumer* AudioConsumer)
	{
		if (AudioConsumers.Remove(AudioConsumer) > 0)
		{
			AudioConsumer->OnConsumerRemoved();
		}
	}

	bool FAudioSink::HasAudioConsumers()
	{
		return AudioConsumers.Num() > 0;
	}
} // namespace UE::PixelStreaming
