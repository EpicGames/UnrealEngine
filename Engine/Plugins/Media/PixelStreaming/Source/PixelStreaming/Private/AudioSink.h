// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WebRTCIncludes.h"
#include "IPixelStreamingAudioSink.h"
#include "IPixelStreamingAudioConsumer.h"
#include "Containers/Set.h"

namespace UE::PixelStreaming 
{
	// This is a PixelStreaming sink AND a WebRTC sink.
	// It collects audio coming in from a WebRTC audio source and passes into into UE's audio system.
	class FAudioSink : public IPixelStreamingAudioSink
	{
	public:
		FAudioSink()
			: AudioConsumers(){};

		// Note: destructor will call destroy on any attached audio consumers
		virtual ~FAudioSink();

		// Begin AudioTrackSinkInterface
		void OnData(const void* audio_data, int bits_per_sample, int sample_rate, size_t number_of_channels, size_t number_of_frames, absl::optional<int64_t> absolute_capture_timestamp_ms) override;
		// End AudioTrackSinkInterface

		// Begin IPixelStreamingAudioSink
		void AddAudioConsumer(IPixelStreamingAudioConsumer* AudioConsumer) override;
		void RemoveAudioConsumer(IPixelStreamingAudioConsumer* AudioConsumer) override;
		bool HasAudioConsumers() override;
		// End IPixelStreamingAudioSink

	private:
		void UpdateAudioSettings(int InNumChannels, int InSampleRate);

	private:
		TSet<IPixelStreamingAudioConsumer*> AudioConsumers;
	};
} // namespace UE::PixelStreaming