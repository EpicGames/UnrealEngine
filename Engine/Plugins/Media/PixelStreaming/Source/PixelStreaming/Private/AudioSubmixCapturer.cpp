// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSubmixCapturer.h"

#include "AudioDevice.h"
#include "AudioInputMixer.h"
#include "Engine/Engine.h"
#include "PixelStreamingPrivate.h"
#include "SampleBuffer.h"
#include "Settings.h"

namespace UE::PixelStreaming
{
	FAudioSubmixCapturer::FAudioSubmixCapturer()
		: bInitialised(false)
		, bCapturing(false)
		, TargetSampleRate(48000)
		, TargetNumChannels(2)
		, bReportedSampleRateMismatch(false)
		, AudioCallback(nullptr)
		, VolumeLevel(FAudioSubmixCapturer::MaxVolumeLevel)
		, RecordingBuffer()
		, CriticalSection()
	{
	}

	uint32_t FAudioSubmixCapturer::GetVolume() const
	{
		return VolumeLevel;
	}

	void FAudioSubmixCapturer::SetVolume(uint32_t NewVolume)
	{
		VolumeLevel = NewVolume;
	}

	bool FAudioSubmixCapturer::Init()
	{
		FScopeLock Lock(&CriticalSection);

		// subscribe to audio data
		if (!GEngine)
		{
			bInitialised = false;
			return false;
		}

		// already initialised
		if (bInitialised)
		{
			return true;
		}

		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
		if (!AudioDevice)
		{
			UE_LOGF(LogPixelStreaming, Warning, "No audio device");
			bInitialised = false;
			return false;
		}

		AudioDevice->RegisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
		bInitialised = true;
		return true;
	}

	const FString& FAudioSubmixCapturer::GetListenerName() const
	{
		static const FString ListenerName = TEXT("PixelStreaming:SubmixCapturer");
		return ListenerName;
	}

	void FAudioSubmixCapturer::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix,
		float* AudioData, int32 NumSamples,
		int32 NumChannels,
		const int32 SampleRate,
		double AudioClock)
	{
		FScopeLock Lock(&CriticalSection);

		if (!bInitialised || !bCapturing)
		{
			return;
		}

		// No point doing anything with UE audio if the callback from WebRTC has not
		// been set yet.
		if (AudioCallback == nullptr)
		{
			return;
		}

		// Check if the sample rate from UE matches our target sample rate
		if (TargetSampleRate != SampleRate)
		{
			// Only report the problem once
			if (!bReportedSampleRateMismatch)
			{
				bReportedSampleRateMismatch = true;
				UE_LOGF(LogPixelStreaming, Error, "Audio sample rate mismatch. Expected: %d | Actual: %d", TargetSampleRate, SampleRate);
			}
			return;
		}

		UE_LOGF(LogPixelStreaming, VeryVerbose, "captured %d samples, %dc, %dHz", NumSamples, NumChannels, SampleRate);

		// We got passed a patch to send the audio to instead of directly to WebRTC
		if (AudioMixerPatchInput.IsValid())
		{
			AudioMixerPatchInput->PushAudio(AudioData, NumSamples, NumChannels, SampleRate);
			return;
		}

		// Note: TSampleBuffer takes in AudioData as float* and internally converts to int16
		Audio::TSampleBuffer<int16> Buffer(AudioData, NumSamples, NumChannels, SampleRate);

		// Ensure we end up with our target number of channels
		if (NumChannels != TargetNumChannels)
		{
			Buffer.MixBufferToChannels(TargetNumChannels);
		}

		// Apply gain
		float AudioGain = Settings::CVarPixelStreamingWebRTCAudioGain.GetValueOnAnyThread();
		if (AudioGain != 1.0f)
		{
			// hacky const cast here, but it saves us from a memcpy
			int16* PCMAudio = const_cast<int16*>(Buffer.GetData());

			// multiply audio by gain multiplier
			for (int i = 0; i < NumSamples; i++)
			{
				*PCMAudio = FMath::Max(-32768, FMath::Min(32767, *PCMAudio * AudioGain));
				PCMAudio++;
			}
		}

		RecordingBuffer.Append(Buffer.GetData(), Buffer.GetNumSamples());

		const float ChunkDurationSecs = 0.01f; // 10ms
		const int32 SamplesPer10Ms = GetSamplesPerDurationSecs(ChunkDurationSecs);

		// Feed in 10ms chunks
		while (RecordingBuffer.Num() > SamplesPer10Ms)
		{

			// Extract a 10ms chunk of samples from recording buffer
			TArray<int16_t> SubmitBuffer(RecordingBuffer.GetData(), SamplesPer10Ms);
			const size_t Frames = SubmitBuffer.Num() / TargetNumChannels;
			const size_t BytesPerFrame = TargetNumChannels * sizeof(int16_t);

			uint32_t OutMicLevel = VolumeLevel;

			int32_t WebRTCRes = AudioCallback->RecordedDataIsAvailable(
				SubmitBuffer.GetData(), Frames, BytesPerFrame, TargetNumChannels,
				TargetSampleRate, 0, 0, VolumeLevel, false, OutMicLevel);

			SetVolume(OutMicLevel);

			// Remove 10ms of samples from the recording buffer now it is submitted
			RecordingBuffer.RemoveAt(0, SamplesPer10Ms, EAllowShrinking::No);
		}
	}

	int32 FAudioSubmixCapturer::GetSamplesPerDurationSecs(float InSeconds) const
	{
		int32 SamplesPerSecond = TargetNumChannels * TargetSampleRate;
		int32 NumSamplesPerDuration = (int32)(SamplesPerSecond * InSeconds);
		return NumSamplesPerDuration;
	}

	void FAudioSubmixCapturer::RegisterAudioCallback(
		webrtc::AudioTransport* AudioTransportCallback)
	{
		FScopeLock Lock(&CriticalSection);
		AudioCallback = AudioTransportCallback;
	}

	bool FAudioSubmixCapturer::IsInitialised() const
	{
		return bInitialised;
	}

	void FAudioSubmixCapturer::Uninitialise()
	{
		FScopeLock Lock(&CriticalSection);

		if (GEngine)
		{
			FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
			if (AudioDevice)
			{
				AudioDevice->UnregisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
			}
		}

		RecordingBuffer.Empty();
		bInitialised = false;
		bCapturing = false;
	}

	bool FAudioSubmixCapturer::StartCapturing()
	{
		if (!bInitialised)
		{
			return false;
		}
		bCapturing = true;
		return true;
	}

	bool FAudioSubmixCapturer::EndCapturing()
	{
		if (!bInitialised)
		{
			return false;
		}
		bCapturing = false;
		return true;
	}

	bool FAudioSubmixCapturer::IsCapturing() const
	{
		return bCapturing;
	}

	void FAudioSubmixCapturer::SetAudioInputMixerPatch(TSharedPtr<FAudioInput> InPatchInput)
	{
		AudioMixerPatchInput = InPatchInput;
	}

} // namespace UE::PixelStreaming
