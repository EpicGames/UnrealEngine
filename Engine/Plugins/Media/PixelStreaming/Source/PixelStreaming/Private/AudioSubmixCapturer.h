// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/ThreadSafeBool.h"
#include "ISubmixBufferListener.h"
#include "Templates/SharedPointer.h"

namespace webrtc
{
	class AudioTransport;
} // namespace webrtc

namespace UE::PixelStreaming
{
	// Captures audio from UE and passes it along to WebRTC.
	class FAudioSubmixCapturer : public ISubmixBufferListener
	{
	public:
		// This magic number is the max volume used in webrtc fake audio device
		// module.
		static const uint32_t MaxVolumeLevel = 14392;

		FAudioSubmixCapturer();
		virtual ~FAudioSubmixCapturer() = default;

		bool Init();
		bool IsInitialised() const;
		bool IsCapturing() const;
		void Uninitialise();
		bool StartCapturing();
		bool EndCapturing();
		uint32_t GetVolume() const;
		void SetVolume(uint32_t NewVolume);
		void RegisterAudioCallback(webrtc::AudioTransport* AudioCb);
		void SetAudioInputMixerPatch(TSharedPtr<class FAudioInput> InPatchInput);

		// ISubmixBufferListener interface
		const FString& GetListenerName() const override;
		void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData,
			int32 NumSamples, int32 NumChannels,
			const int32 SampleRate, double AudioClock) override;

	private:
		int32 GetSamplesPerDurationSecs(float InSeconds) const;

	private:
		FThreadSafeBool bInitialised;
		FThreadSafeBool bCapturing;
		uint32 TargetSampleRate;
		uint8 TargetNumChannels;
		bool bReportedSampleRateMismatch;
		webrtc::AudioTransport* AudioCallback;
		uint32_t VolumeLevel;
		TArray<int16_t> RecordingBuffer;
		FCriticalSection CriticalSection; // One thread captures audio from UE, the
										  // other controls the state of the capturer.

		TSharedPtr<class FAudioInput> AudioMixerPatchInput;
	};
} // namespace UE::PixelStreaming
