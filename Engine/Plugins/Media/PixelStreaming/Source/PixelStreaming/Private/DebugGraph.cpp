// Copyright Epic Games, Inc. All Rights Reserved.

#include "DebugGraph.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"

namespace UE::PixelStreaming
{
	FDebugGraph::FDebugGraph(FName InName, int InSamples, float InMinRange, float InMaxRange, float InRefValue)
		: Name(InName)
		, MaxSamples(InSamples)
		, MinRange(InMinRange)
		, MaxRange(InMaxRange)
		, RefValue(InRefValue)
	{
		Values.SetNumZeroed(InSamples);
		AvgValues.SetNumZeroed(InSamples);
	}

	FDebugGraph::FDebugGraph(const FDebugGraph& Other)
		: Name(Other.Name)
		, MaxSamples(Other.MaxSamples)
		, MinRange(Other.MinRange)
		, MaxRange(Other.MaxRange)
		, RefValue(Other.RefValue)
		, Sum(Other.Sum)
		, Values(Other.Values)
		, AvgValues(Other.AvgValues)
	{
	}

	void FDebugGraph::AddValue(float InValue)
	{
		FScopeLock Lock(&CriticalSection);

		if(bFirstValue)
		{
			// If it the first value, fill the graph with it
			for(int i = 0; i < MaxSamples; i++)
			{
				AddValueInternal(InValue);
			}
			bFirstValue = false;
		}
		else
		{
			AddValueInternal(InValue);
		}
	}

	void FDebugGraph::AddValueInternal(float InValue)
	{
		if(InValue > MaxRange)
		{
			// Increase max range by 50% larger than this new larger value
			float NewMax = InValue * 2.0f;
			float Multiplier = NewMax / MaxRange;
			float Ratio = 1.0f / Multiplier;
			MaxRange = NewMax;

			// Increasing the max range means we must adjust all our existing values/averages/sum by the inverse
			for(int i = 0; i < MaxSamples; i++)
			{
				Values[i] *= Ratio;
				AvgValues[i] *= Ratio;
			}
			Sum *= Ratio;
		}

		InsertIndex = (InsertIndex + 1) % MaxSamples;

		const float NormalizedValue = (InValue - MinRange) / (MaxRange - MinRange);

		Sum -= Values[InsertIndex];
		Sum += NormalizedValue;

		Values[InsertIndex] = NormalizedValue;
		AvgValues[InsertIndex] = Sum / BufferedValues;

		BufferedValues = FGenericPlatformMath::Min(MaxSamples, BufferedValues + 1);
	}

	void FDebugGraph::Draw(FCanvas* Canvas, FVector2D Position, FVector2D Size) const
	{
		FScopeLock Lock(&CriticalSection);

		FCanvasTileItem BgItem(Position, Size, FColor{ 0x10, 0x10, 0x10, 0xBB });
		BgItem.BlendMode = SE_BLEND_Translucent;
		BgItem.Draw(Canvas);

		FVector2D StartPosition;
		FVector2D AvgStartPosition;

		float MinValue = TNumericLimits<float>::Max();
		float MaxValue = TNumericLimits<float>::Min();

		int ReadIndex = (InsertIndex + 1) % MaxSamples;
		for (int i = 0; i < MaxSamples; ++i, ReadIndex = (ReadIndex + 1) % MaxSamples)
		{
			const float NormalizedValue = FMath::Clamp(Values[ReadIndex], 0.0f, 1.0f);
			const float NormalizedAvgValue = FMath::Clamp(AvgValues[ReadIndex], 0.0f, 1.0f);
			const float Unnormalized = NormalizedValue * (MaxRange - MinRange) + MinRange;

			const float ValueBasis = i * Size.X / MaxSamples;
			const float ValueOffset = Size.Y - NormalizedValue * Size.Y;
			const float AvgValueOffset = Size.Y - NormalizedAvgValue * Size.Y;

			const FVector2D EndPosition = Position + FVector2D{ ValueBasis, ValueOffset };
			const FVector2D AvgEndPosition = Position + FVector2D{ ValueBasis, AvgValueOffset };

			if (i > 0)
			{
				FCanvasLineItem Line(StartPosition, EndPosition);
				Line.SetColor(FColor::White);
				Line.Draw(Canvas);

				FCanvasLineItem AvgLine(AvgStartPosition, AvgEndPosition);
				AvgLine.SetColor(FColor::Blue);
				AvgLine.Draw(Canvas);
			}

			StartPosition = EndPosition;
			AvgStartPosition = AvgEndPosition;

			MinValue = FGenericPlatformMath::Min(MinValue, Unnormalized);
			MaxValue = FGenericPlatformMath::Max(MaxValue, Unnormalized);
		}

		// todo: should have a function to shrink the min/max range if min/max value is < 50% of range

		FCanvasTextItem TextItem(Position, FText(), FSlateFontInfo(UEngine::GetTinyFont(), 9), FLinearColor(0, 1, 0));
		TextItem.Text = FText::FromString(FString::Printf(TEXT("%s [%.2f, %.2f]"), *Name.ToString(), MinRange, MaxRange));
		Canvas->DrawItem(TextItem);

		const float Unnormalized = AvgValues[InsertIndex] * (MaxRange - MinRange) + MinRange;
		TextItem.Text = FText::FromString(FString::Printf(TEXT("Avg: %.2f"), Unnormalized));
		TextItem.Position.Y += TextItem.DrawnSize.Y;
		Canvas->DrawItem(TextItem);

		TextItem.Text = FText::FromString(FString::Printf(TEXT("Max: %.2f"), MaxValue));
		TextItem.Position.Y += TextItem.DrawnSize.Y;
		Canvas->DrawItem(TextItem);

		TextItem.Text = FText::FromString(FString::Printf(TEXT("Min: %.2f"), MinValue));
		TextItem.Position.Y += TextItem.DrawnSize.Y;
		Canvas->DrawItem(TextItem);

		if (RefValue != 0.0f)
		{
			const float NormalizedRefValue = FMath::Clamp((RefValue - MinRange) / (MaxRange - MinRange), 0.0f, 1.0f);
			const float ValueOffset = NormalizedRefValue * Size.Y;
			const FVector2D StartRefPosition{ Position.X, Position.Y + Size.Y - ValueOffset };
			const FVector2D EndRefPosition{ Position.X + Size.X, Position.Y + Size.Y - ValueOffset };
			FCanvasLineItem RefLine(StartRefPosition, EndRefPosition);
			RefLine.SetColor(FColor::Red);
			RefLine.Draw(Canvas);

			TextItem.Text = FText::FromString(FString::Printf(TEXT("Ref: %.2f"), RefValue));
			TextItem.Position.Y += TextItem.DrawnSize.Y;
			Canvas->DrawItem(TextItem);
		}
	}
} // namespace UE::PixelStreaming
