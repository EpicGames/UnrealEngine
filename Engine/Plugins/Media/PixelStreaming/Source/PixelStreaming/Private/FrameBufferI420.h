// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebRTCIncludes.h"
#include "PixelCaptureBufferI420.h"

namespace UE::PixelStreaming
{
	/**
	 * A wrapper to adapt the FPixelStreamingI420Buffer to webrtc's I420BufferInterface
	 */
	class WebRTCI420BufferWrapper : public webrtc::I420BufferInterface
	{
	public:
		WebRTCI420BufferWrapper(TSharedPtr<FPixelCaptureBufferI420> InCaptureBuffer)
			: CaptureBuffer(InCaptureBuffer) {}
		virtual ~WebRTCI420BufferWrapper() = default;
		virtual int width() const override { return CaptureBuffer->GetWidth(); }
		virtual int height() const override { return CaptureBuffer->GetHeight(); }
		virtual const uint8_t* DataY() const override { return CaptureBuffer->GetDataY(); }
		virtual const uint8_t* DataU() const override { return CaptureBuffer->GetDataU(); }
		virtual const uint8_t* DataV() const override { return CaptureBuffer->GetDataV(); }
		virtual int StrideY() const override { return CaptureBuffer->GetStrideY(); }
		virtual int StrideU() const override { return CaptureBuffer->GetStrideUV(); }
		virtual int StrideV() const override { return CaptureBuffer->GetStrideUV(); }

	private:
		TSharedPtr<FPixelCaptureBufferI420> CaptureBuffer;
	};

	/**
	 * A frame buffer for our VPX encoders. Contains a single I420 buffer.
	 */
	class FFrameBufferI420 : public webrtc::VideoFrameBuffer
	{
	public:
		FFrameBufferI420(TSharedPtr<FPixelCaptureBufferI420> Buffer)
			: I420BufferWrapper(new rtc::RefCountedObject<WebRTCI420BufferWrapper>(Buffer)) {}

		virtual ~FFrameBufferI420() = default;

		virtual webrtc::VideoFrameBuffer::Type type() const override { return webrtc::VideoFrameBuffer::Type::kNative; }
		virtual int width() const override { return I420BufferWrapper->width(); }
		virtual int height() const override { return I420BufferWrapper->height(); }

		virtual rtc::scoped_refptr<webrtc::I420BufferInterface> ToI420() override { return I420BufferWrapper; }
		virtual const webrtc::I420BufferInterface* GetI420() const override { return I420BufferWrapper.get(); }

	private:
		rtc::scoped_refptr<WebRTCI420BufferWrapper> I420BufferWrapper;
	};
} // namespace UE::PixelStreaming
