// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameBufferMultiFormat.h"

namespace UE::PixelStreaming
{
	FFrameBufferMultiFormatBase::FFrameBufferMultiFormatBase(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, uint32 InStreamId)
		: FrameCapturer(InFrameCapturer)
		, StreamId(InStreamId)
	{
	}

	FFrameBufferMultiFormatLayered::FFrameBufferMultiFormatLayered(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, uint32 InStreamId, FIntPoint SourceResolution)
		: FFrameBufferMultiFormatBase(InFrameCapturer, InStreamId)
		, SourceResolution(SourceResolution)
	{
	}

	int FFrameBufferMultiFormatLayered::width() const
	{
		return SourceResolution.X;
	}

	int FFrameBufferMultiFormatLayered::height() const
	{
		return SourceResolution.Y;
	}

	rtc::scoped_refptr<FFrameBufferMultiFormat> FFrameBufferMultiFormatLayered::GetLayer(FIntPoint Resolution) const
	{
#if WEBRTC_5414
		return rtc::make_ref_counted<FFrameBufferMultiFormat>(FrameCapturer, StreamId, Resolution);
#else
		return new rtc::RefCountedObject<FFrameBufferMultiFormat>(FrameCapturer, StreamId, Resolution);
#endif
	}

	FFrameBufferMultiFormat::FFrameBufferMultiFormat(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, uint32 InStreamId, FIntPoint Resolution)
		: FFrameBufferMultiFormatBase(InFrameCapturer, InStreamId)
		, Resolution(Resolution)
	{
	}

	int FFrameBufferMultiFormat::width() const
	{
		return Resolution.X;
	}

	int FFrameBufferMultiFormat::height() const
	{
		return Resolution.Y;
	}

	IPixelCaptureOutputFrame* FFrameBufferMultiFormat::RequestFormat(int32 Format) const
	{
		// ensure this frame buffer will always refer to the same frame
		if (TSharedPtr<IPixelCaptureOutputFrame>* CachedFrame = CachedFormat.Find(Format))
		{
			return CachedFrame->Get();
		}

		if (!FrameCapturer)
		{
			return nullptr;
		}

		TSharedPtr<IPixelCaptureOutputFrame> Frame = FrameCapturer->WaitForFormat(Format, Resolution);
		CachedFormat.Add(Format, Frame);
		return Frame.Get();
	}
} // namespace UE::PixelStreaming
