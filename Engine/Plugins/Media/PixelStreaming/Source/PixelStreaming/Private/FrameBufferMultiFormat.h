// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturerMultiFormat.h"
#include "WebRTCIncludes.h"

namespace UE::PixelStreaming
{
	class FFrameBufferMultiFormat;

	/**
	 * Base class for our multi format buffers.
	 */
	class FFrameBufferMultiFormatBase : public webrtc::VideoFrameBuffer
	{
	public:
		FFrameBufferMultiFormatBase(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, uint32 InStreamId);
		virtual ~FFrameBufferMultiFormatBase() = default;

		virtual webrtc::VideoFrameBuffer::Type type() const override { return webrtc::VideoFrameBuffer::Type::kNative; }

		virtual rtc::scoped_refptr<webrtc::I420BufferInterface> ToI420() override
		{
			unimplemented();
			return nullptr;
		}

		virtual const webrtc::I420BufferInterface* GetI420() const override
		{
			unimplemented();
			return nullptr;
		}

		void SetSourceStreamId(uint32 InStreamId) { StreamId = InStreamId; }

	protected:
		TSharedPtr<FPixelCaptureCapturerMultiFormat> FrameCapturer;
		uint32										 StreamId;
	};

	/**
	 * A multi layered, multi format frame buffer for our encoder.
	 */
	class FFrameBufferMultiFormatLayered : public FFrameBufferMultiFormatBase
	{
	public:
		FFrameBufferMultiFormatLayered(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, uint32 InStreamId, FIntPoint SourceResolution);
		virtual ~FFrameBufferMultiFormatLayered() = default;

		virtual int width() const override;
		virtual int height() const override;

		rtc::scoped_refptr<FFrameBufferMultiFormat> GetLayer(FIntPoint Resolution) const;

	private:
		FIntPoint SourceResolution;
	};

	/**
	 * A single layer, multi format frame buffer.
	 */
	class FFrameBufferMultiFormat : public FFrameBufferMultiFormatBase
	{
	public:
		FFrameBufferMultiFormat(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, uint32 StreamId, FIntPoint InResolution);
		virtual ~FFrameBufferMultiFormat() = default;

		virtual int width() const override;
		virtual int height() const override;

		IPixelCaptureOutputFrame* RequestFormat(int32 Format) const;

		uint32 GetSourceStreamId() const { return StreamId; }

	private:
		FIntPoint Resolution;
		// we want the frame buffer to always refer to the same frame. so the first request for a format
		// will fill this cache.
		mutable TMap<int32, TSharedPtr<IPixelCaptureOutputFrame>> CachedFormat;
	};
} // namespace UE::PixelStreaming
