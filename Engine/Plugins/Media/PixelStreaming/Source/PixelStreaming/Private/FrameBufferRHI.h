// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebRTCIncludes.h"
#include "Video/Resources/VideoResourceRHI.h"

namespace UE::PixelStreaming
{
	/**
	 * A frame buffer for RHI Video Resources.
	 */
	class FFrameBufferRHI : public webrtc::VideoFrameBuffer
	{
	public:
		FFrameBufferRHI(TSharedPtr<FVideoResourceRHI> VideoResourceRHI) : VideoResourceRHI(VideoResourceRHI) {}


		virtual ~FFrameBufferRHI() = default;

		virtual webrtc::VideoFrameBuffer::Type type() const override { return webrtc::VideoFrameBuffer::Type::kNative; }
		virtual int width() const override { return VideoResourceRHI->GetDescriptor().Width; }
		virtual int height() const override { return VideoResourceRHI->GetDescriptor().Height; }

		virtual rtc::scoped_refptr<webrtc::I420BufferInterface> ToI420() override
		{
			unimplemented();
			return nullptr;
		}

		virtual const webrtc::I420BufferInterface* GetI420() const override
		{
			unimplemented();
			return nullptr;
		}

		TSharedPtr<FVideoResourceRHI> GetVideoResource() { return VideoResourceRHI; }

	private:
		TSharedPtr<FVideoResourceRHI> VideoResourceRHI;
	};
} // namespace UE::PixelStreaming
