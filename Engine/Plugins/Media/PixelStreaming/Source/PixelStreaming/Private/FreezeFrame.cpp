// Copyright Epic Games, Inc. All Rights Reserved.

#include "FreezeFrame.h"
#include "EngineModule.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingInputProtocol.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureBufferI420.h"
#include "TextureResource.h"
#include "RHICommandList.h"
#include "RHIFwd.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "Settings.h"
#include "Utils.h"
#include "MediaShaders.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "libyuv/convert.h"
#include "libyuv/video_common.h"
#include "common_video/libyuv/include/webrtc_libyuv.h"

namespace UE::PixelStreaming
{
	TSharedPtr<FFreezeFrame> FFreezeFrame::Create(TWeakPtr<TThreadSafeMap<FPixelStreamingPlayerId,FPlayerContext>> InPlayers)
	{
		TSharedPtr<FFreezeFrame> FF = TSharedPtr<FFreezeFrame>(new FFreezeFrame());
		FF->SetPlayers(InPlayers);
		return FF;
	}

	FFreezeFrame::~FFreezeFrame()
	{
		RemoveFreezeFrameBinding();
	}

	void FFreezeFrame::SetVideoInput(TWeakPtr<FPixelStreamingVideoInput> InVideoInput)
	{
		VideoInput = InVideoInput;
		RemoveFreezeFrameBinding();
	}

	void FFreezeFrame::SetPlayers(TWeakPtr<TThreadSafeMap<FPixelStreamingPlayerId,FPlayerContext>> InPlayers)
	{
		WeakPlayers = InPlayers;
	}

	/*
	 * Add the commands to the RHI command list to copy a texture from source to dest - even if the format is different.
	 * Assumes SourceTexture is in ERHIAccess::CopySrc and DestTexture is in ERHIAccess::CopyDest
	 */
	void CopyTexture(FRHICommandList& RHICmdList, FTextureRHIRef SourceTexture, FTextureRHIRef DestTexture)
	{
		if (SourceTexture->GetDesc().Format == DestTexture->GetDesc().Format
			&& SourceTexture->GetDesc().Extent.X == DestTexture->GetDesc().Extent.X
			&& SourceTexture->GetDesc().Extent.Y == DestTexture->GetDesc().Extent.Y)
		{

			RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::CopySrc));
			RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::Unknown, ERHIAccess::CopyDest));

			// source and dest are the same. simple copy
			RHICmdList.CopyTexture(SourceTexture, DestTexture, {});
		}
		else
		{
			IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>("Renderer");

			RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

			// source and destination are different. rendered copy
			FRHIRenderPassInfo RPInfo(DestTexture, ERenderTargetActions::Load_Store);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("PixelStreaming::CopyTexture"));
			{
				FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
				TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

				RHICmdList.SetViewport(0, 0, 0.0f, DestTexture->GetDesc().Extent.X, DestTexture->GetDesc().Extent.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), SourceTexture);

				FIntPoint TargetBufferSize(DestTexture->GetDesc().Extent.X, DestTexture->GetDesc().Extent.Y);
				RendererModule->DrawRectangle(RHICmdList, 0, 0, // Dest X, Y
					DestTexture->GetDesc().Extent.X,			// Dest Width
					DestTexture->GetDesc().Extent.Y,			// Dest Height
					0, 0,										// Source U, V
					1, 1,										// Source USize, VSize
					TargetBufferSize,							// Target buffer size
					FIntPoint(1, 1),							// Source texture size
					VertexShader, EDRF_Default);
			}

			RHICmdList.EndRenderPass();

			RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
			RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::RTV, ERHIAccess::CopyDest));
		}
	}

	void FFreezeFrame::StartFreeze(UTexture2D* Texture)
	{
		if (Texture)
		{
			ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)
			([this, Texture](FRHICommandListImmediate& RHICmdList) {
				// A frame is supplied so immediately read its data and send as a JPEG.
				FTextureRHIRef TextureRHI = Texture->GetResource() ? Texture->GetResource()->TextureRHI : nullptr;
				if (!TextureRHI)
				{
					UE_LOGF(LogPixelStreaming, Error, "Attempting freeze frame with texture %ls with no texture RHI", *Texture->GetName());
					return;
				}
				uint32 Width = TextureRHI->GetDesc().Extent.X;
				uint32 Height = TextureRHI->GetDesc().Extent.Y;

				FRHITextureCreateDesc TextureDesc =
						FRHITextureCreateDesc::Create2D(TEXT("PixelStreamingBlankTexture"), Width, Height, EPixelFormat::PF_B8G8R8A8)
						.SetClearValue(FClearValueBinding::None)
						.SetFlags(ETextureCreateFlags::RenderTargetable)
						.SetInitialState(ERHIAccess::Present)
						.DetermineInititialState();

				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(TextureDesc);

				// Copy freeze frame texture to empty texture
				CopyTexture(RHICmdList, TextureRHI, DestTexture);

				TArray<FColor> Data;
				FIntRect Rect(0, 0, Width, Height);
				// This `ReadSurfaceData` makes a blocking call from CPU -> GPU -> CPU
				// WHich is how on the very next line we are able to copy the data out and send it.
				RHICmdList.ReadSurfaceData(DestTexture, Rect, Data, FReadSurfaceDataFlags());
				SendFreezeFrame(MoveTemp(Data), Rect);
			});
		}
		else
		{
			// A frame is not supplied, so we need to get it from the video input
			// at the next opportunity and send as a JPEG.
			SetupFreezeFrameCapture();
		}
	}

	void FFreezeFrame::StopFreeze()
	{
		TSharedPtr<TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>> Players = WeakPlayers.Pin();

		if(!Players)
		{
			return;
		}

		Players->Apply([this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
				if (PlayerContext.DataChannel)
				{
					PlayerContext.DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("UnfreezeFrame")->GetID());
				}
			});

		CachedJpegBytes.Empty();
	}

	void FFreezeFrame::SendCachedFreezeFrameTo(FPixelStreamingPlayerId PlayerId) const
	{
		TSharedPtr<TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>> Players = WeakPlayers.Pin();

		if(!Players)
		{
			return;
		}

		if (CachedJpegBytes.Num() > 0)
		{
			if (const FPlayerContext* PlayerContext = Players->Find(PlayerId))
			{
				if (PlayerContext->DataChannel)
				{
					PlayerContext->DataChannel->SendArbitraryData(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("FreezeFrame")->GetID(), CachedJpegBytes);
				}
			}
		}
	}

	void FFreezeFrame::SendFreezeFrame(TArray<FColor> RawData, const FIntRect & Rect)
	{
		TSharedPtr<TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>> Players = WeakPlayers.Pin();

		if(!Players)
		{
			return;
		}

		IImageWrapperModule& ImageWrapperModule = FModuleManager::GetModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::JPEG);
		bool bSuccess = ImageWrapper->SetRaw(RawData.GetData(), RawData.Num() * sizeof(FColor), Rect.Width(), Rect.Height(), ERGBFormat::BGRA, 8);
		if (bSuccess)
		{
			// Compress to a JPEG of the maximum possible quality.
			int32 Quality = Settings::CVarPixelStreamingFreezeFrameQuality.GetValueOnAnyThread();
			const TArray64<uint8>& JpegBytes = ImageWrapper->GetCompressed(Quality);
			Players->Apply([&JpegBytes, this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
				if (PlayerContext.DataChannel)
				{
					FPixelStreamingInputMessage* Message = FPixelStreamingInputProtocol::FromStreamerProtocol.Find("FreezeFrame");
					if(Message != nullptr)
					{
						PlayerContext.DataChannel->SendArbitraryData(Message->GetID(), JpegBytes);
					}
					else
					{
						UE_LOGF(LogPixelStreaming, Error, "Could not find datachannel message with ID FreezeFrame.");
					}
				}
			});
			CachedJpegBytes = JpegBytes;
		}
		else
		{
			UE_LOGF(LogPixelStreaming, Error, "JPEG image wrapper failed to accept frame data");
		}
	}

	void FFreezeFrame::SetupFreezeFrameCapture()
	{
		// Remove any existing binding
		RemoveFreezeFrameBinding();

		if(TSharedPtr<FPixelStreamingVideoInput> ConcreteVideoInput = VideoInput.Pin())
		{
			OnFrameCapturedForFreezeFrameHandle = ConcreteVideoInput->OnFrameCaptured.AddSP(this, &FFreezeFrame::FreezeFrameCapture);
		}
	}

	void FFreezeFrame::RemoveFreezeFrameBinding()
	{
		if(!OnFrameCapturedForFreezeFrameHandle)
		{
			return;
		}

		if(TSharedPtr<FPixelStreamingVideoInput> ConcreteVideoInput = VideoInput.Pin())
		{
			ConcreteVideoInput->OnFrameCaptured.Remove(OnFrameCapturedForFreezeFrameHandle.GetValue());
			OnFrameCapturedForFreezeFrameHandle.Reset();
		}
	}

	void FFreezeFrame::FreezeFrameCapture()
	{
		TSharedPtr<FPixelStreamingVideoInput> Input = VideoInput.Pin();

		if(!Input)
		{
			return;
		}

		if(Settings::IsCodecVPX())
		{
			// Request output format is I420 for VPX
			TSharedPtr<IPixelCaptureOutputFrame> OutputFrame = Input->RequestFormat(PixelCaptureBufferFormat::FORMAT_I420);
			if (OutputFrame)
			{
				// Can remove binding now we have got the output in the format we need to send a FF
				RemoveFreezeFrameBinding();

				FPixelCaptureOutputFrameI420* I420Frame = StaticCast<FPixelCaptureOutputFrameI420*>(OutputFrame.Get());
				TSharedPtr<FPixelCaptureBufferI420> I420Buffer = I420Frame->GetI420Buffer();

				const uint32_t NumBytes = webrtc::CalcBufferSize(webrtc::VideoType::kARGB, I420Frame->GetWidth(), I420Frame->GetHeight());
				const uint32_t NumPixels = I420Frame->GetWidth() * I420Frame->GetHeight();
				uint8_t* ARGBBuffer = new uint8_t[NumBytes];

				// Convert I420 to ARGB
				libyuv::ConvertFromI420(
					I420Buffer->GetDataY(), I420Buffer->GetStrideY(),
					I420Buffer->GetDataU(), I420Buffer->GetStrideUV(),
					I420Buffer->GetDataV(), I420Buffer->GetStrideUV(),
					ARGBBuffer, 0,
					I420Buffer->GetWidth(), I420Buffer->GetHeight(),
					libyuv::FOURCC_ARGB);

				// We assume FColor packing is same ordering as the Buffer are copying from
				TArray<FColor> PixelArr((FColor*)ARGBBuffer, NumPixels);
				FIntRect Rect(0, 0, I420Frame->GetWidth(), I420Frame->GetHeight());
				SendFreezeFrame(MoveTemp(PixelArr), Rect);
			}
		}
		else
		{
			TSharedPtr<IPixelCaptureOutputFrame> OutputFrame = Input->RequestFormat(PixelCaptureBufferFormat::FORMAT_RHI);
			if (OutputFrame)
			{
				// Can remove binding now we have got the output in the format we need to send a FF
				RemoveFreezeFrameBinding();

				TWeakPtr<FFreezeFrame> WeakSelf = AsShared();

				ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)
				([WeakSelf, OutputFrame](FRHICommandListImmediate& RHICmdList) {
					if (auto ThisPtr = WeakSelf.Pin())
					{
						FPixelCaptureOutputFrameRHI* RHISourceFrame = StaticCast<FPixelCaptureOutputFrameRHI*>(OutputFrame.Get());

						// Read the data out of the back buffer and send as a JPEG.
						FIntRect Rect(0, 0, RHISourceFrame->GetWidth(), RHISourceFrame->GetHeight());
						TArray<FColor> Data;

						RHICmdList.ReadSurfaceData(RHISourceFrame->GetFrameTexture(), Rect, Data, FReadSurfaceDataFlags());
						ThisPtr->SendFreezeFrame(MoveTemp(Data), Rect);
					}
				});
			}
		}

	}

} // namespace UE::PixelStreaming