// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "PlayerContext.h"
#include "PixelStreamingPlayerId.h"
#include "PixelStreamingVideoInput.h"
#include "Engine/Texture2D.h"
#include "ThreadSafeMap.h"
#include "Templates/SharedPointer.h"

namespace UE::PixelStreaming
{
	/*
	* The user of the Pixel Streaming application can trigger a freeze frame to happen at any time during stream.
	* What a freeze frame will do is either:
	* 1) Mode 1 - Send a known texture/image (effectively a still image, e.g. a loading screen).
	* 2) Mode 2 - Send the current rendered frame out of UE as an image
	* This freeze frame will displayed over the top of the video stream with no compression.
	* So a crisp, high quality image, is guaranteed.
	* While this is occuring the stream is not technically frozen, it still encodes and send frames;
	* however, visually, the user is only shown this still. This technique can be used to hide large, known,
	* cpu/gpu lags in the Pixel Streaming experience, for example during a level load or some expensive transition.
	* The better solution is to not lag your application in the first place, but this can be a bandaid.
	*/
	class FFreezeFrame : public TSharedFromThis<FFreezeFrame>
	{
	public:
		static TSharedPtr<FFreezeFrame> Create(TWeakPtr<TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>> InPlayers);
		virtual ~FFreezeFrame();
		void SetVideoInput(TWeakPtr<FPixelStreamingVideoInput> VideoInput);
		void SetPlayers(TWeakPtr<TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>> InPlayers);

		/**
		 * @brief Start the freeze frame process, either mode 1 or 2 depending on if a texture is passed.
		 * @param Texture - The still image to send as a freeze frame, pass nullptr to use send the backbuffer instead.
		*/
		void StartFreeze(UTexture2D* Texture);
		void StopFreeze();
		void SendCachedFreezeFrameTo(FPixelStreamingPlayerId PlayerId) const;

	protected:
		FFreezeFrame() = default;

	private:
		void SendFreezeFrame(TArray<FColor> RawData, const FIntRect& Rect);
		void SetupFreezeFrameCapture();
		void RemoveFreezeFrameBinding();
		void FreezeFrameCapture();

	private:
		TWeakPtr<TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>> WeakPlayers;

		// Video input used to capture the frame if mode 2 is used.
		TWeakPtr<FPixelStreamingVideoInput> VideoInput;

		// When we send a freeze frame we retain the data so we send freeze frame to new peers if they join during a freeze frame.
		TArray64<uint8> CachedJpegBytes;

		// Delegate handle for when we bind to the OnFrameCaptured delegate of the video input - we unbind from this once the freeze frame is captured
		TOptional<FDelegateHandle> OnFrameCapturedForFreezeFrameHandle;
	};

} // namespace UE::PixelStreaming