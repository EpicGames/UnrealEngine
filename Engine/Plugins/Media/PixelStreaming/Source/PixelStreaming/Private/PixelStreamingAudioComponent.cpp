// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingAudioComponent.h"
#include "IPixelStreamingModule.h"
#include "IPixelStreamingStreamer.h"
#include "PixelStreamingPrivate.h"
#include "CoreMinimal.h"
#include "DSP/FloatArrayMath.h"

/*
 * Component that recieves audio from a remote webrtc connection and outputs it into UE using a "synth component".
 */
UPixelStreamingAudioComponent::UPixelStreamingAudioComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PlayerToHear(FPixelStreamingPlayerId())
	, bAutoFindPeer(true)
	, AudioSink(nullptr)
	, SoundGenerator(MakeShared<FWebRTCSoundGenerator, ESPMode::ThreadSafe>())
{
	PreferredBufferLength = 512u;
	NumChannels = 2;
	PrimaryComponentTick.bCanEverTick = true;
	SetComponentTickEnabled(true);
	bAutoActivate = true;
};

ISoundGeneratorPtr UPixelStreamingAudioComponent::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	SoundGenerator->SetParameters(InParams);
	Initialize(InParams.SampleRate);
	return SoundGenerator;
}

void UPixelStreamingAudioComponent::OnBeginGenerate()
{
	SoundGenerator->bGeneratingAudio = true;
}

void UPixelStreamingAudioComponent::OnEndGenerate()
{
	SoundGenerator->bGeneratingAudio = false;
}

void UPixelStreamingAudioComponent::BeginDestroy()
{
	Super::BeginDestroy();
	Reset();
}

bool UPixelStreamingAudioComponent::ListenTo(FString PlayerToListenTo)
{
	IPixelStreamingModule& PixelStreamingModule = IPixelStreamingModule::Get();
	if (!PixelStreamingModule.IsReady())
	{
		return false;
	}
	return StreamerListenTo(PixelStreamingModule.GetDefaultStreamerID(), PlayerToListenTo);
}

bool UPixelStreamingAudioComponent::StreamerListenTo(FString StreamerId, FString PlayerToListenTo)
{
	if (!IPixelStreamingModule::IsAvailable())
	{
		UE_LOGF(LogPixelStreaming, Verbose, "Pixel Streaming audio component could not listen to anything because Pixel Streaming module is not loaded. This is expected on dedicated servers.");
		return false;
	}

	IPixelStreamingModule& PixelStreamingModule = IPixelStreamingModule::Get();
	if (!PixelStreamingModule.IsReady())
	{
		return false;
	}

	PlayerToHear = PlayerToListenTo;

	if (StreamerId == FString())
	{
		TArray<FString> StreamerIds = PixelStreamingModule.GetStreamerIds();
		if (StreamerIds.Num() > 0)
		{
			StreamerToHear = StreamerIds[0];
		}
		else
		{
			StreamerToHear = PixelStreamingModule.GetDefaultStreamerID();
		}
	}
	else
	{
		StreamerToHear = StreamerId;
	}

	TSharedPtr<IPixelStreamingStreamer> Streamer = PixelStreamingModule.FindStreamer(StreamerToHear);
	if (!Streamer)
	{
		return false;
	}
	IPixelStreamingAudioSink* CandidateSink = WillListenToAnyPlayer() ? Streamer->GetUnlistenedAudioSink() : Streamer->GetPeerAudioSink(FPixelStreamingPlayerId(PlayerToHear));

	if (CandidateSink == nullptr)
	{
		return false;
	}

	AudioSink = CandidateSink;
	AudioSink->AddAudioConsumer(this);

	return true;
}

void UPixelStreamingAudioComponent::Reset()
{
	PlayerToHear = FString();
	StreamerToHear = FString();

	if (AudioSink)
	{
		AudioSink->RemoveAudioConsumer(this);
	}
	AudioSink = nullptr;

	if (SoundGenerator)
	{
		SoundGenerator->bShouldGenerateAudio = false;
		SoundGenerator->EmptyBuffers();
	}
}

bool UPixelStreamingAudioComponent::IsListeningToPlayer()
{
	return SoundGenerator->bShouldGenerateAudio;
}

bool UPixelStreamingAudioComponent::WillListenToAnyPlayer()
{
	return PlayerToHear == FString();
}

void UPixelStreamingAudioComponent::ConsumeRawPCM(const int16_t* AudioData, int InSampleRate, size_t NChannels, size_t NFrames)
{
	// Sound generator has not been initialized yet.
	if (SoundGenerator->GetSampleRate() == 0 || GetAudioComponent() == nullptr)
	{
		return;
	}

	// Set pitch multiplier as a way to handle mismatched sample rates
	if (InSampleRate != SoundGenerator->GetSampleRate())
	{
		GetAudioComponent()->SetPitchMultiplier((float)InSampleRate / SoundGenerator->GetSampleRate());
	}
	else if (GetAudioComponent()->PitchMultiplier != 1.0f)
	{
		GetAudioComponent()->SetPitchMultiplier(1.0f);
	}

	SoundGenerator->AddAudio(AudioData, InSampleRate, NChannels, NFrames);
}

void UPixelStreamingAudioComponent::OnConsumerAdded()
{
	SoundGenerator->bShouldGenerateAudio = true;
	Start();
}

void UPixelStreamingAudioComponent::OnConsumerRemoved()
{
	Reset();
}

void UPixelStreamingAudioComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{

	bool bPixelStreamingLoaded = IPixelStreamingModule::IsAvailable();

	// Early out if running in commandlet
	if (IsRunningCommandlet())
	{
		return;
	}

	// if auto connect turned off don't bother
	if (!bAutoFindPeer)
	{
		return;
	}

	// if listening to a peer don't auto connect
	if (IsListeningToPlayer())
	{
		return;
	}

	if (StreamerListenTo(StreamerToHear, PlayerToHear))
	{
		UE_LOGF(LogPixelStreaming, Log, "PixelStreaming audio component found a WebRTC peer to listen to.");
	}
}

/*
 * ---------------- FWebRTCSoundGenerator -------------------------
 */

FWebRTCSoundGenerator::FWebRTCSoundGenerator()
	: Params()
	, Buffer()
	, CriticalSection()
{
}

void FWebRTCSoundGenerator::SetParameters(const FSoundGeneratorInitParams& InitParams)
{
	Params = InitParams;
}

int32 FWebRTCSoundGenerator::GetDesiredNumSamplesToRenderPerCallback() const
{
	return Params.NumFramesPerCallback * Params.NumChannels;
}

void FWebRTCSoundGenerator::EmptyBuffers()
{
	FScopeLock Lock(&CriticalSection);
	Buffer.Empty();
}

void FWebRTCSoundGenerator::AddAudio(const int16_t* AudioData, int InSampleRate, size_t NChannels, size_t NFrames)
{
	if (!bGeneratingAudio)
	{
		return;
	}

	int NSamples = NFrames * NChannels;

	// Critical Section as we are writing into the `Buffer` that `ISoundGenerator` is using on another thread.
	FScopeLock Lock(&CriticalSection);

	Buffer.Append(AudioData, NSamples);
}

// Called when a new buffer is required.
int32 FWebRTCSoundGenerator::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	// Not listening to peer, return zero'd buffer.
	if (!bShouldGenerateAudio || Buffer.Num() == 0)
	{
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
		return NumSamples;
	}

	int32 NumSamplesToCopy = NumSamples;

	// Critical section
	{
		FScopeLock Lock(&CriticalSection);

		NumSamplesToCopy = FGenericPlatformMath::Min(NumSamples, Buffer.Num());

		// Copy from local buffer into OutAudio if we have enough samples
		Audio::ArrayPcm16ToFloat(MakeArrayView(Buffer.GetData(), NumSamplesToCopy), MakeArrayView(OutAudio, NumSamplesToCopy));

		// Remove front NumSamples from the local buffer
		Buffer.RemoveAt(0, NumSamplesToCopy, EAllowShrinking::No);
	}

	if (NumSamplesToCopy < NumSamples)
	{
		FMemory::Memzero(&OutAudio[NumSamplesToCopy], (NumSamples - NumSamplesToCopy) * sizeof(float));
	}

	return NumSamplesToCopy;
}
