// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingAudioDeviceModule.h"
#include "AudioSubmixCapturer.h"
#include "AudioPlayoutRequester.h"
#include "PixelStreamingPrivate.h"
#include "Settings.h"
#include "AudioInputMixer.h"

// These are copied from webrtc internals
#define CHECKinitialized_() \
	{                       \
		if (!bInitialized)  \
		{                   \
			return -1;      \
		};                  \
	}
#define CHECKinitialized__BOOL() \
	{                            \
		if (!bInitialized)       \
		{                        \
			return false;        \
		};                       \
	}

namespace UE::PixelStreaming
{

	FAudioDeviceModule::FAudioDeviceModule()
		: FAudioDeviceModule(nullptr)
	{
	}

	FAudioDeviceModule::FAudioDeviceModule(TSharedPtr<UE::PixelStreaming::FAudioInputMixer> InMixer)
		: bInitialized(false)
		, Capturer(MakeShared<UE::PixelStreaming::FAudioSubmixCapturer, ESPMode::ThreadSafe>())
		, Requester(MakeUnique<UE::PixelStreaming::FAudioPlayoutRequester>())
		, InputMixer(InMixer)
	{
		if (InputMixer.IsValid())
		{
			TSharedPtr<UE::PixelStreaming::FAudioInput> SubmixInput = InputMixer->CreateInput();
			Capturer->SetAudioInputMixerPatch(SubmixInput);
		}
	}

	int32 FAudioDeviceModule::ActiveAudioLayer(AudioLayer* audioLayer) const
	{
		*audioLayer = AudioDeviceModule::kDummyAudio;
		return 0;
	}

	int32 FAudioDeviceModule::RegisterAudioCallback(webrtc::AudioTransport* audioCallback)
	{
		Requester->RegisterAudioCallback(audioCallback);
		Capturer->RegisterAudioCallback(audioCallback);

		if (InputMixer.IsValid())
		{
			InputMixer->RegisterAudioCallback(audioCallback);
		}
		return 0;
	}

	int32 FAudioDeviceModule::Init()
	{
		InitRecording();

		UE_LOGF(LogPixelStreaming, Verbose, "Init PixelStreamingAudioDeviceModule");
		bInitialized = true;
		return 0;
	}

	int32 FAudioDeviceModule::Terminate()
	{
		if (!bInitialized)
		{
			return 0;
		}

		Requester->Uninitialise();
		Capturer->Uninitialise();

		UE_LOGF(LogPixelStreaming, Verbose, "Terminate");

		return 0;
	}

	bool FAudioDeviceModule::Initialized() const
	{
		return bInitialized;
	}

	int16 FAudioDeviceModule::PlayoutDevices()
	{
		CHECKinitialized_();
		return -1;
	}

	int16 FAudioDeviceModule::RecordingDevices()
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioDeviceModule::PlayoutDeviceName(uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize])
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioDeviceModule::RecordingDeviceName(uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize])
	{
		CHECKinitialized_();
		return -1;
	}

	int32 FAudioDeviceModule::SetPlayoutDevice(uint16 index)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioDeviceModule::SetPlayoutDevice(WindowsDeviceType device)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioDeviceModule::SetRecordingDevice(uint16 index)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioDeviceModule::SetRecordingDevice(WindowsDeviceType device)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioDeviceModule::PlayoutIsAvailable(bool* available)
	{
		CHECKinitialized_();
		*available = !UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCDisableReceiveAudio.GetValueOnAnyThread();
		return 0;
	}

	int32 FAudioDeviceModule::InitPlayout()
	{
		CHECKinitialized_();
		bool bIsPlayoutAvailable = false;
		PlayoutIsAvailable(&bIsPlayoutAvailable);
		if (!bIsPlayoutAvailable)
		{
			return -1;
		}

		Requester->InitPlayout();
		return 0;
	}

	bool FAudioDeviceModule::PlayoutIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return Requester->PlayoutIsInitialized();
	}

	int32 FAudioDeviceModule::StartPlayout()
	{
		CHECKinitialized_();

		if (!Requester->PlayoutIsInitialized())
		{
			return -1;
		}

		Requester->StartPlayout();
		return 0;
	}

	int32 FAudioDeviceModule::StopPlayout()
	{
		CHECKinitialized_();
		if (!Requester->PlayoutIsInitialized())
		{
			return -1;
		}

		Requester->StopPlayout();
		return 0;
	}

	bool FAudioDeviceModule::Playing() const
	{
		CHECKinitialized__BOOL();
		return Requester->Playing();
	}

	int32 FAudioDeviceModule::RecordingIsAvailable(bool* available)
	{
		CHECKinitialized_();
		*available = !UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCDisableTransmitAudio.GetValueOnAnyThread();
		return 0;
	}

	int32 FAudioDeviceModule::InitRecording()
	{
		CHECKinitialized_();

		bool bIsRecordingAvailable = false;
		RecordingIsAvailable(&bIsRecordingAvailable);
		if (!bIsRecordingAvailable)
		{
			return -1;
		}

		if (!Capturer->IsInitialised())
		{
			Capturer->Init();
		}

		return 0;
	}

	bool FAudioDeviceModule::RecordingIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return Capturer->IsInitialised();
	}

	int32 FAudioDeviceModule::StartRecording()
	{
		CHECKinitialized_();
		if (!Capturer->IsCapturing())
		{
			Capturer->StartCapturing();
		}
		if (InputMixer.IsValid())
		{
			InputMixer->StartMixing();
		}
		return 0;
	}

	int32 FAudioDeviceModule::StopRecording()
	{
		CHECKinitialized_();
		if (Capturer->IsCapturing())
		{
			Capturer->EndCapturing();
		}
		if (InputMixer.IsValid())
		{
			InputMixer->StopMixing();
		}
		return 0;
	}

	bool FAudioDeviceModule::Recording() const
	{
		CHECKinitialized__BOOL();
		return Capturer->IsCapturing();
	}

	int32 FAudioDeviceModule::InitSpeaker()
	{
		CHECKinitialized_();
		return -1;
	}

	bool FAudioDeviceModule::SpeakerIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return false;
	}

	int32 FAudioDeviceModule::InitMicrophone()
	{
		CHECKinitialized_();
		return 0;
	}

	bool FAudioDeviceModule::MicrophoneIsInitialized() const
	{
		CHECKinitialized__BOOL();
		return true;
	}

	int32 FAudioDeviceModule::SpeakerVolumeIsAvailable(bool* available)
	{
		return -1;
	}
	int32 FAudioDeviceModule::SetSpeakerVolume(uint32 volume)
	{
		return -1;
	}
	int32 FAudioDeviceModule::SpeakerVolume(uint32* volume) const
	{
		return -1;
	}
	int32 FAudioDeviceModule::MaxSpeakerVolume(uint32* maxVolume) const
	{
		return -1;
	}
	int32 FAudioDeviceModule::MinSpeakerVolume(uint32* minVolume) const
	{
		return -1;
	}

	int32 FAudioDeviceModule::MicrophoneVolumeIsAvailable(bool* available)
	{
		return 0;
	}
	int32 FAudioDeviceModule::SetMicrophoneVolume(uint32 volume)
	{
		return 0;
	}
	int32 FAudioDeviceModule::MicrophoneVolume(uint32* volume) const
	{
		return 0;
	}
	int32 FAudioDeviceModule::MaxMicrophoneVolume(uint32* maxVolume) const
	{
		*maxVolume = UE::PixelStreaming::FAudioSubmixCapturer::MaxVolumeLevel;
		return 0;
	}
	int32 FAudioDeviceModule::MinMicrophoneVolume(uint32* minVolume) const
	{
		return 0;
	}

	int32 FAudioDeviceModule::SpeakerMuteIsAvailable(bool* available)
	{
		return -1;
	}
	int32 FAudioDeviceModule::SetSpeakerMute(bool enable)
	{
		return -1;
	}
	int32 FAudioDeviceModule::SpeakerMute(bool* enabled) const
	{
		return -1;
	}

	int32 FAudioDeviceModule::MicrophoneMuteIsAvailable(bool* available)
	{
		*available = false;
		return -1;
	}
	int32 FAudioDeviceModule::SetMicrophoneMute(bool enable)
	{
		return -1;
	}
	int32 FAudioDeviceModule::MicrophoneMute(bool* enabled) const
	{
		return -1;
	}

	int32 FAudioDeviceModule::StereoPlayoutIsAvailable(bool* available) const
	{
		CHECKinitialized_();
		*available = true;
		return 0;
	}

	int32 FAudioDeviceModule::SetStereoPlayout(bool enable)
	{
		CHECKinitialized_();
		FString AudioChannelStr = enable ? TEXT("stereo") : TEXT("mono");
		UE_LOGF(LogPixelStreaming, Log, "WebRTC has requested audio playout in UE be: %ls", *AudioChannelStr);
		return 0;
	}

	int32 FAudioDeviceModule::StereoPlayout(bool* enabled) const
	{
		CHECKinitialized_();
		*enabled = true;
		return 0;
	}

	int32 FAudioDeviceModule::StereoRecordingIsAvailable(bool* available) const
	{
		CHECKinitialized_();
		*available = true;
		return 0;
	}

	int32 FAudioDeviceModule::SetStereoRecording(bool enable)
	{
		CHECKinitialized_();
		return 0;
	}

	int32 FAudioDeviceModule::StereoRecording(bool* enabled) const
	{
		CHECKinitialized_();
		*enabled = true;
		return 0;
	}

	int32 FAudioDeviceModule::PlayoutDelay(uint16* delayMS) const
	{
		*delayMS = 0;
		return 0;
	}

	bool FAudioDeviceModule::BuiltInAECIsAvailable() const
	{
		return false;
	}
	bool FAudioDeviceModule::BuiltInAGCIsAvailable() const
	{
		return false;
	}
	bool FAudioDeviceModule::BuiltInNSIsAvailable() const
	{
		return false;
	}

	int32 FAudioDeviceModule::EnableBuiltInAEC(bool enable)
	{
		return -1;
	}
	int32 FAudioDeviceModule::EnableBuiltInAGC(bool enable)
	{
		return -1;
	}
	int32 FAudioDeviceModule::EnableBuiltInNS(bool enable)
	{
		return -1;
	}

}