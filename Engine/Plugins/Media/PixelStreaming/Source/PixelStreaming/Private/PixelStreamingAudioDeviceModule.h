// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingWebRTCIncludes.h"
#include "AudioSubmixCapturer.h"
#include "AudioPlayoutRequester.h"
#include "HAL/ThreadSafeBool.h"

namespace UE::PixelStreaming
{
	class FAudioInputMixer;

	// A custom audio device module for WebRTC.
	// It lets us receive WebRTC audio in UE
	// and also transmit UE audio to WebRTC.
	class PIXELSTREAMING_API FAudioDeviceModule : public webrtc::AudioDeviceModule
	{
	public:
		/* Use this constructor if the only audio source we want is the UE submix */
		FAudioDeviceModule();

		/* Use this constructor if we want to mix the submix with our audio sources. */
		FAudioDeviceModule(TSharedPtr<UE::PixelStreaming::FAudioInputMixer> InMixer);
		virtual ~FAudioDeviceModule() = default;

	private:
		FThreadSafeBool bInitialized; // True when we setup capturer/playout requester.
		TSharedPtr<UE::PixelStreaming::FAudioSubmixCapturer, ESPMode::ThreadSafe> Capturer;
		TUniquePtr<UE::PixelStreaming::FAudioPlayoutRequester> Requester;
		TSharedPtr<UE::PixelStreaming::FAudioInputMixer> InputMixer;

	private:
		//
		// webrtc::AudioDeviceModule interface
		//
		int32 ActiveAudioLayer(AudioLayer* audioLayer) const override;
		int32 RegisterAudioCallback(webrtc::AudioTransport* audioCallback) override;

		// Main initialization and termination
		int32 Init() override;
		int32 Terminate() override;
		bool Initialized() const override;

		// Device enumeration
		int16 PlayoutDevices() override;
		int16 RecordingDevices() override;
		int32 PlayoutDeviceName(uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize]) override;
		int32 RecordingDeviceName(uint16 index, char name[webrtc::kAdmMaxDeviceNameSize], char guid[webrtc::kAdmMaxGuidSize]) override;

		// Device selection
		int32 SetPlayoutDevice(uint16 index) override;
		int32 SetPlayoutDevice(WindowsDeviceType device) override;
		int32 SetRecordingDevice(uint16 index) override;
		int32 SetRecordingDevice(WindowsDeviceType device) override;

		// Audio transport initialization
		int32 PlayoutIsAvailable(bool* available) override;
		int32 InitPlayout() override;
		bool PlayoutIsInitialized() const override;
		int32 RecordingIsAvailable(bool* available) override;
		int32 InitRecording() override;
		bool RecordingIsInitialized() const override;

		// Audio transport control
		virtual int32 StartPlayout() override;
		virtual int32 StopPlayout() override;

		// True when audio is being pulled by the instance.
		virtual bool Playing() const override;

		virtual int32 StartRecording() override;
		virtual int32 StopRecording() override;
		virtual bool Recording() const override;

		// Audio mixer initialization
		virtual int32 InitSpeaker() override;
		virtual bool SpeakerIsInitialized() const override;
		virtual int32 InitMicrophone() override;
		virtual bool MicrophoneIsInitialized() const override;

		// Speaker volume controls
		virtual int32 SpeakerVolumeIsAvailable(bool* available) override;
		virtual int32 SetSpeakerVolume(uint32 volume) override;
		virtual int32 SpeakerVolume(uint32* volume) const override;
		virtual int32 MaxSpeakerVolume(uint32* maxVolume) const override;
		virtual int32 MinSpeakerVolume(uint32* minVolume) const override;

		// Microphone volume controls
		virtual int32 MicrophoneVolumeIsAvailable(bool* available) override;
		virtual int32 SetMicrophoneVolume(uint32 volume) override;
		virtual int32 MicrophoneVolume(uint32* volume) const override;
		virtual int32 MaxMicrophoneVolume(uint32* maxVolume) const override;
		virtual int32 MinMicrophoneVolume(uint32* minVolume) const override;

		// Speaker mute control
		virtual int32 SpeakerMuteIsAvailable(bool* available) override;
		virtual int32 SetSpeakerMute(bool enable) override;
		virtual int32 SpeakerMute(bool* enabled) const override;

		// Microphone mute control
		virtual int32 MicrophoneMuteIsAvailable(bool* available) override;
		virtual int32 SetMicrophoneMute(bool enable) override;
		virtual int32 MicrophoneMute(bool* enabled) const override;

		// Stereo support
		virtual int32 StereoPlayoutIsAvailable(bool* available) const override;
		virtual int32 SetStereoPlayout(bool enable) override;
		virtual int32 StereoPlayout(bool* enabled) const override;
		virtual int32 StereoRecordingIsAvailable(bool* available) const override;
		virtual int32 SetStereoRecording(bool enable) override;
		virtual int32 StereoRecording(bool* enabled) const override;

		// Playout delay
		virtual int32 PlayoutDelay(uint16* delayMS) const override;
		virtual bool BuiltInAECIsAvailable() const override;
		virtual bool BuiltInAGCIsAvailable() const override;
		virtual bool BuiltInNSIsAvailable() const override;

		// Enables the built-in audio effects. Only supported on Android.
		virtual int32 EnableBuiltInAEC(bool enable) override;
		virtual int32 EnableBuiltInAGC(bool enable) override;
		virtual int32 EnableBuiltInNS(bool enable) override;
	};
}