// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingBlueprints.h"
#include "PixelStreamingPrivate.h"
#include "Misc/FileHelper.h"
#include "PixelStreamingInputComponent.h"
#include "PixelStreamingAudioComponent.h"
#include "PixelStreamingPlayerId.h"
#include "IPixelStreamingStreamer.h"

void UPixelStreamingBlueprints::SendFileAsByteArray(TArray<uint8> ByteArray, FString MimeType, FString FileExtension)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return;
	}
	const TArray64<uint8> LargeByteArray(ByteArray);
	Streamer->SendFileData(LargeByteArray, MimeType, FileExtension);
}

void UPixelStreamingBlueprints::StreamerSendFileAsByteArray(FString StreamerId, TArray<uint8> ByteArray, FString MimeType, FString FileExtension)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(StreamerId);
	if (!Streamer)
	{
		return;
	}
	const TArray64<uint8> LargeByteArray(ByteArray);
	Streamer->SendFileData(LargeByteArray, MimeType, FileExtension);
}

void UPixelStreamingBlueprints::SendFile(FString FilePath, FString MimeType, FString FileExtension)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return;
	}

	TArray64<uint8> ByteData;
	bool			bSuccess = FFileHelper::LoadFileToArray(ByteData, *FilePath);
	if (bSuccess)
	{
		Streamer->SendFileData(ByteData, MimeType, FileExtension);
	}
	else
	{
		UE_LOGF(LogPixelStreaming, Error, "FileHelper failed to load file data");
	}
}

void UPixelStreamingBlueprints::StreamerSendFile(FString StreamerId, FString FilePath, FString MimeType, FString FileExtension)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(StreamerId);
	if (!Streamer)
	{
		return;
	}

	TArray64<uint8> ByteData;
	bool			bSuccess = FFileHelper::LoadFileToArray(ByteData, *FilePath);
	if (bSuccess)
	{
		Streamer->SendFileData(ByteData, MimeType, FileExtension);
	}
	else
	{
		UE_LOGF(LogPixelStreaming, Error, "FileHelper failed to load file data");
	}
}

void UPixelStreamingBlueprints::ForceKeyFrame()
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return;
	}
	Streamer->ForceKeyFrame();
}

void UPixelStreamingBlueprints::FreezeFrame(UTexture2D* Texture)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return;
	}
	Streamer->FreezeStream(Texture);
}

void UPixelStreamingBlueprints::StreamerFreezeStream(FString StreamerId, UTexture2D* Texture)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(StreamerId);
	if (!Streamer)
	{
		return;
	}
	Streamer->FreezeStream(Texture);
}

void UPixelStreamingBlueprints::UnfreezeFrame()
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return;
	}
	Streamer->UnfreezeStream();
}

void UPixelStreamingBlueprints::StreamerUnfreezeStream(FString StreamerId)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(StreamerId);
	if (!Streamer)
	{
		return;
	}
	Streamer->UnfreezeStream();
}

void UPixelStreamingBlueprints::KickPlayer(FString PlayerId)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return;
	}
	Streamer->KickPlayer(ToPlayerId(PlayerId));
}

void UPixelStreamingBlueprints::StreamerKickPlayer(FString StreamerId, FString PlayerId)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(StreamerId);
	if (!Streamer)
	{
		return;
	}
	Streamer->KickPlayer(ToPlayerId(PlayerId));
}

void UPixelStreamingBlueprints::SetPlayerLayerPreference(FString PlayerId, int SpatialLayerId, int TemporalLayerId)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return;
	}
	Streamer->SetPlayerLayerPreference(ToPlayerId(PlayerId), SpatialLayerId, TemporalLayerId);
}

void UPixelStreamingBlueprints::StreamerSetPlayerLayerPreference(FString StreamerId, FString PlayerId, int SpatialLayerId, int TemporalLayerId)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return;
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(StreamerId);
	if (!Streamer)
	{
		return;
	}
	Streamer->SetPlayerLayerPreference(ToPlayerId(PlayerId), SpatialLayerId, TemporalLayerId);
}

TArray<FString> UPixelStreamingBlueprints::GetConnectedPlayers()
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return {};
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(Module->GetDefaultStreamerID());
	if (!Streamer)
	{
		return {};
	}
	return Streamer->GetConnectedPlayers();
}

TArray<FString> UPixelStreamingBlueprints::StreamerGetConnectedPlayers(FString StreamerId)
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return {};
	}
	TSharedPtr<IPixelStreamingStreamer> Streamer = Module->FindStreamer(StreamerId);
	if (!Streamer)
	{
		return {};
	}
	return Streamer->GetConnectedPlayers();
}

FString UPixelStreamingBlueprints::GetDefaultStreamerID()
{
	IPixelStreamingModule* Module = UE::PixelStreaming::FPixelStreamingModule::GetModule();
	if (!Module)
	{
		return FString();
	}
	return Module->GetDefaultStreamerID();
}

UPixelStreamingDelegates* UPixelStreamingBlueprints::GetPixelStreamingDelegates()
{
	return UPixelStreamingDelegates::GetPixelStreamingDelegates();
}