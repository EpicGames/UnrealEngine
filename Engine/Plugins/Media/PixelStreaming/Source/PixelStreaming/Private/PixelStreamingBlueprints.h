// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPixelStreamingModule.h"
#include "PixelStreamingModule.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Containers/Array.h"
#include "PixelStreamingDelegates.h"
#include "PixelStreamingBlueprints.generated.h"

UCLASS()
class PIXELSTREAMING_API UPixelStreamingBlueprints : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	 * Send a specified byte array over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param   ByteArray       The raw data that will be sent over the data channel
	 * @param   MimeType        The mime type of the file. Used for reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void SendFileAsByteArray(TArray<uint8> ByteArray, FString MimeType, FString FileExtension);

	/**
	 * Send a specified byte array over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param	StreamerId		The streamer use when sending the data
	 * @param   ByteArray       The raw data that will be sent over the data channel
	 * @param   MimeType        The mime type of the file. Used for reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void StreamerSendFileAsByteArray(FString StreamerId, TArray<uint8> ByteArray, FString MimeType, FString FileExtension);

	/**
	 * Send a specified file over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param   FilePath        The path to the file that will be sent
	 * @param   MimeType        The mime type of the file. Used for file reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void SendFile(FString Filepath, FString MimeType, FString FileExtension);

	/**
	 * Send a specified file over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param	StreamerId		The streamer use when sending the data
	 * @param   FilePath        The path to the file that will be sent
	 * @param   MimeType        The mime type of the file. Used for file reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void StreamerSendFile(FString StreamerId, FString Filepath, FString MimeType, FString FileExtension);

	/**
	 * Force a key frame to be sent.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void ForceKeyFrame();

	/**
	 * The functions allow Pixel Streaming to be frozen and unfrozen from
	 * Blueprint. When frozen, a freeze frame (a still image) will be used by the
	 * browser instead of the video stream.
	 */

	/**
	 * Freeze Pixel Streaming.
	 * @param   Texture         The freeze frame to display. If null then the back buffer is captured.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void FreezeFrame(UTexture2D* Texture);

	/**
	 * Unfreeze Pixel Streaming.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void UnfreezeFrame();

	/**
	 * Freeze Pixel Streaming.
	 * @param	StreamerId		The id of the streamer to freeze.
	 * @param   Texture         The freeze frame to display. If null then the back buffer is captured.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void StreamerFreezeStream(FString StreamerId, UTexture2D* Texture);

	/**
	 * Unfreeze Pixel Streaming.
	 * @param StreamerId		The id of the streamer to unfreeze.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void StreamerUnfreezeStream(FString StreamerId);

	/**
	 * Kick a player.
	 * @param   PlayerId         The ID of the player to kick.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void KickPlayer(FString PlayerId);

	/**
	 * Kick a player.
	 * @param	StreamerId		The streamer which the player belongs
	 * @param   PlayerId        The ID of the player to kick.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void StreamerKickPlayer(FString StreamerId, FString PlayerId);

	/**
	 * Set the layer preference for a specific player
	 * @param PlayerId			The ID of the player to set the layer preference for.
	 * @param SpatialLayerId	The spatial layer ID
	 * @param TemporalLayerId	The temporal layer ID
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void SetPlayerLayerPreference(FString PlayerId, int SpatialLayerId, int TemporalLayerId);

	/**
	 * Set the layer preference for a specific player
	 * @param StreamerId 		The streamer which the player belongs
	 * @param PlayerId			The ID of the player to set the layer preference for.
	 * @param SpatialLayerId	The spatial layer ID
	 * @param TemporalLayerId	The temporal layer ID
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static void StreamerSetPlayerLayerPreference(FString StreamerId, FString PlayerId, int SpatialLayerId, int TemporalLayerId);

	/**
	 * @brief Get the connected players
	 *
	 * @return TArray<FString> The connected players
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static TArray<FString> GetConnectedPlayers();

	/**
	 * @brief Get the connected players
	 *
	 * @param StreamerId	The streamer whose list of players you wish to get
	 * @return TArray<FString> The connected players
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static TArray<FString> StreamerGetConnectedPlayers(FString StreamerId);

	/**
	 * Get the default Streamer ID
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static FString GetDefaultStreamerID();

	// PixelStreamingDelegates
	/**
	 * Get the singleton. This allows application-specific blueprints to bind
	 * to delegates of interest.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming")
	static UPixelStreamingDelegates* GetPixelStreamingDelegates();
};