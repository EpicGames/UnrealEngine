// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingDataChannel.h"
#include "PixelStreamingPeerConnection.h"
#include "PixelStreamingPrivate.h"
#include "Utils.h"

TSharedPtr<FPixelStreamingDataChannel> FPixelStreamingDataChannel::Create(rtc::scoped_refptr<webrtc::DataChannelInterface> InChannel)
{
	return TSharedPtr<FPixelStreamingDataChannel>(new FPixelStreamingDataChannel(InChannel));
}

TSharedPtr<FPixelStreamingDataChannel> FPixelStreamingDataChannel::Create(FPixelStreamingPeerConnection& Connection, int32 SendStreamId, int32 RecvStreamId)
{
	return TSharedPtr<FPixelStreamingDataChannel>(new FPixelStreamingDataChannel(Connection, SendStreamId, RecvStreamId));
}

FPixelStreamingDataChannel::FPixelStreamingDataChannel(rtc::scoped_refptr<webrtc::DataChannelInterface> InChannel)
	: SendChannel(InChannel)
	, RecvChannel(InChannel)
{
	checkf(RecvChannel, TEXT("Channel cannot be null"));
	RecvChannel->RegisterObserver(this);
}

FPixelStreamingDataChannel::FPixelStreamingDataChannel(FPixelStreamingPeerConnection& Connection, int32 SendStreamId, int32 RecvStreamId)
{
	rtc::scoped_refptr<webrtc::PeerConnectionInterface> PeerConnection = Connection.PeerConnection;

	webrtc::DataChannelInit SendConfig;
	SendConfig.negotiated = true;
	SendConfig.id = SendStreamId;

	webrtc::RTCErrorOr<rtc::scoped_refptr<webrtc::DataChannelInterface>> SendRecvResult = PeerConnection->CreateDataChannelOrError((SendStreamId == RecvStreamId) ? "datachannel" : "senddatachannel", &SendConfig);
	checkf(SendRecvResult.ok(), TEXT("Failed to create Data Channel. Msg=%s"), *FString(SendRecvResult.error().message()));
	RecvChannel = SendChannel = SendRecvResult.MoveValue();

	if (SendStreamId != RecvStreamId)
	{
		webrtc::DataChannelInit RecvConfig;
		RecvConfig.negotiated = true;
		RecvConfig.id = RecvStreamId;

		webrtc::RTCErrorOr<rtc::scoped_refptr<webrtc::DataChannelInterface>> RecvResult = PeerConnection->CreateDataChannelOrError("recvdatachannel", &RecvConfig);
		checkf(RecvResult.ok(), TEXT("Failed to create Data Channel. Msg=%s"), *FString(RecvResult.error().message()));
		RecvChannel = RecvResult.MoveValue();
	}

	checkf(SendChannel, TEXT("Send channel cannot be null"));
	checkf(RecvChannel, TEXT("Recv channel cannot be null"));
	RecvChannel->RegisterObserver(this);
}

FPixelStreamingDataChannel::~FPixelStreamingDataChannel()
{
	RecvChannel->UnregisterObserver();
}

bool FPixelStreamingDataChannel::SendArbitraryData(uint8 Type, const TArray64<uint8>& DataBytes) const
{
	using namespace UE::PixelStreaming;

	if (!SendChannel)
	{
		UE_LOGF(LogPixelStreaming, Error, "Cannot send arbitrary data when data channel is null.");
		return false;
	}

	// int32 results in a maximum 4GB file (4,294,967,296 bytes)
	const int32 DataSize = DataBytes.Num();

	// Maximum size of a single buffer should be 16KB as this is spec compliant message length for a single data channel transmission
	const int32 MaxBufferBytes = 16 * 1024;
	const int32 MessageHeader = sizeof(Type) + sizeof(DataSize);
	const int32 MaxDataBytesPerMsg = MaxBufferBytes - MessageHeader;

	int32 BytesTransmitted = 0;

	while (BytesTransmitted < DataSize)
	{
		int32 RemainingBytes = DataSize - BytesTransmitted;
		int32 BytesToTransmit = FGenericPlatformMath::Min(MaxDataBytesPerMsg, RemainingBytes);

		rtc::CopyOnWriteBuffer Buffer(MessageHeader + BytesToTransmit);

		size_t Pos = 0;

		// Write message type
		Pos = SerializeToBuffer(Buffer, Pos, &Type, sizeof(Type));

		// Write size of payload
		Pos = SerializeToBuffer(Buffer, Pos, &DataSize, sizeof(DataSize));

		// Write the data bytes payload
		Pos = SerializeToBuffer(Buffer, Pos, DataBytes.GetData() + BytesTransmitted, BytesToTransmit);

		uint64_t BufferBefore = SendChannel->buffered_amount();
		while (BufferBefore + BytesToTransmit >= 16 * 1024 * 1024) // 16MB (WebRTC Data Channel buffer size)
		{
			// As per UE docs a Sleep of 0.0 simply lets other threads take CPU cycles while this is happening.
			FPlatformProcess::Sleep(0.0);
			BufferBefore = SendChannel->buffered_amount();
		}

		if (!SendChannel->Send(webrtc::DataBuffer(Buffer, true)))
		{
			UE_LOGF(LogPixelStreaming, Error, "Failed to send data channel packet");
			return false;
		}

		// Increment the number of bytes transmitted
		BytesTransmitted += BytesToTransmit;
	}
	return true;
}

void FPixelStreamingDataChannel::OnStateChange()
{
	// ideally we use AsShared() here so we either get a shared ptr to this or it fails
	// (because the destructor is being called in another thread) and we do nothing. If
	// it succeeds then we know we wont get destructed while in the block.
	// HOWEVER! inside AsShared() is a check to make sure the returned pointer is 'this'
	// which it is NOT if we're in the destructor, SO we have to use DoesSharedInstanceExist
	// FIRST to avoid hitting that check BUT in THEORY the destructor could get called
	// after DoesSharedInstanceExist but before AsShared and so we end up in the same
	// issue. That check really screws this situation.
	if (DoesSharedInstanceExist())
	{
		// make sure we dont destruct while in here
		if (TSharedPtr<FPixelStreamingDataChannel> SharedThis = AsShared())
		{
			// Dispatch this callback to the game thread since we don't want to delay
			// the signalling thread or block it with mutexes etc.
			TWeakPtr<FPixelStreamingDataChannel> WeakChannel = SharedThis;
			webrtc::DataChannelInterface::DataState NewState = RecvChannel->state();
			AsyncTask(ENamedThreads::GameThread, [WeakChannel, NewState]() {
				if (TSharedPtr<FPixelStreamingDataChannel> DataChannel = WeakChannel.Pin())
				{
					switch (NewState)
					{
						case webrtc::DataChannelInterface::DataState::kOpen:
						{
							DataChannel->OnOpen.Broadcast(*DataChannel);
							break;
						}
						case webrtc::DataChannelInterface::DataState::kConnecting:
						case webrtc::DataChannelInterface::DataState::kClosing:
							break;
						case webrtc::DataChannelInterface::DataState::kClosed:
						{
							DataChannel->OnClosed.Broadcast(*DataChannel);
							break;
						}
					}
				}
			});
		}
	}
}

void FPixelStreamingDataChannel::OnMessage(const webrtc::DataBuffer& Buffer)
{
	// See comment in OnStateChange()
	if (DoesSharedInstanceExist())
	{
		if (TSharedPtr<FPixelStreamingDataChannel> SharedThis = AsShared())
		{
			// Dispatch this callback to the game thread (or Main Queue) since we don't want to delay
			// the signalling thread or block it with mutexes etc.
			TWeakPtr<FPixelStreamingDataChannel> WeakChannel = SharedThis;

			// If we're streaming the editor and we hit a BP breakpoint, the gamethread is no longer able to respond to input
			// in that case we post this task to the Main Queue as we know that that will still be running
			ENamedThreads::Type TargetThread = (GFirstFrameIntraFrameDebugging || GIntraFrameDebuggingGameThread) ? ENamedThreads::MainQueue : ENamedThreads::GameThread;
			AsyncTask(TargetThread, [WeakChannel, Buffer = Buffer]() 
			{
				if (TSharedPtr<FPixelStreamingDataChannel> DataChannel = WeakChannel.Pin())
				{
					const uint8 MsgType = static_cast<uint8>(Buffer.data.data()[0]);
					DataChannel->OnMessageReceived.Broadcast(MsgType, Buffer);
				}
			});
		}
	}
}
