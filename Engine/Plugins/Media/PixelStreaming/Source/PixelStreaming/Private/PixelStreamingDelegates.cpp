// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingDelegates.h"

UPixelStreamingDelegates* UPixelStreamingDelegates::Singleton = nullptr;
bool UPixelStreamingDelegates::bIsExiting = false;

UPixelStreamingDelegates* UPixelStreamingDelegates::CreateInstance()
{
	if (Singleton == nullptr && !bIsExiting)
	{
		Singleton = NewObject<UPixelStreamingDelegates>();
		Singleton->AddToRoot();
		return Singleton;
	}
	return Singleton;
}
