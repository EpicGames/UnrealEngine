// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputComponent.h"
#include "IPixelStreamingModule.h"
#include "PixelStreamingModule.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "GameFramework/GameUserSettings.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingInputProtocol.h"
#include "IPixelStreamingStreamer.h"
#include "Settings.h"
#include "PixelStreamingUtils.h"

UPixelStreamingInput::UPixelStreamingInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PixelStreamingModule(UE::PixelStreaming::FPixelStreamingModule::GetModule())
{
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
	SetComponentTickEnabled(true);
}

void UPixelStreamingInput::BeginPlay()
{
	Super::BeginPlay();

	if (PixelStreamingModule)
	{
		// When this component is initializing it registers itself with the Pixel Streaming module.
		PixelStreamingModule->AddInputComponent(this);
	}
	else
	{
		UE_LOGF(LogPixelStreaming, Warning, "Pixel Streaming input component not added because Pixel Streaming module is not loaded. This is expected on dedicated servers.");
	}
}

void UPixelStreamingInput::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	if (PixelStreamingModule)
	{
		// When this component is destructing it unregisters itself with the Pixel Streaming module.
		PixelStreamingModule->RemoveInputComponent(this);
	}
	else
	{
		UE_LOGF(LogPixelStreaming, Warning, "Pixel Streaming input component not removed because Pixel Streaming module is not loaded. This is expected on dedicated servers.");
	}
}

void UPixelStreamingInput::SendPixelStreamingResponse(const FString& Descriptor)
{
	if (PixelStreamingModule)
	{
		PixelStreamingModule->ForEachStreamer([&Descriptor, this](TSharedPtr<IPixelStreamingStreamer> Streamer) {
			Streamer->SendPlayerMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("Response")->GetID(), Descriptor);
		});
	}
	else
	{
		UE_LOGF(LogPixelStreaming, Warning, "Pixel Streaming input component skipped sending response. This is expected on dedicated servers.");
	}
}

void UPixelStreamingInput::GetJsonStringValue(FString Descriptor, FString FieldName, FString& StringValue, bool& Success)
{
	UE::PixelStreaming::ExtractJsonFromDescriptor(Descriptor, FieldName, StringValue, Success);
}

void UPixelStreamingInput::AddJsonStringValue(const FString& Descriptor, FString FieldName, FString StringValue, FString& NewDescriptor, bool& Success)
{
	UE::PixelStreaming::ExtendJsonWithField(Descriptor, FieldName, StringValue, NewDescriptor, Success);
}
