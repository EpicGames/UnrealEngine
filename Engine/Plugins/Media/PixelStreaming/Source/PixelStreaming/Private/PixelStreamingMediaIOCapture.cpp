// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingMediaIOCapture.h"
#include "PixelStreamingVideoInputRHI.h"
#include "PixelCaptureInputFrameRHI.h"
#include "Slate/SceneViewport.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingCodec.h"
#include "PixelStreamingModule.h"
#include "RHI.h"

#include "MediaShaders.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "RenderGraphUtils.h"

void UPixelStreamingMediaIOCapture::OnRHIResourceCaptured_RenderingThread(
	FRHICommandListImmediate& /*RHICmdList*/,
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	FTextureRHIRef										   InTexture)
{
	HandleCapturedFrame(InTexture);
}

void UPixelStreamingMediaIOCapture::OnRHIResourceCaptured_AnyThread(
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	FTextureRHIRef										   InTexture)
{
	HandleCapturedFrame(InTexture);
}

void UPixelStreamingMediaIOCapture::OnFrameCaptured_RenderingThread(
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	void*												   InBuffer,
	int32												   Width,
	int32												   Height,
	int32												   BytesPerRow)
{
	UpdateCaptureResolution(Width, Height);
	// Todo: implement this if we want to support cpu readback captures
}

void UPixelStreamingMediaIOCapture::OnCustomCapture_RenderingThread(
	FRDGBuilder&										   GraphBuilder,
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	FRDGTextureRef										   InSourceTexture,
	FRDGTextureRef										   OutputTexture,
	const FRHICopyTextureInfo&							   CopyInfo,
	FVector2D											   CropU,
	FVector2D											   CropV)
{
	FGPUFenceRHIRef Fence = GDynamicRHI->RHICreateGPUFence(TEXT("UPixelStreamingMediaIOCapture Fence"));
	if (UseExperimentalScheduling() && ShouldCaptureRHIResource())
	{
		Fences.Add(reinterpret_cast<uintptr_t>(OutputTexture->GetRHI()), Fence);
	}

	bool bRequiresFormatConversion = InSourceTexture->Desc.Format != OutputTexture->Desc.Format;
	if (InSourceTexture->Desc.Format == OutputTexture->Desc.Format && InSourceTexture->Desc.Extent.X == OutputTexture->Desc.Extent.X && InSourceTexture->Desc.Extent.Y == OutputTexture->Desc.Extent.Y)
	{
		// The formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			InSourceTexture,
			OutputTexture,
			FRDGDrawTextureInfo());
	}
	else
	{
#if PLATFORM_MAC
		// Create a staging texture that is the same size and format as the final.
		FRDGTextureRef			   StagingTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y), OutputTexture->Desc.Format, OutputTexture->Desc.ClearValue, ETextureCreateFlags::RenderTargetable), TEXT("PixelStreamingMediaIOCapture Staging"));
		FScreenPassTextureViewport StagingViewport(StagingTexture);
#endif

		FScreenPassTextureViewport InputViewport(InSourceTexture);
		FScreenPassTextureViewport OutputViewport(OutputTexture);

		FGlobalShaderMap*			 GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

		// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
		int32										  MediaConversionOperation = 0; // None
		FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(MediaConversionOperation);

		// Rectangle area to use from source
		const FIntRect ViewRect(FIntPoint(0, 0), InSourceTexture->Desc.Extent);

		TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
		FModifyAlphaSwizzleRgbaPS::FParameters*	 PixelShaderParameters = PixelShader->AllocateAndSetParameters(
			 GraphBuilder,
			 InSourceTexture,
#if PLATFORM_MAC
			StagingTexture
#else
			OutputTexture
#endif
		);

		FRHIBlendState*		   BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("PixelStreamingMediaIOCapture Swizzle"),
			FScreenPassViewInfo(),
#if PLATFORM_MAC
			StagingViewport,
#else
			OutputViewport,
#endif

			InputViewport,
			VertexShader,
			PixelShader,
			PixelShaderParameters);

#if PLATFORM_MAC
		// Now we can be certain the formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			StagingTexture,
			OutputTexture,
			FRDGDrawTextureInfo());
#endif
	}

	// clang-format off
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("UPixelStreamingMediaIOCapture WriteFence"), 
		ERDGPassFlags::NeverCull,								
		[Fence](FRDGAsyncTask, FRHICommandList& RHICmdList) {
			RHICmdList.WriteGPUFence(Fence);
		});
	// clang-format on
}

bool UPixelStreamingMediaIOCapture::InitializeCapture()
{
	UE_LOGF(LogPixelStreaming, Log, "Initializing media capture for Pixel Streaming VCam.");
	bViewportResized = false;
	bDoGPUCopy = true;

	SetState(EMediaCaptureState::Capturing);

	return true;
}

void UPixelStreamingMediaIOCapture::StopCaptureImpl(bool bAllowPendingFrameToBeProcess)
{
	// Todo: Any cleanup on capture stop should happen here.
}

// This will activate the _AnyThread method calls when true.
bool UPixelStreamingMediaIOCapture::SupportsAnyThreadCapture() const
{
	EPixelStreamingCodec SelectedCodec = IPixelStreamingModule::Get().GetCodec();
	// If we are using VP8 or VP9 we want to ensure capture happens on the render thread as we do our capture/convert to I420 there
	bool bForceRenderThread = SelectedCodec == EPixelStreamingCodec::VP8 || SelectedCodec == EPixelStreamingCodec::VP9;
	return bForceRenderThread == false;
}

ETextureCreateFlags UPixelStreamingMediaIOCapture::GetOutputTextureFlags() const
{
#if PLATFORM_MAC
	return TexCreate_CPUReadback;
#else
	ETextureCreateFlags Flags = TexCreate_RenderTargetable | TexCreate_UAV;

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		Flags |= TexCreate_External;
	}
	else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		Flags |= TexCreate_Shared;
	}

	return Flags;
#endif
}

void UPixelStreamingMediaIOCapture::WaitForGPU(FRHITexture* InRHITexture)
{
	FGPUFenceRHIRef Fence;
	if (!Fences.RemoveAndCopyValue(reinterpret_cast<uintptr_t>(InRHITexture), Fence))
	{
		UE_LOGF(LogPixelStreaming, Error, "UPixelStreamingMediaIOCapture::WaitForGPU: No associating fence!");
		return;
	}

	while (!Fence->Poll())
	{
		// Exit if capture has been stopped (e.g. during PIE teardown). Without this check the
		// loop polls forever, blocking the SyncPointWatcher thread. Kill(bWaitForThread=true)
		// in ~FSyncPointWatcher() then deadlocks the game thread in PostCaptureCleanup.
		if (GetState() != EMediaCaptureState::Capturing)
		{
			break;
		}
		constexpr float SleepTimeSeconds = 50 * 1E-6f;
		FPlatformProcess::SleepNoStats(SleepTimeSeconds);
	}
}

bool UPixelStreamingMediaIOCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	SceneViewport = TWeakPtr<FSceneViewport>(InSceneViewport);
	OnCaptureViewportInitialized.Broadcast();

	// Listen for viewport resize events as resizes invalidate media capture, so we want to know when to reset capture
	InSceneViewport->ViewportResizedEvent.AddUObject(this, &UPixelStreamingMediaIOCapture::ViewportResized);

	return true;
}

void UPixelStreamingMediaIOCapture::ViewportResized(FViewport* Viewport, uint32 ResizeCode)
{
	bViewportResized = true;

	// If we have not captured a frame yet, we don't care about stopping the capture due to capture size mismatch
	if (!CaptureResolution)
	{
		return;
	}

	// If resolution of viewport is actually the same as the capture resolution no need to stop/restart capturing.
	if (Viewport->GetSizeXY() == *CaptureResolution)
	{
		return;
	}

	if (GetState() == EMediaCaptureState::Capturing)
	{
		UE_LOGF(LogPixelStreaming, Warning, "Stopping PixelStreaming MediaIO capture because viewport was resized.");
		StopCapture(false);
	}
}

void UPixelStreamingMediaIOCapture::HandleCapturedFrame(FTextureRHIRef InTexture)
{
	TSharedPtr<FPixelStreamingVideoInput> VideoInputPtr = VideoInput.Pin();
	if (VideoInputPtr)
	{
		UpdateCaptureResolution(InTexture->GetDesc().Extent.X, InTexture->GetDesc().Extent.Y);
		VideoInputPtr->OnFrame(FPixelCaptureInputFrameRHI(InTexture));
	}
}

void UPixelStreamingMediaIOCapture::UpdateCaptureResolution(int32 Width, int32 Height)
{
	if (!CaptureResolution)
	{
		CaptureResolution = MakeUnique<FIntPoint>();
	}
	CaptureResolution->X = Width;
	CaptureResolution->Y = Height;
}
