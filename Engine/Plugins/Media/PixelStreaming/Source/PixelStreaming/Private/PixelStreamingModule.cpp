// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingModule.h"
#include "IPixelStreamingInputModule.h"
#include "Streamer.h"
#include "PixelStreamingInputComponent.h"
#include "PixelStreamingDelegates.h"
#include "PixelStreamingSignallingConnection.h"
#include "Settings.h"
#include "PixelStreamingPrivate.h"
#include "AudioSink.h"
#include "CoreMinimal.h"
#include "Engine/GameViewportClient.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Texture2D.h"
#include "Slate/SceneViewport.h"
#include "PixelStreamingUtils.h"
#include "PixelStreamingCoderUtils.h"
#include "Utils.h"

#if PLATFORM_LINUX
	#include "CudaModule.h"
#endif

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <VersionHelpers.h>
THIRD_PARTY_INCLUDES_END
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "RenderingThread.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "RendererInterface.h"
#include "Rendering/SlateRenderer.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/ConfigCacheIni.h"
#include "GameFramework/GameModeBase.h"
#include "Dom/JsonObject.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Async/Async.h"
#include "Engine/Engine.h"
#include "VideoEncoderFactoryLayered.h"
#include "WebRTCLogging.h"
#include "WebSocketsModule.h"

#if !UE_BUILD_SHIPPING
	#include "DrawDebugHelpers.h"
#endif

#include "PixelStreamingVideoInputBackBuffer.h"
#include "PixelStreamingVideoInputMediaCapture.h"
#include "VideoSourceGroup.h"
#include "PixelStreamingPeerConnection.h"
#include "Engine/GameEngine.h"
#include "Stats.h"
#include "Video/Resources/VideoResourceRHI.h"
#include "PixelStreamingInputEnums.h"

DEFINE_LOG_CATEGORY(LogPixelStreaming);

IPixelStreamingModule* UE::PixelStreaming::FPixelStreamingModule::PixelStreamingModule = nullptr;

const FPixelStreamingPlayerId INVALID_PLAYER_ID = ToPlayerId(FString(TEXT("Invalid Player Id")));
const FPixelStreamingPlayerId SFU_PLAYER_ID = FString(TEXT("1"));

namespace UE::PixelStreaming
{
	/**
	 * IModuleInterface implementation
	 */
	void FPixelStreamingModule::StartupModule()
	{
#if UE_SERVER
		// Hack to no-op the rest of the module so Blueprints can still work
		return;
#else

		// Initialise all settings from command line args etc
		Settings::InitialiseSettings();

		// Pixel Streaming does not make sense without an RHI so we don't run in commandlets without one.
		if (IsRunningCommandlet() && !IsAllowCommandletRendering())
		{
			return;
		}

		if (!FSlateApplication::IsInitialized())
		{
			return;
		}

		const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
		// only D3D11/D3D12/Vulkan is supported
		if (!(RHIType == ERHIInterfaceType::D3D11 || RHIType == ERHIInterfaceType::D3D12 || RHIType == ERHIInterfaceType::Vulkan || RHIType == ERHIInterfaceType::Metal))
		{
#if !WITH_DEV_AUTOMATION_TESTS
			UE_LOGF(LogPixelStreaming, Warning, "Only D3D11/D3D12/Vulkan/Metal Dynamic RHI is supported. Detected %ls", GDynamicRHI != nullptr ? GDynamicRHI->GetName() : TEXT("[null]"));
#endif
			return;
		}

		// By calling InitDefaultStreamer post engine init we can use pixel streaming in standalone editor mode
		FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([this, RHIType]() {
			// Check to see if we can use the Pixel Streaming plugin on this platform.
			// If not then we avoid setting up our delegates to prevent access to the plugin.
			if (!IsPlatformCompatible())
			{
				return;
			}

			if (!ensure(GEngine != nullptr))
			{
				return;
			}

			// HACK (Luke): Until or if we ever find a workaround for fencing, we need to ensure capture always uses a fence
			// if we don't then we get frequent and intermittent stuttering as textures are rendered to while being encoded.
			// From testing NVENC + CUDA pathway seems acceptable without a fence in most cases so we use the faster, unsafer path there.
			if (RHIType == ERHIInterfaceType::D3D11 || IsRHIDeviceAMD())
			{
				Settings::CVarPixelStreamingCaptureUseFence.AsVariable()->Set(true);
			}

			FApp::SetUnfocusedVolumeMultiplier(1.0f);

			// Allow Pixel Streaming to broadcast to various delegates bound in the application-specific blueprint.
			UPixelStreamingDelegates::CreateInstance();

			// Ensure we have ImageWrapper loaded, used in Freezeframes
			verify(FModuleManager::Get().LoadModule(FName("ImageWrapper")));

			// We don't want to start immediately streaming in editor
			if (!GIsEditor)
			{
				InitDefaultStreamer();
				StartStreaming();
			}

			bModuleReady = true;
			ReadyEvent.Broadcast(*this);
		});

		rtc::InitializeSSL();
		RedirectWebRtcLogsToUnreal(rtc::LoggingSeverity::LS_VERBOSE);
		FModuleManager::LoadModuleChecked<FWebSocketsModule>("WebSockets");

		// ExternalVideoSourceGroup is used so that we can have a video source without a streamer
		ExternalVideoSourceGroup = FVideoSourceGroup::Create();
		// ExternalVideoSourceGroup->SetVideoInput(FPixelStreamingVideoInputBackBuffer::Create());
		// ExternalVideoSourceGroup->Start();

		// Call FStats::Get() to initialize the singleton
		FStats::Get();
		bStartupCompleted = true;
#endif // UE_SERVER
	}

	void FPixelStreamingModule::ShutdownModule()
	{
		if (!bStartupCompleted)
		{
			return;
		}

		// We explicitly call release on streamer so WebRTC gets shutdown before our module is deleted
		// additionally the streamer does a bunch of delegate calls and unbinds which seem to have issues
		// when called during engine destruction rather than here.
		Streamers.Empty();
		DefaultStreamer.Reset();
		ExternalVideoSourceGroup->Stop();

		FPixelStreamingPeerConnection::Shutdown();

		rtc::CleanupSSL();

		bStartupCompleted = false;
	}

	/**
	 * End IModuleInterface implementation
	 */

	/**
	 * IPixelStreamingModule implementation
	 */
	IPixelStreamingModule* FPixelStreamingModule::GetModule()
	{
		if (PixelStreamingModule)
		{
			return PixelStreamingModule;
		}
		IPixelStreamingModule* Module = FModuleManager::Get().LoadModulePtr<IPixelStreamingModule>("PixelStreaming");
		if (Module)
		{
			PixelStreamingModule = Module;
		}
		return PixelStreamingModule;
	}

	void FPixelStreamingModule::SetCodec(EPixelStreamingCodec Codec)
	{
		Settings::SetCodec(Codec);
	}

	EPixelStreamingCodec FPixelStreamingModule::GetCodec() const
	{
		return Settings::GetSelectedCodec();
	}

	IPixelStreamingModule::FReadyEvent& FPixelStreamingModule::OnReady()
	{
		return ReadyEvent;
	}

	bool FPixelStreamingModule::IsReady()
	{
		return bModuleReady;
	}

	bool FPixelStreamingModule::StartStreaming()
	{
		if (DefaultStreamer.IsValid())
		{
			DefaultStreamer->StartStreaming();
			return true;
		}

		return false;
	}

	void FPixelStreamingModule::StopStreaming()
	{
		if (DefaultStreamer.IsValid())
		{
			DefaultStreamer->StopStreaming();
		}
	}

	TSharedPtr<IPixelStreamingStreamer> FPixelStreamingModule::CreateStreamer(const FString& StreamerId)
	{
		TSharedPtr<IPixelStreamingStreamer> ExistingStreamer = FindStreamer(StreamerId);
		if (ExistingStreamer)
		{
			return ExistingStreamer;
		}

		TSharedPtr<FStreamer> NewStreamer = FStreamer::Create(StreamerId);
		{
			FScopeLock Lock(&StreamersCS);
			Streamers.Add(StreamerId, NewStreamer);
		}

		// Any time we create a new streamer, populate it's signalling server URL with whatever is on the command line
		FString SignallingServerURL;
		if (!Settings::GetSignallingServerUrl(SignallingServerURL))
		{
			// didnt get the startup URL for pixel streaming. Check deprecated options...
			FString SignallingServerIP;
			uint16 SignallingServerPort;
			if (Settings::GetSignallingServerIP(SignallingServerIP) && Settings::GetSignallingServerPort(SignallingServerPort))
			{
				// got both old parameters. Warn about deprecation and build the proper url.
				UE_LOGF(LogPixelStreaming, Warning, "PixelStreamingIP and PixelStreamingPort are deprecated flags. Use PixelStreamingURL instead. eg. -PixelStreamingURL=ws://%ls:%d", *SignallingServerIP, SignallingServerPort);
				SignallingServerURL = FString::Printf(TEXT("ws://%s:%d"), *SignallingServerIP, SignallingServerPort);
			}
		}
		NewStreamer->SetSignallingServerURL(SignallingServerURL);

		// Ensure that this new streamer is able to handle pixel streaming relevant input
		RegisterCustomHandlers(NewStreamer);

		return NewStreamer;
	}

	TArray<FString> FPixelStreamingModule::GetStreamerIds()
	{
		TArray<FString> StreamerKeys;
		FScopeLock Lock(&StreamersCS);
		Streamers.GenerateKeyArray(StreamerKeys);
		return StreamerKeys;
	}

	TSharedPtr<IPixelStreamingStreamer> FPixelStreamingModule::FindStreamer(const FString& StreamerId)
	{
		FScopeLock Lock(&StreamersCS);
		if (Streamers.Contains(StreamerId))
		{
			return Streamers[StreamerId].Pin();
		}
		return nullptr;
	}

	TSharedPtr<IPixelStreamingStreamer> FPixelStreamingModule::DeleteStreamer(const FString& StreamerId)
	{
		TSharedPtr<IPixelStreamingStreamer> ToBeDeleted;
		FScopeLock Lock(&StreamersCS);
		if (Streamers.Contains(StreamerId))
		{
			ToBeDeleted = Streamers[StreamerId].Pin();
			Streamers.Remove(StreamerId);
		}
		return ToBeDeleted;
	}

	void FPixelStreamingModule::DeleteStreamer(TSharedPtr<IPixelStreamingStreamer> ToBeDeleted)
	{
		FScopeLock Lock(&StreamersCS);
		for (auto& [Id, Streamer] : Streamers)
		{
			if (Streamer == ToBeDeleted)
			{
				Streamers.Remove(Id);
				break;
			}
		}
	}

	void FPixelStreamingModule::SetExternalVideoSourceFPS(uint32 InFPS)
	{
		ExternalVideoSourceGroup->SetFPS(InFPS);
	}

	void FPixelStreamingModule::SetExternalVideoSourceCoupleFramerate(bool bShouldCoupleFPS)
	{
		ExternalVideoSourceGroup->SetCoupleFramerate(bShouldCoupleFPS);
	}

	void FPixelStreamingModule::SetExternalVideoSourceInput(TSharedPtr<FPixelStreamingVideoInput> InVideoInput)
	{
		ExternalVideoSourceGroup->SetVideoInput(InVideoInput);
		ExternalVideoSourceGroup->Start();
	}

	rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> FPixelStreamingModule::CreateExternalVideoSource()
	{
		// New Video source will not work with stream sharing as bInRequireUniqueId is set to true
		return ExternalVideoSourceGroup->CreateVideoSource(true, []() { return true; });
	}

	void FPixelStreamingModule::ReleaseExternalVideoSource(const webrtc::VideoTrackSourceInterface* InVideoSource)
	{
		ExternalVideoSourceGroup->RemoveVideoSource(InVideoSource);
	}

	void FPixelStreamingModule::AddInputComponent(UPixelStreamingInput* InInputComponent)
	{
		InputComponents.Add(InInputComponent);
	}

	void FPixelStreamingModule::RemoveInputComponent(UPixelStreamingInput* InInputComponent)
	{
		InputComponents.Remove(InInputComponent);
	}

	const TArray<UPixelStreamingInput*> FPixelStreamingModule::GetInputComponents()
	{
		return InputComponents;
	}

	TUniquePtr<webrtc::VideoEncoderFactory> FPixelStreamingModule::CreateVideoEncoderFactory()
	{
		return MakeUnique<FVideoEncoderFactoryLayered>();
	}

	FString FPixelStreamingModule::GetDefaultStreamerID()
	{
		return Settings::GetDefaultStreamerID();
	}

	FString FPixelStreamingModule::GetDefaultSignallingURL()
	{
		return Settings::GetDefaultSignallingURL();
	}

	void FPixelStreamingModule::ForEachStreamer(const TFunction<void(TSharedPtr<IPixelStreamingStreamer>)>& Func)
	{
		TSet<FString> KeySet;
		{
			FScopeLock Lock(&StreamersCS);
			Streamers.GetKeys(KeySet);
		}
		for (auto&& StreamerId : KeySet)
		{
			if (TSharedPtr<IPixelStreamingStreamer> Streamer = FindStreamer(StreamerId))
			{
				Func(Streamer);
			}
		}
	}

	/**
	 * End IPixelStreamingModule implementation
	 */

	void FPixelStreamingModule::InitDefaultStreamer()
	{
		UE_LOGF(LogPixelStreaming, Log, "PixelStreaming streamer ID: %ls", *Settings::GetDefaultStreamerID());

		DefaultStreamer = CreateStreamer(Settings::GetDefaultStreamerID());
		// The PixelStreamingEditorModule handles setting video input in the editor
		if (!GIsEditor)
		{
			// default to the scene viewport if we have a game engine
			if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine); GameEngine != nullptr)
			{
				TSharedPtr<SWindow> TargetWindow = GameEngine->GameViewport ? GameEngine->GameViewport->GetWindow() : nullptr;
				if (!TargetWindow.IsValid())
				{
					TargetWindow = GameEngine->GameViewportWindow.Pin();
				}
			
				if (TargetWindow.IsValid())
				{
					DefaultStreamer->SetTargetWindow(TargetWindow);
				}
				else
				{
					UE_LOGF(LogPixelStreaming, Error, "Cannot set target window - target window is not valid.");
				}

			}
			else
			{
				UE_LOGF(LogPixelStreaming, Error, "Cannot set target window - GEngine is not valid.");
			}
		}

		if (!DefaultStreamer->GetSignallingServerURL().IsEmpty())
		{
			// The user has specified a URL on the command line meaning their intention is to start streaming immediately
			// in that case, set up the video input for them (as long as we're not in editor)
			if (Settings::CVarPixelStreamingUseMediaCapture.GetValueOnAnyThread())
			{
				DefaultStreamer->SetVideoInput(FPixelStreamingVideoInputMediaCapture::CreateActiveViewportCapture());
			}
			else
			{
				DefaultStreamer->SetVideoInput(FPixelStreamingVideoInputBackBuffer::Create());
			}
		}
	}

	bool FPixelStreamingModule::IsPlatformCompatible() const
	{
		bool bCompatible = true;

#if PLATFORM_WINDOWS
		bool bWin8OrHigher = IsWindows8OrGreater();
		if (!bWin8OrHigher)
		{
			FString ErrorString(TEXT("Failed to initialize Pixel Streaming plugin because minimum requirement is Windows 8"));
			FText ErrorText = FText::FromString(ErrorString);
			FText TitleText = FText::FromString(TEXT("Pixel Streaming Plugin"));
			FMessageDialog::Open(EAppMsgType::Ok, ErrorText, TitleText);
			UE_LOGF(LogPixelStreaming, Error, "%ls", *ErrorString);
			bCompatible = false;
		}
#endif

		if ((Settings::CVarPixelStreamingEncoderCodec.GetValueOnAnyThread() == "H264" && !IsEncoderSupported<FVideoEncoderConfigH264>())
			|| (Settings::CVarPixelStreamingEncoderCodec.GetValueOnAnyThread() == "AV1" && !IsEncoderSupported<FVideoEncoderConfigAV1>()))
		{
			UE_LOGF(LogPixelStreaming, Warning, "Could not setup hardware encoder. This is usually a driver issue or hardware limitation, try reinstalling your drivers.");
			UE_LOGF(LogPixelStreaming, Warning, "Falling back to VP8 software video encoding.");
			Settings::CVarPixelStreamingEncoderCodec.AsVariable()->Set(TEXT("VP8"), ECVF_SetByCommandline);
		}

		return bCompatible;
	}

	void FPixelStreamingModule::RegisterCustomHandlers(TSharedPtr<IPixelStreamingStreamer> Streamer)
	{
		if (TSharedPtr<IPixelStreamingInputHandler> InputHandler = Streamer->GetInputHandler().Pin())
		{
			// Set Encoder.MinQP CVar
			InputHandler->SetCommandHandler(TEXT("Encoder.MinQP"), [](FString PlayerId, FString Descriptor, FString MinQPString) {
				int MinQP = FCString::Atoi(*MinQPString);
				UE::PixelStreaming::Settings::CVarPixelStreamingEncoderMinQP->Set(MinQP, ECVF_SetByCommandline);
			});

			// Set Encoder.MaxQP CVar
			InputHandler->SetCommandHandler(TEXT("Encoder.MaxQP"), [](FString PlayerId, FString Descriptor, FString MaxQPString) {
				int MaxQP = FCString::Atoi(*MaxQPString);
				UE::PixelStreaming::Settings::CVarPixelStreamingEncoderMaxQP->Set(MaxQP, ECVF_SetByCommandline);
			});

			// Set WebRTC max FPS
			InputHandler->SetCommandHandler(TEXT("WebRTC.Fps"), [](FString PlayerId, FString Descriptor, FString FPSString) {
				int FPS = FCString::Atoi(*FPSString);
				UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCFps->Set(FPS, ECVF_SetByCommandline);
			});

			// Set MinBitrate
			InputHandler->SetCommandHandler(TEXT("WebRTC.MinBitrate"), [InputHandler](FString PlayerId, FString Descriptor, FString MinBitrateString) {
				if (InputHandler->IsElevated(PlayerId))
				{
					int MinBitrate = FCString::Atoi(*MinBitrateString);
					UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMinBitrate->Set(MinBitrate, ECVF_SetByCommandline);
				}
			});

			// Set MaxBitrate
			InputHandler->SetCommandHandler(TEXT("WebRTC.MaxBitrate"), [InputHandler](FString PlayerId, FString Descriptor, FString MaxBitrateString) {
				if (InputHandler->IsElevated(PlayerId))
				{
					int MaxBitrate = FCString::Atoi(*MaxBitrateString);
					UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMaxBitrate->Set(MaxBitrate, ECVF_SetByCommandline);
				}
			});

			FPixelStreamingInputProtocol::ToStreamerProtocol.Add("UIInteraction", FPixelStreamingInputMessage(50));
			InputHandler->RegisterMessageHandler("UIInteraction", [this](FString PlayerId, FMemoryReader Ar) { HandleUIInteraction(Ar); });

			// Handle sending commands to peers
			TWeakPtr<IPixelStreamingStreamer> WeakStreamer = Streamer;
			InputHandler->OnSendMessage.AddLambda([WeakStreamer](FString MessageName, FMemoryReader Ar) {
				if (TSharedPtr<IPixelStreamingStreamer> Streamer = WeakStreamer.Pin())
				{
					FString Descriptor;
					Ar << Descriptor;
					Streamer->SendPlayerMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find(MessageName)->GetID(), Descriptor);
				}
			});
		}
	}

	void FPixelStreamingModule::HandleUIInteraction(FMemoryReader Ar)
	{
		FString Res;
		Res.GetCharArray().SetNumUninitialized(Ar.TotalSize() / 2 + 1);
		Ar.Serialize(Res.GetCharArray().GetData(), Ar.TotalSize());

		FString Descriptor = Res.Mid(1);

		UE_LOGF(LogPixelStreaming, Verbose, "UIInteraction: %ls", *Descriptor);
		for (UPixelStreamingInput* InputComponent : InputComponents)
		{
			InputComponent->OnInputEvent.Broadcast(Descriptor);
		}
	}
	/**
	 * End own methods
	 */
} // namespace UE::PixelStreaming

IMPLEMENT_MODULE(UE::PixelStreaming::FPixelStreamingModule, PixelStreaming)
