// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingModule.h"
#include "RHI.h"
#include "Tickable.h"
#include "PixelStreamingInputProtocol.h"

class UPixelStreamingInput;
class SWindow;

namespace UE::PixelStreaming
{
	class FStreamer;
	class FVideoInputBackBuffer;
	class FVideoSourceGroup;

	/*
	 * This plugin allows the back buffer to be sent as a compressed video across a network.
	 */
	class FPixelStreamingModule : public IPixelStreamingModule
	{
	public:
		static IPixelStreamingModule* GetModule();

		/** IPixelStreamingModule implementation */
		virtual void SetCodec(EPixelStreamingCodec Codec) override;
		virtual EPixelStreamingCodec GetCodec() const override;
		virtual FReadyEvent& OnReady() override;
		virtual bool IsReady() override;
		virtual bool StartStreaming() override;
		virtual void StopStreaming() override;
		virtual TSharedPtr<IPixelStreamingStreamer> CreateStreamer(const FString& StreamerId) override;
		virtual TArray<FString> GetStreamerIds() override;
		virtual TSharedPtr<IPixelStreamingStreamer> FindStreamer(const FString& StreamerId) override;
		virtual TSharedPtr<IPixelStreamingStreamer> DeleteStreamer(const FString& StreamerId) override;
		void DeleteStreamer(TSharedPtr<IPixelStreamingStreamer> ToBeDeleted) override;
		virtual FString GetDefaultStreamerID() override;
		virtual FString GetDefaultSignallingURL() override;

		// These are staying on the module at the moment as theres no way of the BPs knowing which streamer they are relevant to
		virtual void AddInputComponent(UPixelStreamingInput* InInputComponent) override;
		virtual void RemoveInputComponent(UPixelStreamingInput* InInputComponent) override;
		virtual const TArray<UPixelStreamingInput*> GetInputComponents() override;
		// Don't delete, is used externally to PS
		virtual void SetExternalVideoSourceFPS(uint32 InFPS) override;
		virtual void SetExternalVideoSourceCoupleFramerate(bool bShouldCoupleFPS) override;
		virtual void SetExternalVideoSourceInput(TSharedPtr<FPixelStreamingVideoInput> InVideoInput) override;
		virtual rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> CreateExternalVideoSource() override;
		virtual void ReleaseExternalVideoSource(const webrtc::VideoTrackSourceInterface* InVideoSource) override;
		virtual TUniquePtr<webrtc::VideoEncoderFactory> CreateVideoEncoderFactory() override;
		virtual void ForEachStreamer(const TFunction<void(TSharedPtr<IPixelStreamingStreamer>)>& Func) override;
		/** End IPixelStreamingModule implementation */

	private:
		/** IModuleInterface implementation */
		void StartupModule() override;
		void ShutdownModule() override;
		/** End IModuleInterface implementation */

		// Own methods
		void InitDefaultStreamer();
		bool IsPlatformCompatible() const;
		void RegisterCustomHandlers(TSharedPtr<IPixelStreamingStreamer> Streamer);
		void HandleUIInteraction(FMemoryReader Ar);

	private:
		bool bModuleReady = false;
		bool bStartupCompleted = false;
		static IPixelStreamingModule* PixelStreamingModule;

		FReadyEvent ReadyEvent;
		TArray<UPixelStreamingInput*> InputComponents;
		TSharedPtr<FVideoSourceGroup> ExternalVideoSourceGroup;
		mutable FCriticalSection StreamersCS;
		TMap<FString, TWeakPtr<IPixelStreamingStreamer>> Streamers;
		TSharedPtr<IPixelStreamingStreamer> DefaultStreamer;
	};
} // namespace UE::PixelStreaming
