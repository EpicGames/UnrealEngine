// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingPeerConnection.h"
#include "VideoDecoderFactory.h"
#include "VideoEncoderFactoryLayered.h"
#include "PixelStreamingSessionDescriptionObservers.h"
#include "PixelStreamingPrivate.h"
#include "Settings.h"
#include "AudioCapturer.h"
#include "PixelStreamingAudioDeviceModule.h"
#include "ToStringExtensions.h"
#include "PixelStreamingDataChannel.h"
#include "Stats.h"
#include "AudioInputMixer.h"

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "absl/strings/match.h"
#include "PostWebRTCApi.h"
// End WebRTC Includes

namespace
{
	using namespace UE::PixelStreaming;

	FVideoEncoderFactoryLayered* GVideoEncoderFactory = nullptr;

	std::vector<webrtc::RtpEncodingParameters> CreateRTPEncodingParams(bool IsSFU)
	{
		std::vector<webrtc::RtpEncodingParameters> EncodingParams;

		if (IsSFU && Settings::SimulcastParameters.Layers.Num() > 0)
		{
			using FLayer = FPixelStreamingSimulcastParameters::FPixelStreamingSimulcastLayer;

			// encodings should be lowest res to highest
			TArray<FLayer*> SortedLayers;
			for (FLayer& Layer : Settings::SimulcastParameters.Layers)
			{
				SortedLayers.Add(&Layer);
			}

			SortedLayers.Sort([](const FLayer& LayerA, const FLayer& LayerB) { return LayerA.Scaling > LayerB.Scaling; });

			const int LayerCount = SortedLayers.Num();
			for (int i = 0; i < LayerCount; ++i)
			{
				const FLayer* SimulcastLayer = SortedLayers[i];
				webrtc::RtpEncodingParameters LayerEncoding{};
				LayerEncoding.rid = TCHAR_TO_UTF8(*(FString("simulcast") + FString::FromInt(LayerCount - i)));
				int minBps, maxBps;
				minBps = SimulcastLayer->MinBitrate;
				maxBps = FMath::Max(minBps, SimulcastLayer->MaxBitrate);
				LayerEncoding.min_bitrate_bps = minBps;
				LayerEncoding.max_bitrate_bps = maxBps;
				LayerEncoding.scale_resolution_down_by = SimulcastLayer->Scaling;

				// In M84 this will crash with "Attempted to set an unimplemented parameter of RtpParameters".
				// Try re-enabling this when we upgrade WebRTC versions.
				// LayerEncoding.network_priority = webrtc::Priority::kHigh;

				LayerEncoding.max_framerate = Settings::CVarPixelStreamingWebRTCFps.GetValueOnAnyThread();
				EncodingParams.push_back(LayerEncoding);
			}
		}
		else
		{
			webrtc::RtpEncodingParameters Encoding{};
			Encoding.rid = "base";
			// if the min/max bitrates are in the wrong order the stream will fail
			int MinBps, MaxBps;
			MinBps = Settings::CVarPixelStreamingWebRTCMinBitrate.GetValueOnAnyThread();
			MaxBps = Settings::CVarPixelStreamingWebRTCMaxBitrate.GetValueOnAnyThread();
			if (MinBps > MaxBps)
			{
				MaxBps = MinBps;
				// to try to not be misleading with debug texts etc, we reset these sanitised settings here
				Settings::CVarPixelStreamingWebRTCMinBitrate->Set(MinBps, ECVF_SetByCode);
				Settings::CVarPixelStreamingWebRTCMaxBitrate->Set(MaxBps, ECVF_SetByCode);
			}
			Encoding.max_bitrate_bps = MaxBps;
			Encoding.min_bitrate_bps = MinBps;
			Encoding.max_framerate = Settings::CVarPixelStreamingWebRTCFps.GetValueOnAnyThread();
			Encoding.scale_resolution_down_by.reset();
			Encoding.network_priority = webrtc::Priority::kHigh;
			EncodingParams.push_back(Encoding);
		}

		return EncodingParams;
	}

	std::string GetAudioStreamID()
	{
		const bool bSyncVideoAndAudio = !Settings::CVarPixelStreamingWebRTCDisableAudioSync.GetValueOnAnyThread();
		return bSyncVideoAndAudio ? "pixelstreaming_av_stream_id" : "pixelstreaming_audio_stream_id";
	}

	std::string GetVideoStreamID()
	{
		const bool bSyncVideoAndAudio = !Settings::CVarPixelStreamingWebRTCDisableAudioSync.GetValueOnAnyThread();
		return bSyncVideoAndAudio ? "pixelstreaming_av_stream_id" : "pixelstreaming_video_stream_id";
	}

	EPixelStreamingCodec ExtractVideoCodec(cricket::SessionDescription* SessionDescription)
	{
		std::vector<cricket::ContentInfo>& ContentInfos = SessionDescription->contents();
		for (cricket::ContentInfo& ContentInfo : ContentInfos)
		{
			cricket::MediaContentDescription* MediaDescription = ContentInfo.media_description();
			cricket::MediaType MediaType = MediaDescription->type();

			if (MediaDescription->type() == cricket::MediaType::MEDIA_TYPE_VIDEO)
			{
				cricket::VideoContentDescription* VideoDescription = MediaDescription->as_video();
				std::vector<cricket::VideoCodec> CodecsCopy = VideoDescription->codecs();
				for (cricket::VideoCodec& Codec : CodecsCopy)
				{
					if (absl::EqualsIgnoreCase(Codec.name, cricket::kVp8CodecName))
					{
						return EPixelStreamingCodec::VP8;
					}
					else if (absl::EqualsIgnoreCase(Codec.name, cricket::kVp9CodecName))
					{
						return EPixelStreamingCodec::VP9;
					}
					else if (absl::EqualsIgnoreCase(Codec.name, cricket::kAv1CodecName))
					{
						return EPixelStreamingCodec::AV1;
					}
					else if (absl::EqualsIgnoreCase(Codec.name, cricket::kH264CodecName))
					{
						return EPixelStreamingCodec::H264;
					}
				}
			}
		}
		return EPixelStreamingCodec::Invalid;
	}

	void MungeLocalSDP(cricket::SessionDescription* SessionDescription)
	{
		std::vector<cricket::ContentInfo>& ContentInfos = SessionDescription->contents();
		for (cricket::ContentInfo& ContentInfo : ContentInfos)
		{
			cricket::MediaContentDescription* MediaDescription = ContentInfo.media_description();
			cricket::MediaType MediaType = MediaDescription->type();

			if (MediaDescription->type() == cricket::MediaType::MEDIA_TYPE_VIDEO)
			{
				// todo: munge local video sdp if required
			}
			else if (MediaDescription->type() == cricket::MediaType::MEDIA_TYPE_AUDIO)
			{
				cricket::AudioContentDescription* AudioDescription = MediaDescription->as_audio();
				if (AudioDescription == nullptr)
				{
					continue;
				}
				std::vector<cricket::AudioCodec> CodecsCopy = AudioDescription->codecs();
				for (cricket::AudioCodec& Codec : CodecsCopy)
				{
					if (Codec.name == "opus")
					{
						Codec.SetParam(cricket::kCodecParamPTime, "20");
						Codec.SetParam(cricket::kCodecParamMaxPTime, "120");
						Codec.SetParam(cricket::kCodecParamMinPTime, "3");
						Codec.SetParam(cricket::kCodecParamSPropStereo, "1");
						Codec.SetParam(cricket::kCodecParamStereo, "1");
						Codec.SetParam(cricket::kCodecParamUseInbandFec, "1");
						Codec.SetParam(cricket::kCodecParamUseDtx, "0");
						Codec.SetParam(cricket::kCodecParamMaxAverageBitrate, "510000");
						Codec.SetParam(cricket::kCodecParamMaxPlaybackRate, "48000");
					}
				}
				AudioDescription->set_codecs(CodecsCopy);
			}
		}
	}

	void MungeRemoteSDP(cricket::SessionDescription* RemoteDescription)
	{
		// Munge SDP of remote description to inject min, max, start bitrates
		std::vector<cricket::ContentInfo>& ContentInfos = RemoteDescription->contents();
		for (cricket::ContentInfo& Content : ContentInfos)
		{
			cricket::MediaContentDescription* MediaDescription = Content.media_description();
			if (MediaDescription->type() == cricket::MediaType::MEDIA_TYPE_VIDEO)
			{
				cricket::VideoContentDescription* VideoDescription = MediaDescription->as_video();

				std::vector<cricket::VideoCodec> CodecsCopy = VideoDescription->codecs();
				for (cricket::VideoCodec& Codec : CodecsCopy)
				{
					// Note: These params are passed as kilobits, so divide by 1000.
					Codec.SetParam(cricket::kCodecParamMinBitrate, Settings::CVarPixelStreamingWebRTCMinBitrate.GetValueOnAnyThread() / 1000);
					Codec.SetParam(cricket::kCodecParamStartBitrate, Settings::CVarPixelStreamingWebRTCStartBitrate.GetValueOnAnyThread() / 1000);
					Codec.SetParam(cricket::kCodecParamMaxBitrate, Settings::CVarPixelStreamingWebRTCMaxBitrate.GetValueOnAnyThread() / 1000);
				}
				VideoDescription->set_codecs(CodecsCopy);
			}
		}
	}

	void SetTransceiverDirection(webrtc::RtpTransceiverInterface& InTransceiver, webrtc::RtpTransceiverDirection InDirection)
	{
		webrtc::RTCError Result = InTransceiver.SetDirectionWithError(InDirection);
		checkf(Result.ok(), TEXT("Failed to add Video transceiver to PeerConnection. Msg=%s"), *FString(Result.message()));
	}
} // namespace

// self registering/deregistering object for polling stats
class FPeerWebRTCStatsSource : public IPixelStreamingStatsSource, public TSharedFromThis<FPeerWebRTCStatsSource, ESPMode::ThreadSafe>
{
public:
	FPeerWebRTCStatsSource(rtc::scoped_refptr<webrtc::PeerConnectionInterface> InPC, rtc::scoped_refptr<webrtc::RTCStatsCollectorCallback> InCallback)
		: PeerConnection(InPC)
		, WebRTCStatsCallback(InCallback)
	{
	}

	void BindToStatsPollEvent()
	{
		if (UE::PixelStreaming::FStats* PSStats = UE::PixelStreaming::FStats::Get())
		{
			Handle = PSStats->OnStatsPolled.AddSP(this, &FPeerWebRTCStatsSource::PollStats);
		}
	}

	virtual ~FPeerWebRTCStatsSource()
	{
		if (UE::PixelStreaming::FStats* PSStats = UE::PixelStreaming::FStats::Get())
		{
			PSStats->OnStatsPolled.Remove(Handle);
		}
	}

	virtual void PollStats() const
	{
		if (PeerConnection && WebRTCStatsCallback)
		{
			// Use the top-level PeerConnection::GetStats call as this gets us stats for video, audio, transports, datachannels etc
			// https://w3c.github.io/webrtc-pc/#mandatory-to-implement-stats
			// Note: For the avoidance of doubt, this GetStats() call is posted onto the WebRTC signaling thread internally.
			PeerConnection->GetStats(WebRTCStatsCallback.get());
		}
	}

private:
	rtc::scoped_refptr<webrtc::PeerConnectionInterface> PeerConnection;
	rtc::scoped_refptr<webrtc::RTCStatsCollectorCallback> WebRTCStatsCallback;
	FDelegateHandle Handle;
};

TUniquePtr<rtc::Thread> FPixelStreamingPeerConnection::SignallingThread = nullptr;
rtc::scoped_refptr<webrtc::PeerConnectionFactoryInterface> FPixelStreamingPeerConnection::PeerConnectionFactory = nullptr;
rtc::scoped_refptr<webrtc::AudioSourceInterface> FPixelStreamingPeerConnection::ApplicationAudioSource = nullptr;
TSharedPtr<UE::PixelStreaming::FAudioInputMixer> FPixelStreamingPeerConnection::AudioMixer = MakeShared<UE::PixelStreaming::FAudioInputMixer>();

// Defined here because otherwise the compiler doesnt know how to delete StatsSource since the destructor is declared in
// this cpp. The important part is the destructor here but I put both constructor and destructor for consistency.
FPixelStreamingPeerConnection::FPixelStreamingPeerConnection() = default;

FPixelStreamingPeerConnection::~FPixelStreamingPeerConnection()
{
	// callbacks might still be fired as we're destroying this object, we want to make sure we dont try and access this still.
	bIsDestroying = true;

	// need to remove the audio/video sinks before deleting it.
	if (PeerConnection)
	{
		for (auto& Transceiver : PeerConnection->GetTransceivers())
		{
			if (AudioSink && Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_AUDIO)
			{
				if (rtc::scoped_refptr<webrtc::RtpReceiverInterface> Receiver = Transceiver->receiver())
				{
					if (rtc::scoped_refptr<webrtc::MediaStreamTrackInterface> Track = Receiver->track())
					{
						webrtc::AudioTrackInterface* AudioTrack = static_cast<webrtc::AudioTrackInterface*>(Track.get());
						AudioTrack->RemoveSink(AudioSink.Get());
					}
				}
			}
			else if (VideoSink && Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_VIDEO)
			{
				if (rtc::scoped_refptr<webrtc::RtpReceiverInterface> Receiver = Transceiver->receiver())
				{
					if (rtc::scoped_refptr<webrtc::MediaStreamTrackInterface> Track = Receiver->track())
					{
						webrtc::VideoTrackInterface* VideoTrack = static_cast<webrtc::VideoTrackInterface*>(Track.get());
						VideoTrack->RemoveSink(VideoSink);
					}
				}
			}
		}
	}
}

TUniquePtr<FPixelStreamingPeerConnection> FPixelStreamingPeerConnection::Create(const FRTCConfig& RTCConfig, bool IsSFU)
{
	if (!PeerConnectionFactory)
	{
		CreatePeerConnectionFactory();
	}

	TUniquePtr<FPixelStreamingPeerConnection> NewPeerConnection = TUniquePtr<FPixelStreamingPeerConnection>(new FPixelStreamingPeerConnection());
	NewPeerConnection->IsSFU = IsSFU;

	webrtc::RTCErrorOr<rtc::scoped_refptr<webrtc::PeerConnectionInterface>> Result = PeerConnectionFactory->CreatePeerConnectionOrError(RTCConfig, webrtc::PeerConnectionDependencies(NewPeerConnection.Get()));
	checkf(Result.ok(), TEXT("Failed to create Peer Connection. Msg=%s"), *FString(Result.error().message()));
	NewPeerConnection->PeerConnection = Result.MoveValue();

	// Setup suggested bitrate settings on the Peer Connection based on our CVars
	webrtc::BitrateSettings BitrateSettings;
	BitrateSettings.min_bitrate_bps = Settings::CVarPixelStreamingWebRTCMinBitrate.GetValueOnAnyThread();
	BitrateSettings.max_bitrate_bps = Settings::CVarPixelStreamingWebRTCMaxBitrate.GetValueOnAnyThread();
	BitrateSettings.start_bitrate_bps = Settings::CVarPixelStreamingWebRTCStartBitrate.GetValueOnAnyThread();
	NewPeerConnection->PeerConnection->SetBitrate(BitrateSettings);

	return NewPeerConnection;
}

void FPixelStreamingPeerConnection::Shutdown()
{
	PeerConnectionFactory = nullptr;
	if (SignallingThread)
	{
		SignallingThread->Stop();
	}
	SignallingThread = nullptr;

	if (AudioMixer)
	{
		AudioMixer->StopMixing();
	}
	AudioMixer = nullptr;
}

void FPixelStreamingPeerConnection::CreateOffer(EReceiveMediaOption ReceiveOption, const SDPCallback& SuccessCallback, const ErrorCallback& ErrorCallback)
{
	CreateSDP(ESDPType::Offer, ReceiveOption, SuccessCallback, ErrorCallback);
}

void FPixelStreamingPeerConnection::CreateAnswer(EReceiveMediaOption ReceiveOption, const SDPCallback& SuccessCallback, const ErrorCallback& ErrorCallback)
{
	CreateSDP(ESDPType::Answer, ReceiveOption, SuccessCallback, ErrorCallback);
}

void FPixelStreamingPeerConnection::ReceiveOffer(const FString& Sdp, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback)
{
	RefreshStreamBitrate();
	
	webrtc::SdpParseError Error;
	std::unique_ptr<webrtc::SessionDescriptionInterface> SessionDesc = webrtc::CreateSessionDescription(webrtc::SdpType::kOffer, UE::PixelStreaming::ToString(Sdp), &Error);
	if (SessionDesc)
	{
		SetRemoteDescription(SessionDesc.release(), SuccessCallback, ErrorCallback);
	}
	else
	{
		UE_LOGF(LogPixelStreaming, Error, "Failed to parse offer: %s", Error.description.c_str());
		if (ErrorCallback)
		{
			ErrorCallback(ToString(Error.description));
		}
	}
}

void FPixelStreamingPeerConnection::ReceiveAnswer(const FString& Sdp, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback)
{
	RefreshStreamBitrate();

	webrtc::SdpParseError Error;
	std::unique_ptr<webrtc::SessionDescriptionInterface> SessionDesc = webrtc::CreateSessionDescription(webrtc::SdpType::kAnswer, ToString(Sdp), &Error);
	if (SessionDesc)
	{
		SetRemoteDescription(SessionDesc.release(), SuccessCallback, ErrorCallback);
	}
	else
	{
		UE_LOGF(LogPixelStreaming, Error, "Failed to parse answer: %s", Error.description.c_str());
		if (ErrorCallback)
		{
			ErrorCallback(ToString(Error.description));
		}
	}
}

void FPixelStreamingPeerConnection::SetLocalDescription(webrtc::SessionDescriptionInterface* SDP, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback)
{
	auto OnSetLocalDescriptionSuccess = [SuccessCallback]() {
		if (SuccessCallback)
		{
			SuccessCallback();
		}
	};
	auto OnSetLocalDescriptionFail = [ErrorCallback](const FString& Error) {
		UE_LOGF(LogPixelStreaming, Error, "Failed to set local description: %ls", *Error);
		if (ErrorCallback)
		{
			ErrorCallback(Error);
		}
	};

	FPixelStreamingSetSessionDescriptionObserver* SetLocalDescriptionObserver = FPixelStreamingSetSessionDescriptionObserver::Create(OnSetLocalDescriptionSuccess, OnSetLocalDescriptionFail);
	MungeLocalSDP(SDP->description());

	// Get the negotiated codec from the SDP
	NegotiatedVideoCodec = ExtractVideoCodec(SDP->description());

	PeerConnection->SetLocalDescription(SetLocalDescriptionObserver, SDP);
}

void FPixelStreamingPeerConnection::SetRemoteDescription(webrtc::SessionDescriptionInterface* SDP, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback)
{
	auto OnSetRemoteDescriptionSuccess = [SuccessCallback]() {
		if (SuccessCallback)
		{
			SuccessCallback();
		}
	};

	auto OnSetRemoteDescriptionFail = [ErrorCallback](const FString& Error) {
		UE_LOGF(LogPixelStreaming, Error, "Failed to set remote description: %ls", *Error);
		if (ErrorCallback)
		{
			ErrorCallback(Error);
		}
	};

	FPixelStreamingSetSessionDescriptionObserver* SetRemoteDescriptionObserver = FPixelStreamingSetSessionDescriptionObserver::Create(OnSetRemoteDescriptionSuccess, OnSetRemoteDescriptionFail);
	MungeRemoteSDP(SDP->description());

	// Get the negotiated codec from the SDP
	NegotiatedVideoCodec = ExtractVideoCodec(SDP->description());

	PeerConnection->SetRemoteDescription(SetRemoteDescriptionObserver, SDP);
}

const webrtc::SessionDescriptionInterface* FPixelStreamingPeerConnection::GetLocalDescription() const
{
	return PeerConnection->local_description();
}

const webrtc::SessionDescriptionInterface* FPixelStreamingPeerConnection::GetRemoteDescription() const
{
	return PeerConnection->remote_description();
}

EPixelStreamingCodec FPixelStreamingPeerConnection::GetNegotiatedVideoCodec() const
{
	return NegotiatedVideoCodec;
}

void FPixelStreamingPeerConnection::SetVideoSource(rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> InVideoSource)
{
	const static FString VideoTrackLabel = TEXT("pixelstreaming_video_track_label");

	VideoSource = InVideoSource;

	const bool bTransmitUEVideo = !Settings::CVarPixelStreamingWebRTCDisableTransmitVideo.GetValueOnAnyThread();

	webrtc::RtpTransceiverDirection VideoTransceiverDirection;
	if (bTransmitUEVideo)
	{
		VideoTransceiverDirection = webrtc::RtpTransceiverDirection::kSendOnly;
	}
	else
	{
		VideoTransceiverDirection = webrtc::RtpTransceiverDirection::kInactive;
	}

	// Create video track
	rtc::scoped_refptr<webrtc::VideoTrackInterface> VideoTrack = PeerConnectionFactory->CreateVideoTrack(ToString(VideoTrackLabel), VideoSource.get());
	VideoTrack->set_enabled(true);

	// Set some content hints based on degradation prefs, WebRTC uses these internally.
	webrtc::DegradationPreference DegradationPref = Settings::GetDegradationPreference();
	switch (DegradationPref)
	{
		case webrtc::DegradationPreference::MAINTAIN_FRAMERATE:
			VideoTrack->set_content_hint(webrtc::VideoTrackInterface::ContentHint::kFluid);
			break;
		case webrtc::DegradationPreference::MAINTAIN_RESOLUTION:
			VideoTrack->set_content_hint(webrtc::VideoTrackInterface::ContentHint::kDetailed);
			break;
		default:
			break;
	}

	bool bHasVideoTransceiver = false;
	for (auto& Transceiver : PeerConnection->GetTransceivers())
	{
		rtc::scoped_refptr<webrtc::RtpSenderInterface> Sender = Transceiver->sender();
		if (Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_VIDEO)
		{
			bHasVideoTransceiver = true;
#if WEBRTC_5414
			Sender->SetTrack(VideoTrack.get());
#else
			Sender->SetTrack(VideoTrack);
#endif
			Sender->SetStreams({ GetVideoStreamID() });
			SetTransceiverDirection(*Transceiver, VideoTransceiverDirection);
			webrtc::RtpParameters ExistingParams = Sender->GetParameters();
			ExistingParams.degradation_preference = Settings::GetDegradationPreference();
		}
	}

	// If there is no existing video transceiver, add one.
	if (!bHasVideoTransceiver)
	{
		webrtc::RtpTransceiverInit TransceiverOptions;
		TransceiverOptions.stream_ids = { GetVideoStreamID() };
		TransceiverOptions.direction = VideoTransceiverDirection;
		TransceiverOptions.send_encodings = CreateRTPEncodingParams(IsSFU);

		webrtc::RTCErrorOr<rtc::scoped_refptr<webrtc::RtpTransceiverInterface>> Result = PeerConnection->AddTransceiver(VideoTrack, TransceiverOptions);
		checkf(Result.ok(), TEXT("Failed to add Video transceiver to PeerConnection. Msg=%s"), *FString(Result.error().message()));
		SetTransceiverDirection(*Result.value(), VideoTransceiverDirection);
		webrtc::RtpParameters ExistingParams = Result.value()->sender()->GetParameters();
		ExistingParams.degradation_preference = Settings::GetDegradationPreference();
	}
}

void FPixelStreamingPeerConnection::RefreshStreamBitrate()
{
	webrtc::BitrateSettings bitrateSettings;
	bitrateSettings.min_bitrate_bps = Settings::CVarPixelStreamingWebRTCMinBitrate.GetValueOnAnyThread();
	bitrateSettings.max_bitrate_bps = Settings::CVarPixelStreamingWebRTCMaxBitrate.GetValueOnAnyThread();
	PeerConnection->SetBitrate(bitrateSettings);
}

void FPixelStreamingPeerConnection::SetAudioSource(rtc::scoped_refptr<webrtc::AudioSourceInterface> InAudioSource)
{
	const static FString AudioTrackLabel = TEXT("pixelstreaming_audio_track_label");

	AudioSource = InAudioSource;

	const bool bTransmitUEAudio = !Settings::CVarPixelStreamingWebRTCDisableTransmitAudio.GetValueOnAnyThread();
	const bool bReceiveBrowserAudio = !IsSFU && !Settings::CVarPixelStreamingWebRTCDisableReceiveAudio.GetValueOnAnyThread();

	webrtc::RtpTransceiverDirection AudioTransceiverDirection;
	if (bTransmitUEAudio && bReceiveBrowserAudio)
	{
		AudioTransceiverDirection = webrtc::RtpTransceiverDirection::kSendRecv;
	}
	else if (bTransmitUEAudio)
	{
		AudioTransceiverDirection = webrtc::RtpTransceiverDirection::kSendOnly;
	}
	else if (bReceiveBrowserAudio)
	{
		AudioTransceiverDirection = webrtc::RtpTransceiverDirection::kRecvOnly;
	}
	else
	{
		AudioTransceiverDirection = webrtc::RtpTransceiverDirection::kInactive;
	}

#if WEBRTC_5414
	rtc::scoped_refptr<webrtc::AudioTrackInterface> AudioTrack = PeerConnectionFactory->CreateAudioTrack(ToString(AudioTrackLabel), InAudioSource.get());
#else
	rtc::scoped_refptr<webrtc::AudioTrackInterface> AudioTrack = PeerConnectionFactory->CreateAudioTrack(ToString(AudioTrackLabel), InAudioSource);
#endif
	bool bHasAudioTransceiver = false;
	for (auto& Transceiver : PeerConnection->GetTransceivers())
	{
		rtc::scoped_refptr<webrtc::RtpSenderInterface> Sender = Transceiver->sender();
		if (Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_AUDIO)
		{
			bHasAudioTransceiver = true;
#if WEBRTC_5414
			Sender->SetTrack(AudioTrack.get());
#else
			Sender->SetTrack(AudioTrack);
#endif
			Sender->SetStreams({ GetAudioStreamID() });
			SetTransceiverDirection(*Transceiver, AudioTransceiverDirection);
		}
	}

	if (!bHasAudioTransceiver)
	{
		// Add the track
		webrtc::RTCErrorOr<rtc::scoped_refptr<webrtc::RtpSenderInterface>> Result = PeerConnection->AddTrack(AudioTrack, { GetAudioStreamID() });
		checkf(Result.ok(), TEXT("Failed to add audio track to PeerConnection. Msg=%s"), *FString(Result.error().message()));
		// SetTransceiverDirection(*Result.value(), AudioTransceiverDirection); // TODO why cant we do something simpler like this?
		for (auto& Transceiver : PeerConnection->GetTransceivers())
		{
			rtc::scoped_refptr<webrtc::RtpSenderInterface> Sender = Transceiver->sender();
			if (Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_AUDIO)
			{
				SetTransceiverDirection(*Transceiver, AudioTransceiverDirection);
			}
		}
	}
}

void FPixelStreamingPeerConnection::SetVideoSink(rtc::VideoSinkInterface<webrtc::VideoFrame>* InSink)
{
	VideoSink = InSink;

	// add it to existing tracks if we call this late
	if (PeerConnection)
	{
		for (auto& Transceiver : PeerConnection->GetTransceivers())
		{
			if (Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_VIDEO)
			{
				if (rtc::scoped_refptr<webrtc::RtpReceiverInterface> Receiver = Transceiver->receiver())
				{
					if (rtc::scoped_refptr<webrtc::MediaStreamTrackInterface> Track = Receiver->track())
					{
						webrtc::VideoTrackInterface* VideoTrack = static_cast<webrtc::VideoTrackInterface*>(Track.get());
						VideoTrack->AddOrUpdateSink(VideoSink, rtc::VideoSinkWants());
					}
				}
			}
		}
	}
}

void FPixelStreamingPeerConnection::SetAudioSink(TSharedPtr<IPixelStreamingAudioSink> InSink)
{
	AudioSink = InSink;

	// add it to existing tracks if we call this late
	if (PeerConnection)
	{
		for (auto& Transceiver : PeerConnection->GetTransceivers())
		{
			if (Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_AUDIO)
			{
				if (rtc::scoped_refptr<webrtc::RtpReceiverInterface> Receiver = Transceiver->receiver())
				{
					if (rtc::scoped_refptr<webrtc::MediaStreamTrackInterface> Track = Receiver->track())
					{
						webrtc::AudioTrackInterface* AudioTrack = static_cast<webrtc::AudioTrackInterface*>(Track.get());
						// There's no mechanism to check if this sink already exists on the track, so attempt to remove it first to make sure we
						// don't accidentally add it twice (note it is always safe to call remove because it is a no-op if the sink is not found).
						AudioTrack->RemoveSink(AudioSink.Get());
						AudioTrack->AddSink(AudioSink.Get());
					}
				}
			}
		}
	}
}

void FPixelStreamingPeerConnection::ForEachTransceiver(const TFunction<void(rtc::scoped_refptr<webrtc::RtpTransceiverInterface>)>& Func)
{
	if(!PeerConnection)
	{
		return;
	}

	for (auto& Transceiver : PeerConnection->GetTransceivers())
	{
		Func(Transceiver);
	}
}

void FPixelStreamingPeerConnection::AddRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp)
{
	webrtc::SdpParseError Error;
	std::unique_ptr<webrtc::IceCandidateInterface> Candidate(webrtc::CreateIceCandidate(UE::PixelStreaming::ToString(SdpMid), SdpMLineIndex, UE::PixelStreaming::ToString(Sdp), &Error));
	if (!Candidate)
	{
		UE_LOGF(LogPixelStreaming, Error, "Failed to create ICE candicate: %s", Error.description.c_str());
		return;
	}

	PeerConnection->AddIceCandidate(std::move(Candidate), [](webrtc::RTCError error) {
		if (!error.ok() && !Settings::CVarPixelStreamingSuppressICECandidateErrors.GetValueOnAnyThread())
		{
			UE_LOGF(LogPixelStreaming, Error, "AddIceCandidate failed: %s", error.message());
		}
	});
}

TSharedPtr<FPixelStreamingDataChannel> FPixelStreamingPeerConnection::CreateDataChannel(int Id, bool Negotiated)
{
	webrtc::DataChannelInit DataChannelConfig;
	DataChannelConfig.reliable = true;
	DataChannelConfig.ordered = true;
	DataChannelConfig.negotiated = Negotiated;
	DataChannelConfig.id = Id;

	webrtc::RTCErrorOr<rtc::scoped_refptr<webrtc::DataChannelInterface>> Result = PeerConnection->CreateDataChannelOrError("datachannel", &DataChannelConfig);
	checkf(Result.ok(), TEXT("Failed to create Data Channel. Msg=%s"), *FString(Result.error().message()));
	return FPixelStreamingDataChannel::Create(Result.MoveValue());
}

void FPixelStreamingPeerConnection::SetWebRTCStatsCallback(rtc::scoped_refptr<webrtc::RTCStatsCollectorCallback> InCallback)
{
	TSharedPtr<FPeerWebRTCStatsSource> NewStatsSource = MakeShared<FPeerWebRTCStatsSource>(PeerConnection, InCallback);
	NewStatsSource->BindToStatsPollEvent();
	StatsSource = NewStatsSource;
}

rtc::scoped_refptr<webrtc::AudioSourceInterface> FPixelStreamingPeerConnection::GetApplicationAudioSource()
{
	if (!PeerConnectionFactory)
	{
		CreatePeerConnectionFactory();
	}

	// Create one and only one audio source for Pixel Streaming.
	if (!ApplicationAudioSource)
	{
		// Setup audio source options, we turn off many of the "nice" audio settings that
		// would traditionally be used in a conference call because the audio source we are
		// transmitting is UE application audio (not some unknown microphone).
		cricket::AudioOptions AudioSourceOptions;
		AudioSourceOptions.echo_cancellation = false;
		AudioSourceOptions.auto_gain_control = false;
		AudioSourceOptions.noise_suppression = false;
		AudioSourceOptions.highpass_filter = false;
		AudioSourceOptions.stereo_swapping = false;
		AudioSourceOptions.audio_jitter_buffer_max_packets = 1000;
		AudioSourceOptions.audio_jitter_buffer_fast_accelerate = false;
		AudioSourceOptions.audio_jitter_buffer_min_delay_ms = 0;
#if !WEBRTC_5414
		AudioSourceOptions.audio_jitter_buffer_enable_rtx_handling = false;
		AudioSourceOptions.typing_detection = false;
		AudioSourceOptions.experimental_agc = false;
		AudioSourceOptions.experimental_ns = false;
		AudioSourceOptions.residual_echo_detector = false;
#endif
		// Create audio source
		ApplicationAudioSource = PeerConnectionFactory->CreateAudioSource(AudioSourceOptions);
	}

	return ApplicationAudioSource;
}

void FPixelStreamingPeerConnection::ForceVideoKeyframe()
{
	if (GVideoEncoderFactory)
	{
		GVideoEncoderFactory->ForceKeyFrame();
	}
}

void FPixelStreamingPeerConnection::OnSignalingChange(webrtc::PeerConnectionInterface::SignalingState NewState)
{
	UE_LOGF(LogPixelStreaming, Log, "OnSignalingChange (%ls)", UE::PixelStreaming::ToString(NewState));
}

void FPixelStreamingPeerConnection::OnAddStream(rtc::scoped_refptr<webrtc::MediaStreamInterface> Stream)
{
	UE_LOGF(LogPixelStreaming, Log, "OnAddStream");
}

void FPixelStreamingPeerConnection::OnRemoveStream(rtc::scoped_refptr<webrtc::MediaStreamInterface> Stream)
{
	UE_LOGF(LogPixelStreaming, Log, "OnRemoveStream");
}

void FPixelStreamingPeerConnection::OnDataChannel(rtc::scoped_refptr<webrtc::DataChannelInterface> Channel)
{
	UE_LOGF(LogPixelStreaming, Log, "OnDataChannel");

	TSharedPtr<FPixelStreamingDataChannel> NewChannel = FPixelStreamingDataChannel::Create(Channel);
	OnNewDataChannel.Broadcast(NewChannel);
}

void FPixelStreamingPeerConnection::OnRenegotiationNeeded()
{
	UE_LOGF(LogPixelStreaming, Log, "OnRenegotiationNeeded");
}

void FPixelStreamingPeerConnection::OnIceConnectionChange(webrtc::PeerConnectionInterface::IceConnectionState NewState)
{
	UE_LOGF(LogPixelStreaming, Log, "OnIceConnectionChange (%ls)", UE::PixelStreaming::ToString(NewState));

	if (NewState == webrtc::PeerConnectionInterface::IceConnectionState::kIceConnectionConnected)
	{
		// Might want to fire an event here
		// once ICE is connected frames should start going through so make sure we send a full frame.
		ForceVideoKeyframe();
	}
	else if (NewState == webrtc::PeerConnectionInterface::IceConnectionState::kIceConnectionDisconnected)
	{
		// Might want to fire an event here
	}

	OnIceStateChanged.Broadcast(NewState);
}

void FPixelStreamingPeerConnection::OnIceGatheringChange(webrtc::PeerConnectionInterface::IceGatheringState NewState)
{
	UE_LOGF(LogPixelStreaming, Log, "OnIceGatheringChange (%ls)", UE::PixelStreaming::ToString(NewState));
}

void FPixelStreamingPeerConnection::OnIceCandidate(const webrtc::IceCandidateInterface* Candidate)
{
	// UE_LOGF(LogPixelStreaming, Log, "OnIceCandidate");
	OnEmitIceCandidate.Broadcast(Candidate);
}

void FPixelStreamingPeerConnection::OnIceCandidatesRemoved(const std::vector<cricket::Candidate>& candidates)
{
	UE_LOGF(LogPixelStreaming, Log, "OnIceCandidatesRemoved");
}

void FPixelStreamingPeerConnection::OnIceConnectionReceivingChange(bool Receiving)
{
	UE_LOGF(LogPixelStreaming, Log, "OnIceConnectionReceivingChange");
}

void FPixelStreamingPeerConnection::OnTrack(rtc::scoped_refptr<webrtc::RtpTransceiverInterface> Transceiver)
{
	UE_LOGF(LogPixelStreaming, Log, "OnTrack");
	if (VideoSink && Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_VIDEO)
	{
		webrtc::VideoTrackInterface* VideoTrack = static_cast<webrtc::VideoTrackInterface*>(Transceiver->receiver()->track().get());
		VideoTrack->AddOrUpdateSink(VideoSink, rtc::VideoSinkWants());
	}
	else if (AudioSink && Transceiver->media_type() == cricket::MediaType::MEDIA_TYPE_AUDIO)
	{
		webrtc::AudioTrackInterface* AudioTrack = static_cast<webrtc::AudioTrackInterface*>(Transceiver->receiver()->track().get());
		// There's no mechanism to check if this sink already exists on the track, so attempt to remove it first to make sure we
		// don't accidentally add it twice (note it is always safe to call remove because it is a no-op if the sink is not found).
		// To future maintainers: yes, this code exsists in two spots, this is intentional.
		AudioTrack->RemoveSink(AudioSink.Get());
		AudioTrack->AddSink(AudioSink.Get());
	}

	return;
}

void FPixelStreamingPeerConnection::OnRemoveTrack(rtc::scoped_refptr<webrtc::RtpReceiverInterface> receiver)
{
	UE_LOGF(LogPixelStreaming, Log, "OnRemoveTrack");
}

void FPixelStreamingPeerConnection::CreateSDP(ESDPType SDPType, EReceiveMediaOption ReceiveOption, const SDPCallback& SuccessCallback, const ErrorCallback& ErrorCallback)
{
	auto OnCreateSDPSuccess = [this, SuccessCallback, ErrorCallback](webrtc::SessionDescriptionInterface* SDP) {
		auto OnSetLocalDescriptionSuccess = [this, SuccessCallback]() {
			if (SuccessCallback)
			{
				SuccessCallback(PeerConnection->local_description());
			}
		};
		auto OnSetLocalDescriptionFail = [ErrorCallback](const FString& Error) {
			if (ErrorCallback)
			{
				ErrorCallback(Error);
			}
		};

		// prevents this callback from continuing if we're currently in the destructor
		if (!bIsDestroying)
		{
			SetLocalDescription(SDP, OnSetLocalDescriptionSuccess, OnSetLocalDescriptionFail);
		}
	};

	auto OnCreateSDPFail = [ErrorCallback](const FString& Error) {
		UE_LOGF(LogPixelStreaming, Error, "Failed to create SDP: %ls", *Error);
		if (ErrorCallback)
		{
			ErrorCallback(Error);
		}
	};

	int offer_to_receive_video = ((ReceiveOption & EReceiveMediaOption::Video) != 0) ? 1 : 0;
	int offer_to_receive_audio = ((ReceiveOption & EReceiveMediaOption::Audio) != 0) ? 1 : 0;
	bool voice_activity_detection = false;
	bool ice_restart = true;
	bool use_rtp_mux = true;
	
	webrtc::PeerConnectionInterface::RTCOfferAnswerOptions SDPOption{
		offer_to_receive_video,
		offer_to_receive_audio,
		voice_activity_detection,
		ice_restart,
		use_rtp_mux
	};

	FPixelStreamingCreateSessionDescriptionObserver* CreateSDPObserver = FPixelStreamingCreateSessionDescriptionObserver::Create(OnCreateSDPSuccess, OnCreateSDPFail);

	if (SDPType == ESDPType::Offer)
	{
		PeerConnection->CreateOffer(CreateSDPObserver, SDPOption);
	}
	else if (SDPType == ESDPType::Answer)
	{
		PeerConnection->CreateAnswer(CreateSDPObserver, SDPOption);
	}
}

TSharedPtr<IPixelStreamingAudioInput> FPixelStreamingPeerConnection::CreateAudioInput()
{
	return AudioMixer->CreateInput();
}

void FPixelStreamingPeerConnection::RemoveAudioInput(TSharedPtr<IPixelStreamingAudioInput> AudioInput)
{
	AudioMixer->DisconnectInput(StaticCastSharedPtr<FAudioInput>(AudioInput));
}

void InitializeFieldTrials()
{
	FString FieldTrials = Settings::CVarPixelStreamingWebRTCFieldTrials.GetValueOnAnyThread();

	// Set the WebRTC-FrameDropper/Disabled/ if the CVar is set
	if (Settings::CVarPixelStreamingWebRTCDisableFrameDropper.GetValueOnAnyThread())
	{
		FieldTrials += TEXT("WebRTC-FrameDropper/Disabled/");
	}

	if (Settings::CVarPixelStreamingWebRTCEnableFlexFec.GetValueOnAnyThread())
	{
		FieldTrials += TEXT("WebRTC-FlexFEC-03-Advertised/Enabled/WebRTC-FlexFEC-03/Enabled/");
	}

	// Parse "WebRTC-Video-Pacing/" field trial
	{
		float OutPacingFactor = -1.0f;
		float OutPacingMaxDelayMs = -1.0f;
		bool bVideoPacingFieldTrial = Settings::GetVideoPacing(OutPacingFactor, OutPacingMaxDelayMs);
		if (bVideoPacingFieldTrial)
		{
			FString VideoPacingFieldTrialStr = TEXT("WebRTC-Video-Pacing/");
			bool bHasPacingFactor = OutPacingFactor >= 0.0f;
			if (bHasPacingFactor)
			{
				VideoPacingFieldTrialStr += FString::Printf(TEXT("factor:%.1f"), OutPacingFactor);
			}
			bool bHasMaxDelay = OutPacingMaxDelayMs >= 0.0f;
			if (bHasMaxDelay)
			{
				VideoPacingFieldTrialStr += bHasPacingFactor ? TEXT(",") : TEXT("");
				VideoPacingFieldTrialStr += FString::Printf(TEXT("max_delay:%.0f"), OutPacingMaxDelayMs);
			}
			VideoPacingFieldTrialStr += TEXT("/");
			FieldTrials += VideoPacingFieldTrialStr;
		}
	}

	if (!FieldTrials.IsEmpty())
	{
		// Pass the field trials string to WebRTC. String must never be destroyed.
		TStringConversion<TStringConvert<TCHAR, ANSICHAR>> Str = StringCast<ANSICHAR>(*FieldTrials);
		int length = Str.Length() + 1;
		char* WRTCFieldTrials = (char*)FMemory::SystemMalloc(length);
		FMemory::Memcpy(WRTCFieldTrials, Str.Get(), Str.Length());
		WRTCFieldTrials[length - 1] = '\0';
		webrtc::field_trial::InitFieldTrialsFromString(WRTCFieldTrials);
	}
}

void FPixelStreamingPeerConnection::CreatePeerConnectionFactory()
{
	using namespace UE::PixelStreaming;

	SignallingThread = MakeUnique<rtc::Thread>(rtc::CreateDefaultSocketServer());
	SignallingThread->SetName("FPixelStreamingPeerConnection SignallingThread", nullptr);
	SignallingThread->Start();

	rtc::scoped_refptr<webrtc::AudioDeviceModule> AudioDeviceModule;
	{
		if (Settings::CVarPixelStreamingWebRTCUseLegacyAudioDevice.GetValueOnAnyThread())
		{
			AudioDeviceModule = new rtc::RefCountedObject<FAudioCapturer>();
		}
		else
		{
			// If experimental audio input is enabled we pass the mixer, if not we don't - no mixer means only use the UE submix.
			if (Settings::CVarPixelStreamingExperimentalAudioInput.GetValueOnAnyThread())
			{
				AudioDeviceModule = new rtc::RefCountedObject<FAudioDeviceModule>(AudioMixer);
				UE_LOGF(LogPixelStreaming, Log, "Using -PixelStreamingExperimentalAudioInput. Pixel Streaming audio will mix the UE submix with user audio inputs.");
			}
			else
			{
				AudioDeviceModule = new rtc::RefCountedObject<FAudioDeviceModule>();
				UE_LOGF(LogPixelStreaming, Log, "Not using -PixelStreamingExperimentalAudioInput. Pixel Streaming audio will only transmit the UE submix.");
			}
		}
	}

	rtc::scoped_refptr<webrtc::AudioProcessing> AudioProcessingModule = webrtc::AudioProcessingBuilder().Create();
	{
		webrtc::AudioProcessing::Config Config;
		// Enabled multi channel audio capture/render
		Config.pipeline.multi_channel_capture = true;
		Config.pipeline.multi_channel_render = true;
		Config.pipeline.maximum_internal_processing_rate = 48000;
		// Turn off all other audio processing effects in UE's WebRTC. We want to stream audio from UE as pure as possible.
		Config.pre_amplifier.enabled = false;
		Config.high_pass_filter.enabled = false;
		Config.echo_canceller.enabled = false;
		Config.noise_suppression.enabled = false;
		Config.transient_suppression.enabled = false;
		Config.gain_controller1.enabled = false;
		Config.gain_controller2.enabled = false;
#if !WEBRTC_5414
		Config.voice_detection.enabled = false;
		Config.residual_echo_detector.enabled = false;
		Config.level_estimation.enabled = false;
#endif

		// Apply the config.
		AudioProcessingModule->ApplyConfig(Config);
	}

	std::unique_ptr<FVideoEncoderFactoryLayered> VideoEncoderFactory = std::make_unique<FVideoEncoderFactoryLayered>();
	GVideoEncoderFactory = VideoEncoderFactory.get();

	// Set up the field trials, read things from CVars etc
	InitializeFieldTrials();

	PeerConnectionFactory = webrtc::CreatePeerConnectionFactory(
		nullptr,													   // network_thread
		nullptr,													   // worker_thread
		SignallingThread.Get(),										   // signaling_thread
		AudioDeviceModule,											   // default_adm
		webrtc::CreateAudioEncoderFactory<webrtc::AudioEncoderOpus>(), // audio_encoder_factory
		webrtc::CreateAudioDecoderFactory<webrtc::AudioDecoderOpus>(), // audio_decoder_factory
		std::move(VideoEncoderFactory),								   // video_encoder_factory
		std::make_unique<FVideoDecoderFactory>(),					   // video_decoder_factory
		nullptr,													   // audio_mixer
		AudioProcessingModule);										   // audio_processing
	check(PeerConnectionFactory);
}
