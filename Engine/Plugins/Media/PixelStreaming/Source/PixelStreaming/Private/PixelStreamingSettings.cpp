// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingSettings.h"
#include "Settings.h"

FPixelStreamingSimulcastParameters FPixelStreamingSettings::GetSimulcastParameters() 
{
	return UE::PixelStreaming::Settings::SimulcastParameters;
}

bool FPixelStreamingSettings::GetCaptureUseFence() 
{
	return UE::PixelStreaming::Settings::CVarPixelStreamingCaptureUseFence.GetValueOnAnyThread();
}

bool FPixelStreamingSettings::GetVPXUseCompute() 
{
	return UE::PixelStreaming::Settings::CVarPixelStreamingVPXUseCompute.GetValueOnAnyThread();
}