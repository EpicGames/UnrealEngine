// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingSignallingConnection.h"
#include "ToStringExtensions.h"
#include "WebSocketsModule.h"
#include "IWebSocket.h"
#include "Engine/World.h"
#include "Serialization/JsonSerializer.h"
#include "Settings.h"
#include "TimerManager.h"
#include "PixelStreamingDelegates.h"
#include "Utils.h"
#include "ToStringExtensions.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "PixelStreamingModule.h"
#include "Engine/GameEngine.h"
#include "Engine/GameInstance.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreamingSS, Log, VeryVerbose);
DEFINE_LOG_CATEGORY(LogPixelStreamingSS);

template<typename T>
void FPixelStreamingSignallingConnection::SafeGameThreadExecute(T&& Func)
{
	TWeakPtr<int> WeakSemaphore = AliveSemaphore;
	UE::PixelStreaming::DoOnGameThread([this, WeakSemaphore, Func]() {
		if (WeakSemaphore.IsValid())
		{
			Func();
		}
	});
}

template<typename T>
void FPixelStreamingSignallingConnection::SafeGameThreadExecuteAndWait(uint32 Timeout, T&& Func)
{
	TWeakPtr<int> WeakSemaphore = AliveSemaphore;
	UE::PixelStreaming::DoOnGameThreadAndWait(Timeout, [this, WeakSemaphore, Func]() {
		if (WeakSemaphore.IsValid())
		{
			Func();
		}
	});
}

FPixelStreamingSignallingConnection::FPixelStreamingSignallingConnection(TSharedPtr<IPixelStreamingSignallingConnectionObserver> InObserver, FString InStreamerId, TSharedPtr<IWebSocket> InWebSocket)
	: Observer(InObserver)
	, StreamerId(InStreamerId)
	, WebSocket(InWebSocket)
{
	AliveSemaphore = MakeShared<int>(1);
	RegisterHandler("identify", [this](FJsonObjectPtr JsonMsg) { OnIdRequested(); });
	RegisterHandler("config", [this](FJsonObjectPtr JsonMsg) { OnConfig(JsonMsg); });
	RegisterHandler("offer", [this](FJsonObjectPtr JsonMsg) { OnSessionDescription(JsonMsg); });
	RegisterHandler("answer", [this](FJsonObjectPtr JsonMsg) { OnSessionDescription(JsonMsg); });
	RegisterHandler("iceCandidate", [this](FJsonObjectPtr JsonMsg) { OnIceCandidate(JsonMsg); });
	RegisterHandler("ping", [this](FJsonObjectPtr JsonMsg) { OnPing(JsonMsg); });
	RegisterHandler("pong", [this](FJsonObjectPtr JsonMsg) { OnPong(JsonMsg); });
	RegisterHandler("playerCount", [this](FJsonObjectPtr JsonMsg) { OnPlayerCount(JsonMsg); });
	RegisterHandler("playerConnected", [this](FJsonObjectPtr JsonMsg) { OnPlayerConnected(JsonMsg); });
	RegisterHandler("playerDisconnected", [this](FJsonObjectPtr JsonMsg) { OnPlayerDisconnected(JsonMsg); });
	RegisterHandler("streamerDataChannels", [this](FJsonObjectPtr JsonMsg) { OnSFUPeerDataChannels(JsonMsg); });
	RegisterHandler("peerDataChannels", [this](FJsonObjectPtr JsonMsg) { OnPeerDataChannels(JsonMsg); });
	RegisterHandler("streamerList", [this](FJsonObjectPtr JsonMsg) { OnStreamerList(JsonMsg); });
}

FPixelStreamingSignallingConnection::~FPixelStreamingSignallingConnection()
{
	Disconnect(TEXT("Streamer destroying websocket connection"));

	WebSocket.Reset();
	Observer.Reset();
	AliveSemaphore.Reset();
}

void FPixelStreamingSignallingConnection::Connect(FString InUrl, bool bIsReconnect)
{
	if (WebSocket && WebSocket->IsConnected())
	{
		if (bIsReconnect)
		{
			// If we somehow got here, turn off reconnect timer as we are already connected.
			StopReconnectTimer();
		}
		UE_LOGF(LogPixelStreamingSS, Log, "Skipping `Connect()` because we are already connected. If you want to reconnect then disconnect first.");
		return;
	}

	// Reconnecting on an existing websocket can be problematic depending what state it was
	// left in. Easier and safer to disconnect any existing socket/delegates and start fresh.
	Disconnect(TEXT("Streamer cleaning up old connection"));

	Url = InUrl;

	// add the default port if none is provided
	TOptional<uint16> MaybePort = FGenericPlatformHttp::GetUrlPort(Url);
	if (!MaybePort)
	{
		const FString Protocol = FGenericPlatformHttp::IsSecureProtocol(Url).Get(false) ? "wss" : "ws";
		const FString Domain = FGenericPlatformHttp::GetUrlDomain(Url);
		const uint16 Port = 8888;
		const FString Path = FGenericPlatformHttp::GetUrlPath(Url);
		const FString Final = FString::Printf(TEXT("%s://%s:%d%s"), *Protocol, *Domain, Port, *Path);
		Url = Final;
	}

	if(!WebSocket.IsValid())
	{	
		WebSocket = FWebSocketsModule::Get().CreateWebSocket(Url, TEXT(""));
		verifyf(WebSocket, TEXT("Web Socket Factory failed to return a valid Web Socket."));
	}

	OnConnectedHandle = WebSocket->OnConnected().AddLambda([this]() { OnConnected(); });
	OnConnectionErrorHandle = WebSocket->OnConnectionError().AddLambda([this](const FString& Error) { OnConnectionError(Error); });
	OnClosedHandle = WebSocket->OnClosed().AddLambda([this](int32 StatusCode, const FString& Reason, bool bWasClean) { OnClosed(StatusCode, Reason, bWasClean); });
	OnMessageHandle = WebSocket->OnMessage().AddLambda([this](const FString& Msg) { OnMessage(Msg); });
	OnBinaryMessageHandle = WebSocket->OnBinaryMessage().AddLambda([this](const void* Data, int32 Count, bool bIsLastFragment) { OnBinaryMessage((const uint8*)Data, Count, bIsLastFragment); });

	if (bIsReconnect)
	{
		UE_LOGF(LogPixelStreamingSS, Log, "Reconnecting to SS %ls", *Url);
	}
	else
	{
		UE_LOGF(LogPixelStreamingSS, Log, "Connecting to SS %ls", *Url);
	}

	WebSocket->Connect();
	bIsConnected = true;
}

void FPixelStreamingSignallingConnection::TryConnect(FString InUrl)
{
	Connect(InUrl, false /*bIsReconnect*/);
}

void FPixelStreamingSignallingConnection::Disconnect()
{
	// Do not call this deprecated method, please call FPixelStreamingSignallingConnection::Disconnect(FString Reason)
	Disconnect(TEXT("Unknown reason"));
}

void FPixelStreamingSignallingConnection::Disconnect(FString Reason)
{
	if (!IsEngineExitRequested())
	{
		StopKeepAliveTimer();
		StopReconnectTimer();
	}
	else
	{
		Reason = TEXT("Streamed application is shutting down");
	}

	if (!WebSocket || !bIsConnected)
	{
		return;
	}

	WebSocket->OnConnected().Remove(OnConnectedHandle);
	WebSocket->OnConnectionError().Remove(OnConnectionErrorHandle);
	WebSocket->OnClosed().Remove(OnClosedHandle);
	WebSocket->OnMessage().Remove(OnMessageHandle);
	WebSocket->OnBinaryMessage().Remove(OnBinaryMessageHandle);

	WebSocket->Close(1000, Reason);
	UE_LOGF(LogPixelStreamingSS, Log, "Closing websocket to SS %ls", *Url);

	bIsConnected = false;
}

bool FPixelStreamingSignallingConnection::IsConnected() const
{
	return WebSocket != nullptr && WebSocket.IsValid() && WebSocket->IsConnected();
}

void FPixelStreamingSignallingConnection::SendOffer(FPixelStreamingPlayerId PlayerId, const webrtc::SessionDescriptionInterface& SDP)
{
	FJsonObjectPtr OfferJson = MakeShared<FJsonObject>();
	OfferJson->SetStringField(TEXT("type"), TEXT("offer"));
	SetPlayerIdJson(OfferJson, PlayerId);

	std::string SdpAnsi;
	SDP.ToString(&SdpAnsi);
	FString SdpStr = UE::PixelStreaming::ToString(SdpAnsi);
	OfferJson->SetStringField(TEXT("sdp"), SdpStr);

	UE_LOGF(LogPixelStreamingSS, Log, "Sending player=%ls \"offer\" to SS %ls", *PlayerId, *Url);
	UE_LOGF(LogPixelStreamingSS, Verbose, "SDP offer\n%ls", *SdpStr);

	SendMessage(UE::PixelStreaming::ToString(OfferJson, false));
}

void FPixelStreamingSignallingConnection::SendAnswer(FPixelStreamingPlayerId PlayerId, const webrtc::SessionDescriptionInterface& SDP)
{
	FJsonObjectPtr AnswerJson = MakeShared<FJsonObject>();
	AnswerJson->SetStringField(TEXT("type"), TEXT("answer"));
	SetPlayerIdJson(AnswerJson, PlayerId);

	std::string SdpAnsi;

	if (SDP.ToString(&SdpAnsi))
	{
		FString SdpStr = UE::PixelStreaming::ToString(SdpAnsi);
		AnswerJson->SetStringField(TEXT("sdp"), SdpStr);

		UE_LOGF(LogPixelStreamingSS, Log, "Sending player=%ls \"answer\" to SS %ls", *PlayerId, *Url);
		UE_LOGF(LogPixelStreamingSS, Verbose, "SDP answer\n%ls", *SdpStr);

		SendMessage(UE::PixelStreaming::ToString(AnswerJson, false));
	}
	else
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to serialise local SDP.");
	}
}

void FPixelStreamingSignallingConnection::SendIceCandidate(FPixelStreamingPlayerId PlayerId, const webrtc::IceCandidateInterface& IceCandidate)
{
	FJsonObjectPtr IceCandidateJson = MakeShared<FJsonObject>();

	IceCandidateJson->SetStringField(TEXT("type"), TEXT("iceCandidate"));
	SetPlayerIdJson(IceCandidateJson, PlayerId);

	FJsonObjectPtr CandidateJson = MakeShared<FJsonObject>();
	CandidateJson->SetStringField(TEXT("sdpMid"), IceCandidate.sdp_mid().c_str());
	CandidateJson->SetNumberField(TEXT("sdpMLineIndex"), static_cast<double>(IceCandidate.sdp_mline_index()));

	std::string CandidateAnsi;
	if (IceCandidate.ToString(&CandidateAnsi))
	{
		FString CandidateStr = UE::PixelStreaming::ToString(CandidateAnsi);
		CandidateJson->SetStringField(TEXT("candidate"), CandidateStr);
		IceCandidateJson->SetObjectField(TEXT("candidate"), CandidateJson);
		UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: iceCandidate\n%ls", *UE::PixelStreaming::ToString(IceCandidateJson));
		SendMessage(UE::PixelStreaming::ToString(IceCandidateJson, false));
	}
	else
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to serialize IceCandidate");
	}
}

void FPixelStreamingSignallingConnection::SendDisconnectPlayer(FPixelStreamingPlayerId PlayerId, const FString& Reason)
{
	FJsonObjectPtr Json = MakeShared<FJsonObject>();

	Json->SetStringField(TEXT("type"), TEXT("disconnectPlayer"));
	SetPlayerIdJson(Json, PlayerId);
	Json->SetStringField(TEXT("reason"), Reason);

	FString Msg = UE::PixelStreaming::ToString(Json, false);
	UE_LOGF(LogPixelStreamingSS, Verbose, "Sending player=%ls \"disconnectPlayer\" to SS %ls", *PlayerId, *Url);

	SendMessage(Msg);
}

void FPixelStreamingSignallingConnection::RequestStreamerList()
{
	FJsonObjectPtr Json = MakeShared<FJsonObject>();
	Json->SetStringField(TEXT("type"), TEXT("listStreamers"));

	const FString JsonStr = UE::PixelStreaming::ToString(Json, false);
	UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: listStreamers\n%ls", *JsonStr);

	SendMessage(JsonStr);
}

void FPixelStreamingSignallingConnection::SendSubscribe(const FString& ToStreamerId)
{
	FJsonObjectPtr SubscribeJson = MakeShared<FJsonObject>();
	SubscribeJson->SetStringField(TEXT("type"), TEXT("subscribe"));
	SubscribeJson->SetStringField(TEXT("streamerId"), ToStreamerId);

	const FString SubscribeJsonStr = UE::PixelStreaming::ToString(SubscribeJson, false);
	UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: subscribe\n%ls", *SubscribeJsonStr);

	SendMessage(SubscribeJsonStr);
}

void FPixelStreamingSignallingConnection::SendUnsubscribe()
{
	FJsonObjectPtr UnsubscribeJson = MakeShared<FJsonObject>();
	UnsubscribeJson->SetStringField(TEXT("type"), TEXT("unsubscribe"));

	const FString UnsubscribeJsonStr = UE::PixelStreaming::ToString(UnsubscribeJson, false);
	UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: unsubscribe\n%ls", *UnsubscribeJsonStr);

	SendMessage(UnsubscribeJsonStr);
}

void FPixelStreamingSignallingConnection::SendOffer(const webrtc::SessionDescriptionInterface& SDP)
{
	FJsonObjectPtr OfferJson = MakeShared<FJsonObject>();
	OfferJson->SetStringField(TEXT("type"), TEXT("offer"));

	FString SdpStr = UE::PixelStreaming::ToString(SDP);
	OfferJson->SetStringField(TEXT("sdp"), UE::PixelStreaming::ToString(SDP));

	UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: offer\n%ls", *SdpStr);

	SendMessage(UE::PixelStreaming::ToString(OfferJson, false));
}

void FPixelStreamingSignallingConnection::SendAnswer(const webrtc::SessionDescriptionInterface& SDP)
{
	FJsonObjectPtr AnswerJson = MakeShared<FJsonObject>();
	AnswerJson->SetStringField(TEXT("type"), TEXT("answer"));

	FString SdpStr = UE::PixelStreaming::ToString(SDP);
	AnswerJson->SetStringField(TEXT("sdp"), UE::PixelStreaming::ToString(SDP));

	UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: answer\n%ls", *SdpStr);

	SendMessage(UE::PixelStreaming::ToString(AnswerJson, false));
}

void FPixelStreamingSignallingConnection::SendIceCandidate(const webrtc::IceCandidateInterface& IceCandidate)
{
	FJsonObjectPtr IceCandidateJson = MakeShared<FJsonObject>();

	IceCandidateJson->SetStringField(TEXT("type"), TEXT("iceCandidate"));

	FJsonObjectPtr CandidateJson = MakeShared<FJsonObject>();
	CandidateJson->SetStringField(TEXT("sdpMid"), IceCandidate.sdp_mid().c_str());
	CandidateJson->SetNumberField(TEXT("sdpMLineIndex"), static_cast<double>(IceCandidate.sdp_mline_index()));

	std::string CandidateAnsi;
	verifyf(IceCandidate.ToString(&CandidateAnsi), TEXT("Failed to serialize IceCandidate"));
	FString CandidateStr = UE::PixelStreaming::ToString(CandidateAnsi);
	CandidateJson->SetStringField(TEXT("candidate"), CandidateStr);

	IceCandidateJson->SetObjectField(TEXT("candidate"), CandidateJson);

	UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: ice-candidate\n%ls", *UE::PixelStreaming::ToString(IceCandidateJson));

	SendMessage(UE::PixelStreaming::ToString(IceCandidateJson, false));
}

void FPixelStreamingSignallingConnection::SetAutoReconnect(bool bAutoReconnect)
{
	if (bAutoReconnectEnabled == bAutoReconnect)
	{
		return;
	}

	StopReconnectTimer();
	if (bAutoReconnect)
	{
		StartReconnectTimer();
	}
	bAutoReconnectEnabled = bAutoReconnect;
}

void FPixelStreamingSignallingConnection::SetKeepAlive(bool bKeepAlive)
{
	if (bKeepAliveEnabled == bKeepAlive)
	{
		return;
	}

	StopKeepAliveTimer();
	if (bKeepAlive)
	{
		StartKeepAliveTimer();
	}
	bKeepAliveEnabled = bKeepAlive;
}

void FPixelStreamingSignallingConnection::OnConnected()
{
	UE_LOGF(LogPixelStreamingSS, Log, "Connected to SS %ls", *Url);

	// No need to to do reconnect now that we are connected
	StopReconnectTimer();

	Observer->OnSignallingConnected();

	if (bKeepAliveEnabled)
	{
		StartKeepAliveTimer();
	}

	if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
	{
		Delegates->OnConnectedToSignallingServer.Broadcast();
		Delegates->OnConnectedToSignallingServerNative.Broadcast();
	}
}

void FPixelStreamingSignallingConnection::OnPing(const FJsonObjectPtr& Json)
{
	UE_LOGF(LogPixelStreamingSS, Verbose, "Streamer got pinged.");
}
void FPixelStreamingSignallingConnection::OnPong(const FJsonObjectPtr& Json)
{
	UE_LOGF(LogPixelStreamingSS, Verbose, "Streamer got ponged.");
}

void FPixelStreamingSignallingConnection::OnConnectionError(const FString& Error)
{
	UE_LOGF(LogPixelStreamingSS, Log, "Failed to connect to SS %ls - signalling server may not be up yet. Message: \"%ls\"", *Url, *Error);

	Observer->OnSignallingError(Error);

	StopKeepAliveTimer();

	if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
	{
		Delegates->OnDisconnectedFromSignallingServer.Broadcast();
		Delegates->OnDisconnectedFromSignallingServerNative.Broadcast();
	}

	if (bAutoReconnectEnabled)
	{
		StartReconnectTimer();
	}
}

void FPixelStreamingSignallingConnection::OnClosed(int32 StatusCode, const FString& Reason, bool bWasClean)
{
	UE_LOGF(LogPixelStreamingSS, Log, "Closed connection to SS %ls - \n\tstatus %d\n\treason: %ls\n\twas clean: %ls", *Url, StatusCode, *Reason, bWasClean ? TEXT("true") : TEXT("false"));

	Observer->OnSignallingDisconnected(StatusCode, Reason, bWasClean);

	StopKeepAliveTimer();

	if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
	{
		Delegates->OnDisconnectedFromSignallingServer.Broadcast();
		Delegates->OnDisconnectedFromSignallingServerNative.Broadcast();
	}

	if (bAutoReconnectEnabled)
	{
		StartReconnectTimer();
	}
}

void FPixelStreamingSignallingConnection::OnMessage(const FString& Msg)
{
	FJsonObjectPtr JsonMsg;
	const auto JsonReader = TJsonReaderFactory<TCHAR>::Create(Msg);

	if (!FJsonSerializer::Deserialize(JsonReader, JsonMsg))
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to parse SS message:\n%ls", *Msg);
		return;
	}

	FString MsgType;
	if (!JsonMsg->TryGetStringField(TEXT("type"), MsgType))
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Cannot find `type` field in SS message:\n%ls", *Msg);
		return;
	}

	TFunction<void(FJsonObjectPtr)>* Handler = MessageHandlers.Find(MsgType);
	if (Handler != nullptr)
	{
		(*Handler)(JsonMsg);
	}
	else
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Unsupported message `%ls` received from SS", *MsgType);
	}
}

void FPixelStreamingSignallingConnection::OnBinaryMessage(const uint8* Data, int32 Length, bool bIsLastFragment)
{
	FUTF8ToTCHAR Convert((const ANSICHAR*)Data, Length);
	const TCHAR* PayloadChars = Convert.Get();
	FString Msg = FString::ConstructFromPtrSize(PayloadChars, Convert.Length());
	OnMessage(Msg);
}

void FPixelStreamingSignallingConnection::RegisterHandler(const FString& messageType, const TFunction<void(FJsonObjectPtr)>& handler)
{
	MessageHandlers.Add(messageType, handler);
}

// This function returns the instance ID to the signalling server. This is useful for identifying individual instances in scalable cloud deployments
void FPixelStreamingSignallingConnection::OnIdRequested()
{
	FJsonObjectPtr Json = MakeShared<FJsonObject>();
	Json->SetStringField(TEXT("type"), TEXT("endpointId"));
	Json->SetStringField(TEXT("id"), StreamerId);
	FString Msg = UE::PixelStreaming::ToString(Json, false);
	UE_LOGF(LogPixelStreamingSS, Verbose, "-> SS: endpointId\n%ls", *Msg);
	SendMessage(Msg);
}

void FPixelStreamingSignallingConnection::OnConfig(const FJsonObjectPtr& Json)
{
	// SS sends `config` that looks like:
	// `{peerConnectionOptions: { 'iceServers': [{'urls': ['stun:34.250.222.95:19302', 'turn:34.250.222.95:19303']}] }}`
	// where `peerConnectionOptions` is `RTCConfiguration` (except in native `RTCConfiguration` "iceServers" = "servers").
	// As `RTCConfiguration` doesn't implement parsing from a string (or `UE::PixelStreaming::ToString` method),
	// we just get `stun`/`turn` URLs from it and ignore other options

	const TSharedPtr<FJsonObject>* PeerConnectionOptions = nullptr;
	if (!Json->TryGetObjectField(TEXT("peerConnectionOptions"), PeerConnectionOptions))
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Cannot find `peerConnectionOptions` field in SS config\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	webrtc::PeerConnectionInterface::RTCConfiguration RTCConfig;

	const TArray<TSharedPtr<FJsonValue>>* IceServers = nullptr;
	if ((*PeerConnectionOptions)->TryGetArrayField(TEXT("iceServers"), IceServers))
	{
		for (const TSharedPtr<FJsonValue>& IceServerVal : *IceServers)
		{
			const TSharedPtr<FJsonObject>* IceServerJson;
			if (!IceServerVal->TryGetObject(IceServerJson))
			{
				UE_LOGF(LogPixelStreamingSS, Error, "Failed to parse SS config: `iceServer` - not an object\n%ls", *UE::PixelStreaming::ToString(*PeerConnectionOptions));
				return;
			}

			RTCConfig.servers.push_back(webrtc::PeerConnectionInterface::IceServer{});
			webrtc::PeerConnectionInterface::IceServer& IceServer = RTCConfig.servers.back();

			TArray<FString> Urls;
			if ((*IceServerJson)->TryGetStringArrayField(TEXT("urls"), Urls))
			{
				for (const FString& UrlElem : Urls)
				{
					IceServer.urls.push_back(UE::PixelStreaming::ToString(UrlElem));
				}
			}
			else
			{
				// in the RTC Spec, "urls" can be an array or a single string
				// https://www.w3.org/TR/webrtc/#dictionary-rtciceserver-members
				FString UrlsSingle;
				if ((*IceServerJson)->TryGetStringField(TEXT("urls"), UrlsSingle))
				{
					IceServer.urls.push_back(UE::PixelStreaming::ToString(UrlsSingle));
				}
			}

			FString Username;
			if ((*IceServerJson)->TryGetStringField(TEXT("username"), Username))
			{
				IceServer.username = UE::PixelStreaming::ToString(Username);
			}

			FString Credential;
			if ((*IceServerJson)->TryGetStringField(TEXT("credential"), Credential))
			{
				IceServer.password = UE::PixelStreaming::ToString(Credential);
			}
		}
	}

	// force `UnifiedPlan` as we control both ends of WebRTC streaming
	RTCConfig.sdp_semantics = webrtc::SdpSemantics::kUnifiedPlan;

#if WEBRTC_5414
	int MinPort, MaxPort;
	MinPort = UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMinPort.GetValueOnAnyThread();
	MaxPort = UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMaxPort.GetValueOnAnyThread();

	if(MinPort < 0 || MinPort > 65535)
	{
		UE_LOGF(LogPixelStreamingSS, Warning, "Invalid PixelStreaming.WebRTC.MinPort specified. Value must be within 0 to 65535 inclusive");
		MinPort = 49152;
		UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMinPort->Set(MinPort, ECVF_SetByCode);
	}

	if(MaxPort < 0 || MaxPort > 65535)
	{
		UE_LOGF(LogPixelStreamingSS, Warning, "Invalid PixelStreaming.WebRTC.MaxPort specified. Value must be within 0 to 65535 inclusive");
		MaxPort = 65535;
		UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMaxPort->Set(MaxPort, ECVF_SetByCode);
	}

	if (MinPort > MaxPort)
	{
		int OldMax = MaxPort;
		MaxPort = MinPort;
		MinPort = OldMax;

		// To try to not be misleading with debug texts etc, we reset these sanitised settings here
		UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMinPort->Set(MinPort, ECVF_SetByCode);
		UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCMaxPort->Set(MaxPort, ECVF_SetByCode);
	}

	RTCConfig.set_min_port(MinPort);
	RTCConfig.set_max_port(MaxPort);

	RTCConfig.set_port_allocator_flags(UE::PixelStreaming::Settings::PortAllocatorParameters);
#endif

	Observer->OnSignallingConfig(RTCConfig);
}

void FPixelStreamingSignallingConnection::OnSessionDescription(const FJsonObjectPtr& Json)
{
	webrtc::SdpType Type = Json->GetStringField(TEXT("type")) == TEXT("offer") ? webrtc::SdpType::kOffer : webrtc::SdpType::kAnswer;

	FString Sdp;
	if (!Json->TryGetStringField(TEXT("sdp"), Sdp))
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Cannot find `sdp` in Streamer's answer\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	FPixelStreamingPlayerId PlayerId;
	bool bGotPlayerId = GetPlayerIdJson(Json, PlayerId);
	if (bGotPlayerId)
	{
		int MinBitrate;
		int MaxBitrate;
		bool bGotMinBitrate;
		bool bGotMaxBitrate;

		bGotMinBitrate = Json->TryGetNumberField(TEXT("minBitrate"), MinBitrate);
		bGotMaxBitrate = Json->TryGetNumberField(TEXT("maxBitrate"), MaxBitrate);

		if (bGotMinBitrate && bGotMaxBitrate && MinBitrate > 0 && MaxBitrate > 0)
		{
			Observer->OnPlayerRequestsBitrate(PlayerId, MinBitrate, MaxBitrate);
		}

		Observer->OnSignallingSessionDescription(PlayerId, Type, Sdp);
	}
	else
	{
		Observer->OnSignallingSessionDescription(Type, Sdp);
	}
}

void FPixelStreamingSignallingConnection::OnIceCandidate(const FJsonObjectPtr& Json)
{
	FPixelStreamingPlayerId PlayerId;
	bool bGotPlayerId = GetPlayerIdJson(Json, PlayerId);

	const FJsonObjectPtr* CandidateJson;
	if (!Json->TryGetObjectField(TEXT("candidate"), CandidateJson))
	{
		PlayerError(PlayerId, TEXT("Failed to get `candiate` from remote `iceCandidate` message\n%s"), *UE::PixelStreaming::ToString(Json));
	}

	FString SdpMid;
	if (!(*CandidateJson)->TryGetStringField(TEXT("sdpMid"), SdpMid))
	{
		PlayerError(PlayerId, TEXT("Failed to get `sdpMid` from remote `iceCandidate` message\n%s"), *UE::PixelStreaming::ToString(Json));
	}

	int32 SdpMLineIndex = 0;
	if (!(*CandidateJson)->TryGetNumberField(TEXT("sdpMlineIndex"), SdpMLineIndex))
	{
		PlayerError(PlayerId, TEXT("Failed to get `sdpMlineIndex` from remote `iceCandidate` message\n%s"), *UE::PixelStreaming::ToString(Json));
	}

	FString CandidateStr;
	if (!(*CandidateJson)->TryGetStringField(TEXT("candidate"), CandidateStr))
	{
		PlayerError(PlayerId, TEXT("Failed to get `candidate` from remote `iceCandidate` message\n%s"), *UE::PixelStreaming::ToString(Json));
	}

	if (bGotPlayerId)
	{
		Observer->OnSignallingRemoteIceCandidate(PlayerId, SdpMid, SdpMLineIndex, CandidateStr);
	}
	else
	{
		Observer->OnSignallingRemoteIceCandidate(SdpMid, SdpMLineIndex, CandidateStr);
	}
}

void FPixelStreamingSignallingConnection::OnPlayerCount(const FJsonObjectPtr& Json)
{
	uint32 Count;
	if (!Json->TryGetNumberField(TEXT("count"), Count))
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `count` from `playerCount` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	Observer->OnSignallingPlayerCount(Count);
}

void FPixelStreamingSignallingConnection::OnPlayerConnected(const FJsonObjectPtr& Json)
{
	FPixelStreamingPlayerId PlayerId;
	bool bGotPlayerId = GetPlayerIdJson(Json, PlayerId);
	if (!bGotPlayerId)
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `playerId` from `join` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	UE_LOGF(LogPixelStreamingSS, Log, "Got player connected, player id=%ls", *PlayerId);

	FPixelStreamingPlayerConfig PlayerConfig;

	// Default to always making datachannel, unless explicitly set to false.
	Json->TryGetBoolField(TEXT("dataChannel"), PlayerConfig.SupportsDataChannel);

	// Default peer is not an SFU, unless explictly set as SFU
	Json->TryGetBoolField(TEXT("sfu"), PlayerConfig.IsSFU);

	// Default to always sending an offer, unless explicitly set to false
	bool bSendOffer = true;
	Json->TryGetBoolField(TEXT("sendOffer"), bSendOffer);

	Observer->OnSignallingPlayerConnected(PlayerId, PlayerConfig, bSendOffer);
}

void FPixelStreamingSignallingConnection::OnPlayerDisconnected(const FJsonObjectPtr& Json)
{
	FPixelStreamingPlayerId PlayerId;
	bool bGotPlayerId = GetPlayerIdJson(Json, PlayerId);
	if (!bGotPlayerId)
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `playerId` from `playerDisconnected` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	Observer->OnSignallingPlayerDisconnected(PlayerId);
}

void FPixelStreamingSignallingConnection::OnSFUPeerDataChannels(const FJsonObjectPtr& Json)
{
	FPixelStreamingPlayerId SFUId;
	bool bSuccess = GetPlayerIdJson(Json, SFUId, TEXT("sfuId"));
	if (!bSuccess)
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `sfuId` from `streamerDataChannels` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	FPixelStreamingPlayerId PlayerId;
	bSuccess = GetPlayerIdJson(Json, PlayerId, TEXT("playerId"));
	if (!bSuccess)
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `playerId` from `streamerDataChannels` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	int32 SendStreamId;
	bSuccess = Json->TryGetNumberField(TEXT("sendStreamId"), SendStreamId);
	if (!bSuccess)
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `sendStreamId` from `streamerDataChannels` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	int32 RecvStreamId;
	bSuccess = Json->TryGetNumberField(TEXT("recvStreamId"), RecvStreamId);
	if (!bSuccess)
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `recvStreamId` from `streamerDataChannels` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}

	Observer->OnSignallingSFUPeerDataChannels(SFUId, PlayerId, SendStreamId, RecvStreamId);
}

void FPixelStreamingSignallingConnection::OnPeerDataChannels(const FJsonObjectPtr& Json)
{
	int32 SendStreamId = 0;
	if (!Json->TryGetNumberField(TEXT("sendStreamId"), SendStreamId))
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `sendStreamId` from remote `peerDataChannels` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}
	int32 RecvStreamId = 0;
	if (!Json->TryGetNumberField(TEXT("recvStreamId"), RecvStreamId))
	{
		UE_LOGF(LogPixelStreamingSS, Error, "Failed to get `recvStreamId` from remote `peerDataChannels` message\n%ls", *UE::PixelStreaming::ToString(Json));
		return;
	}
	Observer->OnSignallingPeerDataChannels(SendStreamId, RecvStreamId);
}

void FPixelStreamingSignallingConnection::OnStreamerList(const FJsonObjectPtr& Json)
{
	TArray<FString> ResultList;
	const TArray<TSharedPtr<FJsonValue>>* JsonStreamerIds = nullptr;
	if (Json->TryGetArrayField(TEXT("ids"), JsonStreamerIds))
	{
		for (auto& JsonId : *JsonStreamerIds)
		{
			ResultList.Add(JsonId->AsString());
		}
	}
	Observer->OnSignallingStreamerList(ResultList);
}

void FPixelStreamingSignallingConnection::SetPlayerIdJson(FJsonObjectPtr& JsonObject, FPixelStreamingPlayerId PlayerId)
{
	bool bSendAsInteger = UE::PixelStreaming::Settings::CVarSendPlayerIdAsInteger.GetValueOnAnyThread();
	if (bSendAsInteger)
	{
		int32 PlayerIdAsInt = PlayerIdToInt(PlayerId);
		JsonObject->SetNumberField(TEXT("playerId"), PlayerIdAsInt);
	}
	else
	{
		JsonObject->SetStringField(TEXT("playerId"), PlayerId);
	}
}

bool FPixelStreamingSignallingConnection::GetPlayerIdJson(const FJsonObjectPtr& Json, FPixelStreamingPlayerId& OutPlayerId, const FString& FieldId)
{
	bool bSendAsInteger = UE::PixelStreaming::Settings::CVarSendPlayerIdAsInteger.GetValueOnAnyThread();
	if (bSendAsInteger)
	{
		uint32 PlayerIdInt;
		if (Json->TryGetNumberField(FieldId, PlayerIdInt))
		{
			OutPlayerId = ToPlayerId(PlayerIdInt);
			return true;
		}
	}
	else if (Json->TryGetStringField(FieldId, OutPlayerId))
	{
		return true;
	}
	return false;
}

namespace UE::PixelStreamingSignallingConnection::Private
{
FTimerManager* GetTimerManager()
{
#if WITH_EDITOR
	if (GEditor)
	{
		// In editor use the editor manager
		if (GEditor->IsTimerManagerValid())
		{
			return &GEditor->GetTimerManager().Get();
		}
	}
	else
#endif
	{
		// Otherwise we should always have a game instance
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& WorldContext : WorldContexts)
		{
			if (WorldContext.WorldType == EWorldType::Game && WorldContext.OwningGameInstance)
			{
				return &WorldContext.OwningGameInstance->GetTimerManager();
			}
		}
	}

	return nullptr;
}
} // namespace UE::PixelStreamingSignallingConnection::Private

void FPixelStreamingSignallingConnection::StartKeepAliveTimer()
{
	// Dereferencing needs to happen on the game thread
	// we dont need to wait since its just setting the timer
	SafeGameThreadExecute([this]() {
		using namespace UE::PixelStreamingSignallingConnection::Private;
		FTimerManager* TimerManager = GetTimerManager();
		if (TimerManager && !TimerManager->IsTimerActive(TimerHandle_KeepAlive))
		{
			TimerManager->SetTimer(TimerHandle_KeepAlive, FTimerDelegate::CreateLambda([this, TimerManager, TimerHandle = TimerHandle_KeepAlive, WeakSemaphore = AliveSemaphore.ToWeakPtr()]() {
				if (WeakSemaphore.IsValid())
				{
					FJsonObjectPtr Json = MakeShared<FJsonObject>();
					const double UnixTime = FDateTime::UtcNow().ToUnixTimestamp();
					Json->SetStringField(TEXT("type"), TEXT("ping"));
					Json->SetNumberField(TEXT("time"), UnixTime);
					SendMessage(UE::PixelStreaming::ToString(Json, false));
				}
				else
				{
					// Owning connection has gone away for whatever reason. Clear this timer.
					// TimerHandle captured is const and we cant use TimerHandle_KeepAlive because 'this' might have been deleted.
					// ClearTimer takes a non const reference so we cant use TimerHandle.
					FTimerHandle Handle = TimerHandle;
					TimerManager->ClearTimer(Handle);
				}
			}), KEEP_ALIVE_INTERVAL, true);
		}
	});
}

void FPixelStreamingSignallingConnection::StopKeepAliveTimer()
{
	// Dereferencing needs to happen on the game thread
	// we need to wait because if we're destructing this object we dont
	// want to call the callback mid/post destruction
	SafeGameThreadExecuteAndWait(MAX_uint32, [this]() {
		using namespace UE::PixelStreamingSignallingConnection::Private;
		FTimerManager* TimerManager = GetTimerManager();
		if (TimerManager)
		{
			TimerManager->ClearTimer(TimerHandle_KeepAlive);
		}
	});
}

void FPixelStreamingSignallingConnection::StartReconnectTimer()
{
	// Dereferencing needs to happen on the game thread
	SafeGameThreadExecute([this]() {
		if (IsEngineExitRequested())
		{
			return;
		}
		using namespace UE::PixelStreamingSignallingConnection::Private;
		FTimerManager* TimerManager = GetTimerManager();
		if (TimerManager && !TimerManager->IsTimerActive(TimerHandle_Reconnect))
		{
			float ReconnectInterval = UE::PixelStreaming::Settings::CVarPixelStreamingSignalingReconnectInterval.GetValueOnAnyThread();
			FTimerDelegate delegate = FTimerDelegate::CreateLambda([this, WeakSemaphore = AliveSemaphore.ToWeakPtr()]() {
				if (WeakSemaphore.IsValid())
				{
					Connect(Url, true);
				}
				});
			TimerManager->SetTimer(TimerHandle_Reconnect, delegate, ReconnectInterval, true);
		}
	});
}

void FPixelStreamingSignallingConnection::StopReconnectTimer()
{
	// GWorld dereferencing needs to happen on the game thread
	SafeGameThreadExecute([this]() {
		if (IsEngineExitRequested())
		{
			return;
		}
		using namespace UE::PixelStreamingSignallingConnection::Private;
		FTimerManager* TimerManager = GetTimerManager();
		if (TimerManager)
		{
			TimerManager->ClearTimer(TimerHandle_Reconnect);
		}
	});
}

void FPixelStreamingSignallingConnection::SendMessage(const FString& Msg)
{
	if (WebSocket && WebSocket->IsConnected())
	{
		WebSocket->Send(Msg);
	}
}

void FPixelStreamingSignallingConnection::PlayerError(FPixelStreamingPlayerId PlayerId, const FString& Msg)
{
	UE_LOGF(LogPixelStreamingSS, Error, "player %ls: %ls", *PlayerId, *Msg);
	SendDisconnectPlayer(PlayerId, Msg);
}
