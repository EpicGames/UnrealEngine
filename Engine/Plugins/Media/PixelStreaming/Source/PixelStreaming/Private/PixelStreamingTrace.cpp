// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingTrace.h"

#include "Trace/Trace.inl"

UE_TRACE_CHANNEL_DEFINE(PixelStreamingChannel, "Traces PixelStreaming (first generation) remote rendering and streaming operations including video \
encoding / decoding (hardware / software / VPX), audio processing, streaming state, and performance metrics for debugging \
streaming sessions.");
