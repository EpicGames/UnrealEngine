// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInput.h"
#include "Settings.h"
#include "FrameBufferMultiFormat.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelStreamingTrace.h"

uint32 FPixelStreamingVideoInput::NextStreamId = 1;

FPixelStreamingVideoInput::FPixelStreamingVideoInput()
	: StreamId(NextStreamId++)
{
	CreateFrameCapturer();
}

void FPixelStreamingVideoInput::AddOutputFormat(int32 Format)
{
	PreInitFormats.Add(Format);
	FrameCapturer->AddOutputFormat(Format);
}

void FPixelStreamingVideoInput::OnFrame(const IPixelCaptureInputFrame& InputFrame)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Video Input Frame", PixelStreamingChannel);
	if (LastFrameWidth != -1 && LastFrameHeight != -1)
	{
		if (InputFrame.GetWidth() != LastFrameWidth || InputFrame.GetHeight() != LastFrameHeight)
		{
			CreateFrameCapturer();
		}
	}

	Ready = true;
	
	LastFrameWidth = InputFrame.GetWidth();
	LastFrameHeight = InputFrame.GetHeight();

	FrameCapturer->Capture(InputFrame);
}

rtc::scoped_refptr<webrtc::VideoFrameBuffer> FPixelStreamingVideoInput::GetFrameBuffer()
{
#if WEBRTC_5414
	return rtc::make_ref_counted<UE::PixelStreaming::FFrameBufferMultiFormatLayered>(FrameCapturer, StreamId, FIntPoint(LastFrameWidth, LastFrameHeight));
#else
	return new rtc::RefCountedObject<UE::PixelStreaming::FFrameBufferMultiFormatLayered>(FrameCapturer, StreamId, FIntPoint(LastFrameWidth, LastFrameHeight));
#endif
}

rtc::scoped_refptr<webrtc::VideoFrameBuffer> FPixelStreamingVideoInput::GetEmptyFrameBuffer()
{
#if WEBRTC_5414
	return rtc::make_ref_counted<UE::PixelStreaming::FFrameBufferMultiFormatLayered>(nullptr, StreamId, FIntPoint(LastFrameWidth, LastFrameHeight));
#else
	return new rtc::RefCountedObject<UE::PixelStreaming::FFrameBufferMultiFormatLayered>(nullptr, StreamId, FIntPoint(LastFrameWidth, LastFrameHeight));
#endif
}

TSharedPtr<IPixelCaptureOutputFrame> FPixelStreamingVideoInput::RequestFormat(int32 Format, TOptional<FIntPoint> Resolution)
{
	if (FrameCapturer != nullptr)
	{
		if (!Resolution.IsSet())
		{
			Resolution = { LastFrameWidth, LastFrameHeight };
		}
		return FrameCapturer->RequestFormat(Format, *Resolution);
	}
	return nullptr;
}

void FPixelStreamingVideoInput::CreateFrameCapturer()
{
	if (FrameCapturer != nullptr)
	{
		FrameCapturer->OnDisconnected();
		FrameCapturer->OnComplete.Remove(CaptureCompleteHandle);
		FrameCapturer = nullptr;
	}

	FrameCapturer = FPixelCaptureCapturerMultiFormat::Create(this);
	CaptureCompleteHandle = FrameCapturer->OnComplete.AddRaw(this, &FPixelStreamingVideoInput::OnCaptureComplete);

	for (auto& Format : PreInitFormats)
	{
		FrameCapturer->AddOutputFormat(Format);
	}
}

void FPixelStreamingVideoInput::OnCaptureComplete()
{
	OnFrameCaptured.Broadcast();
}

FString FPixelStreamingVideoInput::ToString()
{
	return TEXT("a Video Input");
}

uint32 FPixelStreamingVideoInput::GetUniqueStreamId()
{
	return NextStreamId++;
}
