// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInputBackBuffer.h"
#include "PixelStreamingPrivate.h"
#include "Settings.h"
#include "Utils.h"

#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"

#include "Slate/SlateViewportProvider.h"
#include "Framework/Application/SlateApplication.h"

TSharedPtr<FPixelStreamingVideoInputBackBuffer> FPixelStreamingVideoInputBackBuffer::Create()
{
	// this was added to fix packaging
	if (!FSlateApplication::IsInitialized())
	{
		return nullptr;
	}

	TSharedPtr<FPixelStreamingVideoInputBackBuffer> NewInput = TSharedPtr<FPixelStreamingVideoInputBackBuffer>(new FPixelStreamingVideoInputBackBuffer());
	TWeakPtr<FPixelStreamingVideoInputBackBuffer> WeakInput = NewInput;

	// Set up the callback on the game thread since FSlateApplication::Get() can only be used there
	UE::PixelStreaming::DoOnGameThread([WeakInput]() {
		if (TSharedPtr<FPixelStreamingVideoInputBackBuffer> Input = WeakInput.Pin())
		{
			FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
			Input->DelegateHandle = Renderer->OnBackBufferReadyToPresent().AddSP(Input.ToSharedRef(), &FPixelStreamingVideoInputBackBuffer::OnBackBufferReady);
		}
	});

	return NewInput;
}

FPixelStreamingVideoInputBackBuffer::~FPixelStreamingVideoInputBackBuffer()
{
	if (!IsEngineExitRequested())
	{
		UE::PixelStreaming::DoOnGameThread([HandleCopy = DelegateHandle]() {
			FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().Remove(HandleCopy);
		});
	}
}

void FPixelStreamingVideoInputBackBuffer::OnBackBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider)
{
	OnFrame(FPixelCaptureInputFrameRHI(ViewportProvider.GetBackBufferResource()));
}

FString FPixelStreamingVideoInputBackBuffer::ToString()
{
	return TEXT("the Back Buffer");
}