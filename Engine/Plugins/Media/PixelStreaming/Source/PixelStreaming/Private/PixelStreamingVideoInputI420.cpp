// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInputI420.h"
#include "PixelStreamingPrivate.h"

#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureCapturerI420.h"
#include "PixelCaptureCapturerI420ToRHI.h"

TSharedPtr<FPixelCaptureCapturer> FPixelStreamingVideoInputI420::CreateCapturer(int32 FinalFormat, FIntPoint Resolution)
{
	switch (FinalFormat)
	{
		case PixelCaptureBufferFormat::FORMAT_RHI:
		{
			return FPixelCaptureCapturerI420ToRHI::Create();
		}
		case PixelCaptureBufferFormat::FORMAT_I420:
		{
			return FPixelCaptureCapturerI420::Create();
		}
		default:
			UE_LOGF(LogPixelStreaming, Error, "Unsupported final format %d", FinalFormat);
			return nullptr;
	}
}
