// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInputMediaCapture.h"
#include "PixelStreamingSettings.h"
#include "PixelCaptureCapturerRHI.h"
#include "PixelCaptureCapturerRHIRDG.h"
#include "PixelCaptureCapturerRHINoCopy.h"
#include "PixelCaptureCapturerRHIToI420CPU.h"
#include "PixelCaptureCapturerRHIToI420Compute.h"
#include "PixelStreamingPrivate.h"
#include "Engine/GameEngine.h"

TSharedPtr<FPixelStreamingVideoInputMediaCapture> FPixelStreamingVideoInputMediaCapture::CreateActiveViewportCapture()
{
	TSharedPtr<FPixelStreamingVideoInputMediaCapture> NewInput = MakeShared<FPixelStreamingVideoInputMediaCapture>();
	NewInput->LateStartActiveViewportCapture();
	return NewInput;
}

TSharedPtr<FPixelStreamingVideoInputMediaCapture> FPixelStreamingVideoInputMediaCapture::Create(TObjectPtr<UPixelStreamingMediaIOCapture> MediaCapture)
{
	TSharedPtr<FPixelStreamingVideoInputMediaCapture> VideoInput = MakeShared<FPixelStreamingVideoInputMediaCapture>(MediaCapture);
	MediaCapture->SetVideoInput(VideoInput);
	return VideoInput;
}

FPixelStreamingVideoInputMediaCapture::FPixelStreamingVideoInputMediaCapture(TObjectPtr<UPixelStreamingMediaIOCapture> InMediaCapture)
	: MediaCapture(InMediaCapture)
{
}

FPixelStreamingVideoInputMediaCapture::FPixelStreamingVideoInputMediaCapture()
{
}

FPixelStreamingVideoInputMediaCapture::~FPixelStreamingVideoInputMediaCapture()
{
	// We don't need to remove mediacapture from root and clear delegate if engine is shutting down
	// as UE will already have killed all UObjects by this point.
	if (!IsEngineExitRequested() && MediaCapture)
	{
		MediaCapture->OnStateChangedNative.RemoveAll(this);
		MediaCapture->RemoveFromRoot();
	}
}

void FPixelStreamingVideoInputMediaCapture::StartActiveViewportCapture()
{
	// If we were bound to the OnFrameEnd delegate to ensure a frame was rendered before starting, then we can unset it here.
	if (OnFrameEndDelegateHandle.IsSet())
	{
		FCoreDelegates::OnEndFrame.Remove(OnFrameEndDelegateHandle.GetValue());
		OnFrameEndDelegateHandle.Reset();
	}

	if (MediaCapture)
	{
		MediaCapture->OnStateChangedNative.RemoveAll(this);
		MediaCapture->RemoveFromRoot();
	}

	MediaCapture = NewObject<UPixelStreamingMediaIOCapture>();
	MediaCapture->AddToRoot(); // prevent GC on this
	MediaCapture->SetMediaOutput(NewObject<UPixelStreamingMediaIOOutput>());
	MediaCapture->SetVideoInput(AsShared());
	MediaCapture->OnStateChangedNative.AddSP(this, &FPixelStreamingVideoInputMediaCapture::OnCaptureActiveViewportStateChanged);

	FMediaCaptureOptions Options;
	Options.bSkipFrameWhenRunningExpensiveTasks = false;
	Options.OverrunAction = EMediaCaptureOverrunAction::Skip;
	Options.ResizeMethod = EMediaCaptureResizeMethod::ResizeInCapturePass;

	// Start capturing the active viewport
	MediaCapture->CaptureActiveSceneViewport(Options);
}

void FPixelStreamingVideoInputMediaCapture::LateStartActiveViewportCapture()
{
	// Bind the OnEndFrame delegate to ensure we only start capture once a frame has been rendered
	OnFrameEndDelegateHandle = FCoreDelegates::OnEndFrame.AddSP(this, &FPixelStreamingVideoInputMediaCapture::StartActiveViewportCapture);
}

FString FPixelStreamingVideoInputMediaCapture::ToString()
{
	return TEXT("Video Input Media Capture");
}

TSharedPtr<FPixelCaptureCapturer> FPixelStreamingVideoInputMediaCapture::CreateCapturer(int32 FinalFormat, FIntPoint Resolution)
{
	switch (FinalFormat)
	{
		case PixelCaptureBufferFormat::FORMAT_RHI:
		{
			if (FPixelStreamingSettings::GetSimulcastParameters().Layers.Num() == 1
				&& FPixelStreamingSettings::GetSimulcastParameters().Layers[0].Scaling == 1.0)
			{
				// If we only have a single layer (and it's scale is 1), we can use the no copy capturer
				// as we know the output from the media capture will already be the correct format and scale
				return FPixelCaptureCapturerRHINoCopy::Create();
			}
			else
			{
				// "Safe Texture Copy" polls a fence to ensure a GPU copy is complete
				// the RDG pathway does not poll a fence so is more unsafe but offers
				// a significant performance increase
				if (FPixelStreamingSettings::GetCaptureUseFence())
				{
					return FPixelCaptureCapturerRHI::Create({ .OutputResolution = Resolution });
				}
				else
				{
					return FPixelCaptureCapturerRHIRDG::Create({ .OutputResolution = Resolution });
				}
			}
		}
		case PixelCaptureBufferFormat::FORMAT_I420:
		{
			if (FPixelStreamingSettings::GetVPXUseCompute())
			{
				return FPixelCaptureCapturerRHIToI420Compute::Create({ .OutputResolution = Resolution });
			}
			else
			{
				return FPixelCaptureCapturerRHIToI420CPU::Create({ .OutputResolution = Resolution });
			}
		}
		default:
			// UE_LOGF(LogPixelStreaming, Error, "Unsupported final format %d", FinalFormat);
			return nullptr;
	}
}

void FPixelStreamingVideoInputMediaCapture::OnCaptureActiveViewportStateChanged()
{
	if (!MediaCapture)
	{
		return;
	}

	switch (MediaCapture->GetState())
	{
		case EMediaCaptureState::Capturing:
			UE_LOGF(LogPixelStreaming, Log, "Starting media capture for Pixel Streaming.");
			break;
		case EMediaCaptureState::Stopped:
			if (MediaCapture->WasViewportResized())
			{
				UE_LOGF(LogPixelStreaming, Log, "Pixel Streaming capture was stopped due to resize, going to restart capture.");
				// If it was stopped and viewport resized we assume resize caused the stop, so try a restart of capture here.
				StartActiveViewportCapture();
			}
			else
			{
				UE_LOGF(LogPixelStreaming, Log, "Stopping media capture for Pixel Streaming.");
			}
			break;
		case EMediaCaptureState::Error:
			UE_LOGF(LogPixelStreaming, Log, "Pixel Streaming capture hit an error, capturing will stop.");
			break;
		default:
			break;
	}
}
