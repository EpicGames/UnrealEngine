// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInputNV12.h"
#include "PixelStreamingPrivate.h"

#include "PixelCaptureBufferFormat.h"
// #include "PixelCaptureCapturerNV12.h"
#include "PixelCaptureCapturerNV12ToRHI.h"

TSharedPtr<FPixelCaptureCapturer> FPixelStreamingVideoInputNV12::CreateCapturer(int32 FinalFormat, FIntPoint Resolution)
{
	switch (FinalFormat)
	{
		case PixelCaptureBufferFormat::FORMAT_RHI:
		{
			return FPixelCaptureCapturerNV12ToRHI::Create();
		}
		// TODO (aidan) when we dont want NV12 on the RHI
		// case PixelCaptureBufferFormat::FORMAT_NV12:
		// {
		// 	return MakeShared<FPixelCaptureCapturerNV12>();
		// }
		default:
			UE_LOGF(LogPixelStreaming, Error, "Unsupported final format %d", FinalFormat);
			return nullptr;
	}
}
