// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInputPIEViewport.h"
#include "PixelCaptureInputFrameRHI.h"
#include "Utils.h"
#include "UnrealClient.h"
#include "Engine/GameViewportClient.h"
#include "RenderingThread.h"

TSharedPtr<FPixelStreamingVideoInputPIEViewport> FPixelStreamingVideoInputPIEViewport::Create()
{
	TSharedPtr<FPixelStreamingVideoInputPIEViewport> NewInput = TSharedPtr<FPixelStreamingVideoInputPIEViewport>(new FPixelStreamingVideoInputPIEViewport());
	TWeakPtr<FPixelStreamingVideoInputPIEViewport> WeakInput = NewInput;

	UE::PixelStreaming::DoOnGameThread([WeakInput]() {
		if (TSharedPtr<FPixelStreamingVideoInputPIEViewport> Input = WeakInput.Pin())
		{
			Input->DelegateHandle = UGameViewportClient::OnViewportRendered().AddSP(Input.ToSharedRef(), &FPixelStreamingVideoInputPIEViewport::OnViewportRendered);
		}
	});
	
	return NewInput;
}

FPixelStreamingVideoInputPIEViewport::~FPixelStreamingVideoInputPIEViewport()
{
	if (!IsEngineExitRequested())
	{
		UGameViewportClient::OnViewportRendered().Remove(DelegateHandle);
	}
}

void FPixelStreamingVideoInputPIEViewport::OnViewportRendered(FViewport* InViewport)
{
	if (!InViewport->IsPlayInEditorViewport())
	{
		return;
	}

	const FTextureRHIRef& FrameBuffer = InViewport->GetRenderTargetTexture();
	if (!FrameBuffer)
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(StreamViewportTextureCommand)
	([&, FrameBuffer](FRHICommandList& RHICmdList) {
		OnFrame(FPixelCaptureInputFrameRHI(FrameBuffer));
	});
}

FString FPixelStreamingVideoInputPIEViewport::ToString()
{
	return TEXT("the PIE Viewport");
}
