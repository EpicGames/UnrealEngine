// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInputRHI.h"
#include "PixelStreamingPrivate.h"
#include "Settings.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureCapturerRHI.h"
#include "PixelCaptureCapturerRHIRDG.h"
#include "PixelCaptureCapturerRHIToI420CPU.h"
#include "PixelCaptureCapturerRHIToI420Compute.h"

TSharedPtr<FPixelCaptureCapturer> FPixelStreamingVideoInputRHI::CreateCapturer(int32 FinalFormat, FIntPoint Resolution)
{
	switch (FinalFormat)
	{
		case PixelCaptureBufferFormat::FORMAT_RHI:
		{
			// "Safe Texture Copy" polls a fence to ensure a GPU copy is complete
			// the RDG pathway does not poll a fence so is more unsafe but offers
			// a significant performance increase
			if (UE::PixelStreaming::Settings::CVarPixelStreamingCaptureUseFence.GetValueOnAnyThread())
			{
				return FPixelCaptureCapturerRHI::Create({ .OutputResolution = Resolution });
			}
			else
			{
				return FPixelCaptureCapturerRHIRDG::Create({ .OutputResolution = Resolution });
			}
		}
		case PixelCaptureBufferFormat::FORMAT_I420:
		{
			if (UE::PixelStreaming::Settings::CVarPixelStreamingVPXUseCompute.GetValueOnAnyThread())
			{
				return FPixelCaptureCapturerRHIToI420Compute::Create({ .OutputResolution = Resolution });
			}
			else
			{
				return FPixelCaptureCapturerRHIToI420CPU::Create({ .OutputResolution = Resolution });
			}
		}
		default:
			UE_LOGF(LogPixelStreaming, Error, "Unsupported final format %d", FinalFormat);
			return nullptr;
	}
}
