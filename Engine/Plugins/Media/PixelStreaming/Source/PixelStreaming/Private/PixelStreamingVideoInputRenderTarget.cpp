// Copyright Epic Games, Inc. All Rights Reserved.
#include "PixelStreamingVideoInputRenderTarget.h"
#include "PixelStreamingVideoInputRHI.h"
#include "TextureResource.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PixelCaptureInputFrameRHI.h"

TSharedPtr<FPixelStreamingVideoInputRenderTarget> FPixelStreamingVideoInputRenderTarget::Create(UTextureRenderTarget2D* Target)
{
	return TSharedPtr<FPixelStreamingVideoInputRenderTarget>(new FPixelStreamingVideoInputRenderTarget(Target));
}

FPixelStreamingVideoInputRenderTarget::~FPixelStreamingVideoInputRenderTarget()
{
	FCoreDelegates::OnEndFrameRT.Remove(DelegateHandle);
}

FPixelStreamingVideoInputRenderTarget::FPixelStreamingVideoInputRenderTarget(UTextureRenderTarget2D* InTarget)
	: Target(InTarget)
	, DelegateHandle(FCoreDelegates::OnEndFrameRT.AddRaw(this, &FPixelStreamingVideoInputRenderTarget::OnEndFrameRenderThread))
{
}

void FPixelStreamingVideoInputRenderTarget::OnEndFrameRenderThread()
{
	if (Target)
	{
		if (FTextureRHIRef Texture = Target->GetResource()->GetTexture2DRHI())
		{
			OnFrame(FPixelCaptureInputFrameRHI(Texture));
		}
	}
}

FString FPixelStreamingVideoInputRenderTarget::ToString()
{
	return TEXT("a Render Target");
}