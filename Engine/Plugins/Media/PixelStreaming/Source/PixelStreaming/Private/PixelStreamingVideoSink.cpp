// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoSink.h"
#include "Async/Async.h"
#include "FrameBufferRHI.h"
#include "FrameBufferI420.h"

void FPixelStreamingVideoSink::OnFrame(const webrtc::VideoFrame& Frame)
{
    static constexpr auto TEXTURE_PIXEL_FORMAT = PF_B8G8R8A8;
	static constexpr auto WEBRTC_PIXEL_FORMAT = webrtc::VideoType::kARGB;

	uint32 SizeX = Frame.width();
	uint32 SizeY = Frame.height();
	auto Format = TEXTURE_PIXEL_FORMAT;
	auto NumMips = 0;
	auto SamplerAddressMode = ESamplerAddressMode::AM_Clamp;

    if (Frame.video_frame_buffer()->type() == webrtc::VideoFrameBuffer::Type::kNative)
    {
        UE::PixelStreaming::FFrameBufferRHI* const FrameBuffer = static_cast<UE::PixelStreaming::FFrameBufferRHI*>(Frame.video_frame_buffer().get());

        if(FrameBuffer == nullptr)
        {
            return;
        }

        TSharedPtr<FVideoResourceRHI, ESPMode::ThreadSafe> VideoResource = FrameBuffer->GetVideoResource();
        if (VideoResource->GetFormat() != EVideoFormat::BGRA)
        {
            VideoResource = VideoResource->TransformResource(FVideoDescriptor(EVideoFormat::BGRA, SizeX, SizeY));
        }
		
        auto& Raw = StaticCastSharedPtr<FVideoResourceRHI>(VideoResource)->GetRaw();
        OnFrame(Raw.Texture);
    }
    else if (Frame.video_frame_buffer()->type() == webrtc::VideoFrameBuffer::Type::kI420)
	{
	    {
		    FScopeLock Lock(&RenderSyncContext);

		    const uint32_t Size = webrtc::CalcBufferSize(WEBRTC_PIXEL_FORMAT, SizeX, SizeY);
		    if (Size > static_cast<uint32_t>(Buffer.Num()))
		    {
    			Buffer.SetNum(Size);
		    }
		    webrtc::ConvertFromI420(Frame, WEBRTC_PIXEL_FORMAT, 0, Buffer.GetData());
	    }

		AsyncTask(ENamedThreads::ActualRenderingThread, [SizeX, SizeY, Format, this]() 
    	{
    	    FScopeLock Lock(&RenderSyncContext);

    	    const FIntPoint FrameSize = FIntPoint(SizeX, SizeY);
    	    FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

            FTextureRHIRef SourceTexture;

    	    if (!RenderTargetDescriptor.IsValid() || RenderTargetDescriptor.GetSize() != FIntVector(FrameSize.X, FrameSize.Y, 0))
    	    {
    	        // Create the RenderTarget descriptor
    	        RenderTargetDescriptor = FPooledRenderTargetDesc::Create2DDesc(FrameSize,
    	            Format,
    	            FClearValueBinding::None,
    	            TexCreate_None,
    	            TexCreate_RenderTargetable,
    	            false);

    	        // Update the shader resource for the 'SourceTexture'
    	        FRHITextureCreateDesc RenderTargetTextureDesc =
    	            FRHITextureCreateDesc::Create2D(TEXT(""), FrameSize.X, FrameSize.Y, Format)
    	                .SetClearValue(FClearValueBinding::None)
#if PLATFORM_MAC
    	                .SetFlags(ETextureCreateFlags::CPUReadback | ETextureCreateFlags::SRGB)
	                    .SetInitialState(ERHIAccess::CPURead);
#else
        	            .SetFlags(ETextureCreateFlags::ShaderResource | TexCreate_RenderTargetable | ETextureCreateFlags::SRGB)
    	                .SetInitialState(ERHIAccess::SRVMask);
#endif

            	SourceTexture = RHICmdList.CreateTexture(RenderTargetTextureDesc);

            	// Find a free target-able texture from the render pool
            	GRenderTargetPool.FindFreeElement(RHICmdList,
                	RenderTargetDescriptor,
                	RenderTarget,
                	TEXT("PIXELSTEAMINGPLAYER"));

				// Create the update region structure
        		const FUpdateTextureRegion2D Region(0, 0, 0, 0, FrameSize.X, FrameSize.Y);

        		// Set the Pixel data of the webrtc Frame to the SourceTexture
        		RHICmdList.UpdateTexture2D(SourceTexture, 0, Region, FrameSize.X * 4, Buffer.GetData());

            	OnFrame(SourceTexture);
        	}
    	});
	}
}
