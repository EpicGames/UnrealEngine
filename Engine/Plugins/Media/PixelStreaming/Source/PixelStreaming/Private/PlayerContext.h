// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingPlayerConfig.h"
#include "PixelStreamingPeerConnection.h"
#include "PixelStreamingDataChannel.h"

namespace UE::PixelStreaming
{
	struct FPlayerContext
	{
		FPixelStreamingPlayerConfig Config;
		TSharedPtr<FPixelStreamingPeerConnection> PeerConnection;
		TSharedPtr<FPixelStreamingDataChannel> DataChannel;
	};
}