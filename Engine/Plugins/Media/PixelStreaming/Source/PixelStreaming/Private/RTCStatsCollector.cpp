// Copyright Epic Games, Inc. All Rights Reserved.

#include "RTCStatsCollector.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingStatNames.h"
#include "Settings.h"
#include "ToStringExtensions.h"

namespace UE::PixelStreaming
{

	// ------------- FRTCStatsCollector-------------------

	FRTCStatsCollector::FRTCStatsCollector()
		: FRTCStatsCollector(INVALID_PLAYER_ID)
	{
	}

	FRTCStatsCollector::FRTCStatsCollector(FPixelStreamingPlayerId PlayerId)
		: AssociatedPlayerId(PlayerId)
		, LastCalculationCycles(FPlatformTime::Cycles64())
		, bIsEnabled(!Settings::CVarPixelStreamingWebRTCDisableStats.GetValueOnAnyThread())
	{

		// Add each of the sinks for each of the stat types we are interested in tracking
		RTCStatSinks.Add(MakeShared<FTrackStatsSink>());
		RTCStatSinks.Add(MakeShared<FRTPMediaStatsSink>(RTCStatTypes::InboundRTP, "video"));
		RTCStatSinks.Add(MakeShared<FRTPMediaStatsSink>(RTCStatTypes::OutboundRTP, "video"));
		RTCStatSinks.Add(MakeShared<FRTPMediaStatsSink>(RTCStatTypes::InboundRTP, "audio"));
		RTCStatSinks.Add(MakeShared<FRTPMediaStatsSink>(RTCStatTypes::OutboundRTP, "audio"));
		RTCStatSinks.Add(MakeShared<FVideoSourceStatsSink>());
		RTCStatSinks.Add(MakeShared<FDataChannelStatsSink>());
	}

	void FRTCStatsCollector::AddRef() const
	{
		FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	rtc::RefCountReleaseStatus FRTCStatsCollector::Release() const
	{
		if (FPlatformAtomics::InterlockedDecrement(&RefCount) == 0)
		{
			return rtc::RefCountReleaseStatus::kDroppedLastRef;
		}

		return rtc::RefCountReleaseStatus::kOtherRefsRemained;
	}

	void FRTCStatsCollector::OnStatsDelivered(const rtc::scoped_refptr<const webrtc::RTCStatsReport>& Report)
	{
		FStats* PSStats = FStats::Get();

		if (!bIsEnabled || !PSStats || !Report)
		{
			return;
		}

		uint64 CyclesNow = FPlatformTime::Cycles64();
		double SecondsDelta = FGenericPlatformTime::ToSeconds64(CyclesNow - LastCalculationCycles);

		for (const webrtc::RTCStats& Stats : *Report)
		{
			for(TSharedPtr<FStatsSink> Sink : RTCStatSinks)
			{
				if(Sink && Sink->Wants(Stats))
				{
					FString PeerId = Sink->DerivePeerId(Stats, AssociatedPlayerId);
					Sink->Process(Stats, PeerId);
					Sink->PostProcess(PeerId, SecondsDelta);
				}
			}

			// For debugging to see all stats in the log
			// UE_LOGF(LogPixelStreaming, Log, "------------%ls---%ls------------", *FString(Stats.id().c_str()), *FString(Stats.type()));
			// for (const webrtc::RTCStatsMemberInterface* StatMember : StatMembers)
			// {
			// 	UE_LOGF(LogPixelStreaming, Log, "%ls", *FString(StatMember->name()));
			// }
		}

		LastCalculationCycles = FPlatformTime::Cycles64();
	}

	/*
	* ---------------- FStatsSink --------------------
	*/

	void FRTCStatsCollector::FStatsSink::Process(const webrtc::RTCStats& InStats, FString PeerId)
	{
		FStats* PSStats = FStats::Get();

		if (!PSStats)
		{
			return;
		}

		std::vector<const webrtc::RTCStatsMemberInterface*> StatMembers = InStats.Members();

		for (const webrtc::RTCStatsMemberInterface* StatMember : StatMembers)
		{
			const FName StatName = FName(StatMember->name());

			FRTCTrackedStat* StatToEmit = Get(StatName);
			if(!StatToEmit)
			{
				continue;
			}

			if (ExtractValueAndSet(StatMember, StatToEmit))
			{
				PSStats->StorePeerStat(PeerId, SinkType, StatToEmit->GetLatestStat());
			}
		}
	}

	void FRTCStatsCollector::FStatsSink::PostProcess(FString PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();

		if (!PSStats)
		{
			return;
		}

		// Run all the stat calculators
		for (auto& Calculator : Calculators)
		{
			TOptional<FStatData> OptStatData = Calculator(*this, SecondsDelta);
			if (OptStatData.IsSet())
			{
				FStatData& StatData = *OptStatData;
				CalculatedStats.Add(StatData.StatName, StatData);
				PSStats->StorePeerStat(PeerId, SinkType, StatData);
			}
		}
	}

	bool FRTCStatsCollector::FStatsSink::ExtractValueAndSet(const webrtc::RTCStatsMemberInterface* ExtractFrom, FRTCTrackedStat* SetValueHere)
	{
		const bool bIsDefined = ExtractFrom->is_defined();
		if(!bIsDefined)
		{
			return false;
		}

		const bool bZeroInitially = SetValueHere->GetLatestStat().StatValue == 0.0;
		FString StatValueStr =  ToString(ExtractFrom->ValueToString());
		double StatValueDouble = FCString::Atod(*StatValueStr);
		SetValueHere->SetLatestValue(StatValueDouble);
		const bool bZeroStill = SetValueHere->GetLatestStat().StatValue == 0.0;
		return !(bZeroInitially && bZeroStill);
	}

void FRTCStatsCollector::FStatsSink::AddAliased(FName StatName, FName AliasedName, int NDecimalPlaces, uint8 DisplayFlags)
{
	FRTCTrackedStat Stat = FRTCTrackedStat(StatName, AliasedName, NDecimalPlaces, DisplayFlags);
	Stats.Add(StatName, Stat);
}

	/*
	* ---------------- FRTPMediaStatsSink --------------------
	*/

	FRTCStatsCollector::FRTPMediaStatsSink::FRTPMediaStatsSink(FName InSinkType, FString InMediaKind)
		: FStatsSink(InSinkType)
		, MediaKind(InMediaKind)
	{
		// Todo: These stats could be split up across more specific classes once we want start tracking audio stats.

		// These stats will be extracted from the stat reports and emitted straight to screen
		Add(PixelStreamingStatNames::FirCount, 0);
		Add(PixelStreamingStatNames::PliCount, 0);
		Add(PixelStreamingStatNames::NackCount, 0);
		Add(PixelStreamingStatNames::SliCount, 0);
		Add(PixelStreamingStatNames::RetransmittedBytesSent, 0);
		Add(PixelStreamingStatNames::TotalEncodeBytesTarget, 0);
		Add(PixelStreamingStatNames::KeyFramesEncoded, 0);
		Add(PixelStreamingStatNames::FrameWidth, 0);
		Add(PixelStreamingStatNames::FrameHeight, 0);
		Add(PixelStreamingStatNames::HugeFramesSent, 0);
		Add(PixelStreamingStatNames::AvgSendDelay, 0);

		// These are values used to calculate extra values (stores time deltas etc)
		AddNonRendered(PixelStreamingStatNames::TargetBitrate);
		AddNonRendered(PixelStreamingStatNames::FramesSent);
		AddNonRendered(PixelStreamingStatNames::FramesReceived);
		AddNonRendered(PixelStreamingStatNames::BytesSent);
		AddNonRendered(PixelStreamingStatNames::BytesReceived);
		AddNonRendered(PixelStreamingStatNames::QPSum);
		AddNonRendered(PixelStreamingStatNames::TotalEncodeTime);
		AddNonRendered(PixelStreamingStatNames::FramesEncoded);
		AddNonRendered(PixelStreamingStatNames::FramesDecoded);

		// Calculated stats below:

		// FrameSent Per Second
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* FramesSentStat = StatSource.Get(PixelStreamingStatNames::FramesSent);
			if (FramesSentStat && FramesSentStat->GetLatestStat().StatValue > 0)
			{
				const double FramesSentPerSecond = FramesSentStat->CalculateDelta(Period);
				FStatData FpsStat = FStatData(PixelStreamingStatNames::FramesSentPerSecond, FramesSentPerSecond, 0);
				FpsStat.DisplayFlags = FStatData::EDisplayFlags::TEXT | FStatData::EDisplayFlags::GRAPH;
				return FpsStat;
			}
			return {};
		});

		// FramesReceived Per Second
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* FramesReceivedStat = StatSource.Get(PixelStreamingStatNames::FramesReceived);
			if (FramesReceivedStat && FramesReceivedStat->GetLatestStat().StatValue > 0)
			{
				const double FramesReceivedPerSecond = FramesReceivedStat->CalculateDelta(Period);
				return FStatData(PixelStreamingStatNames::FramesReceivedPerSecond, FramesReceivedPerSecond, 0);
			}
			return {};
		});

		// Megabits sent Per Second
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* BytesSentStat = StatSource.Get(PixelStreamingStatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetLatestStat().StatValue > 0)
			{
				const double BytesSentPerSecond = BytesSentStat->CalculateDelta(Period);
				const double MegabitsPerSecond = BytesSentPerSecond / 1'000'000.0 * 8.0;
				return FStatData(PixelStreamingStatNames::BitrateMegabits, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Bits sent Per Second
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* BytesSentStat = StatSource.Get(PixelStreamingStatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetLatestStat().StatValue > 0)
			{
				const double BytesSentPerSecond = BytesSentStat->CalculateDelta(Period);
				const double BitsPerSecond = BytesSentPerSecond * 8.0;
				FStatData Stat = FStatData(PixelStreamingStatNames::Bitrate, BitsPerSecond, 0);
				Stat.DisplayFlags = FStatData::EDisplayFlags::HIDDEN; // We don't want to display bits per second (too many digits)
				return Stat;
			}
			return {};
		});

		// Target megabits sent Per Second
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* TargetBpsStats = StatSource.Get(PixelStreamingStatNames::TargetBitrate);
			if (TargetBpsStats && TargetBpsStats->GetLatestStat().StatValue > 0)
			{
				const double TargetBps = TargetBpsStats->Average();
				const double MegabitsPerSecond = TargetBps / 1'000'000.0;
				return FStatData(PixelStreamingStatNames::TargetBitrateMegabits, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Megabits received Per Second
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* BytesReceivedStat = StatSource.Get(PixelStreamingStatNames::BytesReceived);
			if (BytesReceivedStat && BytesReceivedStat->GetLatestStat().StatValue > 0)
			{
				const double BytesReceivedPerSecond = BytesReceivedStat->CalculateDelta(Period);
				const double MegabitsPerSecond = BytesReceivedPerSecond / 1000.0 * 8.0;
				return FStatData(PixelStreamingStatNames::Bitrate, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Encoded fps
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* EncodedFramesStat = StatSource.Get(PixelStreamingStatNames::FramesEncoded);
			if (EncodedFramesStat && EncodedFramesStat->GetLatestStat().StatValue > 0)
			{
				const double EncodedFramesPerSecond = EncodedFramesStat->CalculateDelta(Period);
				return FStatData(PixelStreamingStatNames::EncodedFramesPerSecond, EncodedFramesPerSecond, 0);
			}
			return {};
		});

		// Decoded fps
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* DecodedFramesStat = StatSource.Get(PixelStreamingStatNames::FramesDecoded);
			if (DecodedFramesStat && DecodedFramesStat->GetLatestStat().StatValue > 0)
			{
				const double DecodedFramesPerSecond = DecodedFramesStat->CalculateDelta(Period);
				return FStatData(PixelStreamingStatNames::DecodedFramesPerSecond, DecodedFramesPerSecond, 0);
			}
			return {};
		});

		// Avg QP Per Second
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* QPSumStat = StatSource.Get(PixelStreamingStatNames::QPSum);
			FStatData* EncodedFramesPerSecond = StatSource.GetCalculatedStat(PixelStreamingStatNames::EncodedFramesPerSecond);
			if (QPSumStat && QPSumStat->GetLatestStat().StatValue > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->StatValue > 0.0)
			{
				const double QPSumDeltaPerSecond = QPSumStat->CalculateDelta(Period);
				const double MeanQPPerFrame = QPSumDeltaPerSecond / EncodedFramesPerSecond->StatValue;
				FName StatName = PixelStreamingStatNames::MeanQPPerSecond;
				return FStatData(StatName, MeanQPPerFrame, 0);
			}
			return {};
		});

		// Mean EncodeTime (ms) Per Frame
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* TotalEncodeTimeStat = StatSource.Get(PixelStreamingStatNames::TotalEncodeTime);
			FStatData* EncodedFramesPerSecond = StatSource.GetCalculatedStat(PixelStreamingStatNames::EncodedFramesPerSecond);
			if (TotalEncodeTimeStat && TotalEncodeTimeStat->GetLatestStat().StatValue > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->StatValue > 0.0)
			{
				const double TotalEncodeTimePerSecond = TotalEncodeTimeStat->CalculateDelta(Period);
				const double MeanEncodeTimePerFrameMs = TotalEncodeTimePerSecond / EncodedFramesPerSecond->StatValue * 1000.0;
				return FStatData(PixelStreamingStatNames::MeanEncodeTime, MeanEncodeTimePerFrameMs, 2);
			}
			return {};
		});

		// Mean SendDelay (ms) Per Frame
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* TotalSendDelayStat = StatSource.Get(PixelStreamingStatNames::TotalPacketSendDelay);
			FStatData* FramesSentPerSecond = StatSource.GetCalculatedStat(PixelStreamingStatNames::FramesSentPerSecond);
			if (TotalSendDelayStat && TotalSendDelayStat->GetLatestStat().StatValue > 0
				&& FramesSentPerSecond && FramesSentPerSecond->StatValue > 0.0)
			{
				const double TotalSendDelayPerSecond = TotalSendDelayStat->CalculateDelta(Period);
				const double MeanSendDelayPerFrameMs = TotalSendDelayPerSecond / FramesSentPerSecond->StatValue * 1000.0;
				return FStatData(PixelStreamingStatNames::MeanSendDelay, MeanSendDelayPerFrameMs, 2);
			}
			return {};
		});

		// JitterBufferDelay (ms)
		AddStatCalculator([](FStatsSink& StatSource, double Period) -> TOptional<FStatData> {
			FRTCTrackedStat* JitterBufferDelayStat = StatSource.Get(PixelStreamingStatNames::JitterBufferDelay);
			FStatData* FramesReceivedPerSecond = StatSource.GetCalculatedStat(PixelStreamingStatNames::FramesReceivedPerSecond);
			if (JitterBufferDelayStat && JitterBufferDelayStat->GetLatestStat().StatValue > 0
				&& FramesReceivedPerSecond && FramesReceivedPerSecond->StatValue > 0.0)
			{
				const double TotalJitterBufferDelayPerSecond = JitterBufferDelayStat->CalculateDelta(Period);
				const double MeanJitterBufferDelayMs = TotalJitterBufferDelayPerSecond / FramesReceivedPerSecond->StatValue * 1000.0;
				return FStatData(PixelStreamingStatNames::JitterBufferDelay, MeanJitterBufferDelayMs, 2);
			}
			return {};
		});
	}

bool FRTCStatsCollector::FRTPMediaStatsSink::Wants(const webrtc::RTCStats& InStats) const
	{
		bool bWants = FStatsSink::Wants(InStats);

		if(!bWants)
		{
			return false;
		}
		// Check the `kind` field to see if it matches what we want
		std::vector<const webrtc::RTCStatsMemberInterface*> StatMembers = InStats.Members();
		for (const webrtc::RTCStatsMemberInterface* StatMember : StatMembers)
		{
			const FString StatName = FString(StatMember->name());
			if (StatName == "kind")
			{
				FString StatMediaKind = StatMember->is_defined() ? ToString(StatMember->ValueToString()) : TEXT("");
				return MediaKind == StatMediaKind;
			}
		}
		return false;
	}

	FString FRTCStatsCollector::FRTPMediaStatsSink::DerivePeerId(const webrtc::RTCStats& InStats, FPixelStreamingPlayerId PlayerId) const
	{
		std::vector<const webrtc::RTCStatsMemberInterface*> StatMembers = InStats.Members();
		const FString StatsType = FString(InStats.type());

		if (PlayerId == SFU_PLAYER_ID && (StatsType == RTCStatTypes::OutboundRTP || StatsType == RTCStatTypes::InboundRTP))
		{
			// Extract the `ssrc` to uniquely id the stream
			for (const webrtc::RTCStatsMemberInterface* StatMember : StatMembers)
			{
				const FString StatName = FString(StatMember->name());
				if (StatName == "ssrc")
				{
					FString Ssrc = StatMember->is_defined() ? ToString(StatMember->ValueToString()) : TEXT("");
					return Ssrc;
				}
			}
		}

		return PlayerId;
	}

	/*
	* ---------------- FTrackStatsSink ----------------
	*/
	FRTCStatsCollector::FTrackStatsSink::FTrackStatsSink() : FStatsSink(RTCStatTypes::Track)
	{
		// Basic stats we wish to store and emit
		Add(PixelStreamingStatNames::JitterBufferDelay, 2);
		Add(PixelStreamingStatNames::FramesPerSecond, 0);
		Add(PixelStreamingStatNames::FramesDecoded, 0);
		Add(PixelStreamingStatNames::FramesDropped, 0);
		Add(PixelStreamingStatNames::FramesCorrupted, 0);
		Add(PixelStreamingStatNames::PartialFramesLost, 0);
		Add(PixelStreamingStatNames::FullFramesLost, 0);
		Add(PixelStreamingStatNames::JitterBufferTargetDelay, 2);
		Add(PixelStreamingStatNames::InterruptionCount, 0);
		Add(PixelStreamingStatNames::TotalInterruptionDuration, 2);
		Add(PixelStreamingStatNames::FreezeCount, 0);
		Add(PixelStreamingStatNames::PauseCount, 0);
		Add(PixelStreamingStatNames::TotalFreezesDuration, 2);
		Add(PixelStreamingStatNames::TotalPausesDuration, 2);
	}

	/*
	* ---------------- FVideoSourceStatsSink ----------------
	*/
	FRTCStatsCollector::FVideoSourceStatsSink::FVideoSourceStatsSink() : FStatsSink(RTCStatTypes::MediaSource)
	{
		// Track video source fps
		Add(PixelStreamingStatNames::SourceFps, 0);
	}

	bool FRTCStatsCollector::FVideoSourceStatsSink::Wants(const webrtc::RTCStats & InStats) const
	{
		const FString StatsType = FString(InStats.type());
		if(StatsType != RTCStatTypes::MediaSource)
		{
			return false;
		}
		for (const webrtc::RTCStatsMemberInterface* StatMember : InStats.Members())
		{
			const FString StatName = FString(StatMember->name());
			if (StatName == "kind")
			{
				FString StatMediaKind = StatMember->is_defined() ? ToString(StatMember->ValueToString()) : TEXT("");
				return StatMediaKind == "video";
			}
		}
		return false;
	}

	/*
	* ---------- FRTCTrackedStat -------------------
	*/
	FRTCStatsCollector::FRTCTrackedStat::FRTCTrackedStat(FName StatName, FName Alias, int NDecimalPlaces, uint8 DisplayFlags) 
		: LatestStat(StatName, 0.0, NDecimalPlaces)
	{
		LatestStat.DisplayFlags = DisplayFlags;
		LatestStat.Alias = Alias;
	}

	double FRTCStatsCollector::FRTCTrackedStat::CalculateDelta(double Period) const
	{
		return (LatestStat.StatValue - PrevValue) * Period;
	}

	double FRTCStatsCollector::FRTCTrackedStat::Average() const
	{
		return (LatestStat.StatValue + PrevValue) * 0.5;
	}

	void FRTCStatsCollector::FRTCTrackedStat::SetLatestValue(double InValue)
	{
		PrevValue = LatestStat.StatValue;
		LatestStat.StatValue = InValue;
	}

	/*
	* ----------- FDataChannelStatsSink -----------
	*/
	FRTCStatsCollector::FDataChannelStatsSink::FDataChannelStatsSink() : FStatsSink("data-channel")
	{
		// These names are added as aliased names because `bytesSent` is ambiguous stat that is used across inbound-rtp, outbound-rtp, and data-channel
		// so to disambiguate which state we are referring to we record the `bytesSent` stat for the data-channel but store and report it as `data-channel-bytesSent`
		AddAliased(PixelStreamingStatNames::MessagesSent, PixelStreamingStatNames::DataChannelMessagesSent, 0, FStatData::EDisplayFlags::TEXT | FStatData::EDisplayFlags::GRAPH);
		AddAliased(PixelStreamingStatNames::MessagesReceived, PixelStreamingStatNames::DataChannelBytesReceived, 0, FStatData::EDisplayFlags::TEXT | FStatData::EDisplayFlags::GRAPH);
		AddAliased(PixelStreamingStatNames::BytesSent, PixelStreamingStatNames::DataChannelBytesSent, 0, FStatData::EDisplayFlags::TEXT | FStatData::EDisplayFlags::GRAPH);
		AddAliased(PixelStreamingStatNames::BytesReceived, PixelStreamingStatNames::DataChannelMessagesReceived, 0, FStatData::EDisplayFlags::TEXT | FStatData::EDisplayFlags::GRAPH);
	}

} // namespace UE::PixelStreaming
