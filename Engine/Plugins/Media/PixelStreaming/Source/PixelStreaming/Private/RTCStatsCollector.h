// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebRTCIncludes.h"
#include "PixelStreamingPlayerId.h"
#include "Stats.h"
#include "Templates/SharedPointer.h"

namespace UE::PixelStreaming
{
	namespace RTCStatTypes
	{
		const FName DataChannel = FName(TEXT("data-channel"));
		const FName OutboundRTP = FName(TEXT("outbound-rtp"));
		const FName InboundRTP = FName(TEXT("inbound-rtp"));
		const FName Track = FName(TEXT("track"));
		const FName MediaSource = FName(TEXT("media-source"));
	}

	class PIXELSTREAMING_API FRTCStatsCollector : public webrtc::RTCStatsCollectorCallback
	{
	public:
		FRTCStatsCollector();
		FRTCStatsCollector(FPixelStreamingPlayerId PlayerId);

		// Begin RTCStatsCollectorCallback interface
		void OnStatsDelivered(const rtc::scoped_refptr<const webrtc::RTCStatsReport>& report) override;
		void AddRef() const override;
		rtc::RefCountReleaseStatus Release() const override;
		// End RTCStatsCollectorCallback interface

	private:
		/*
		* ---------- FRTCTrackedStat -------------------
		* Tracks a stat from WebRTC. Stores the current value and previous value.
		*/
		class FRTCTrackedStat
		{
		public:
			FRTCTrackedStat(FName StatName, int NDecimalPlaces) : LatestStat(StatName, 0.0, NDecimalPlaces){}
			FRTCTrackedStat(FName StatName, int NDecimalPlaces, uint8 DisplayFlags) : LatestStat(StatName, 0.0, NDecimalPlaces){ LatestStat.DisplayFlags = DisplayFlags; }
			FRTCTrackedStat(FName StatName, FName Alias, int NDecimalPlaces, uint8 DisplayFlags);
			double CalculateDelta(double Period) const;
			double Average() const;
			const FStatData& GetLatestStat() const { return LatestStat; }
			void SetLatestValue(double InValue);
		private:
			FStatData LatestStat;
			double PrevValue = 0.0;
		};

		/*
		* ---------- FStatsSink -------------------
		* A sink for processing webrtc::RTCStats - processing will include extracting stats and storing them.  
		*/
		class FStatsSink
		{
		public:
			FStatsSink(FName InSinkType) : SinkType(InSinkType) {}
			virtual ~FStatsSink() = default;
			virtual bool Wants(const webrtc::RTCStats& InStats) const { return FString(InStats.type()) == SinkType.ToString(); };
			virtual void Process(const webrtc::RTCStats& InStats, FString PeerId);
			virtual void PostProcess(FString PeerId, double SecondsDelta);
			virtual FString DerivePeerId(const webrtc::RTCStats& InStats, FPixelStreamingPlayerId PlayerId) const { return PlayerId; }
			// returns true if the value is worth storing (false if it started and remains zero)
			virtual bool ExtractValueAndSet(const webrtc::RTCStatsMemberInterface* ExtractFrom, FRTCTrackedStat* SetValueHere);
			virtual bool Contains(const FName StatName) const { return Stats.Contains(StatName); };
			virtual bool ContainsCalculatedStat(const FName& StatName) const { return CalculatedStats.Contains(StatName); }
			virtual void Add(FName StatName, int NDecimalPlaces){ Stats.Add(StatName, FRTCTrackedStat(StatName, NDecimalPlaces, FStatData::EDisplayFlags::TEXT)); }
			virtual void AddAliased(FName StatName, FName AliasedName, int NDecimalPlaces, uint8 DisplayFlags);
			virtual void AddNonRendered(FName StatName) { Stats.Add(StatName, FRTCTrackedStat(StatName, 2, FStatData::EDisplayFlags::HIDDEN)); }
			virtual void AddStatCalculator(const TFunction<TOptional<FStatData>(FStatsSink&, double)>& Calculator) { Calculators.Add(Calculator); }
			virtual FRTCTrackedStat* Get(const FName& StatName) { return Stats.Find(StatName); };
			virtual FStatData* GetCalculatedStat(const FName& StatName) { return CalculatedStats.Find(StatName); }

			// Stats that are stored as is.
			TMap<FName, FRTCTrackedStat> Stats;

			// Stats we calculate based on the stats map above. This calculation is done in FStatsSink::PostProcess by the `Calculators` below.
			TMap<FName, FStatData> CalculatedStats;
			TArray<TFunction<TOptional<FStatData>(FStatsSink&, double)>> Calculators;
		protected:
			FName SinkType;
		};

		/*
		* ---------- FRTPMediaStatsSink -------------------
		* Sink useful for RTP media (audio or video) that is inbound or outbound.
		*/
		class FRTPMediaStatsSink : public FStatsSink
		{
		public:
			FRTPMediaStatsSink(FName InSinkType, FString InMediaKind);
			virtual ~FRTPMediaStatsSink() = default;
			virtual bool Wants(const webrtc::RTCStats& InStats) const override;
			virtual FString DerivePeerId(const webrtc::RTCStats& InStats, FPixelStreamingPlayerId PlayerId) const override;
		private:
			FString MediaKind;
		};

		/*
		* ---------- FTrackStatsSink -------------------
		* Stats sink for `track` stats
		*/
		class FTrackStatsSink : public FStatsSink
		{
		public:
			FTrackStatsSink();
			virtual ~FTrackStatsSink() = default;
		};

		/*
		* ---------- FVideoSourceStatsSink -------------------
		* Stats sink for video `media-source` stats
		*/
		class FVideoSourceStatsSink : public FStatsSink
		{
		public:
			FVideoSourceStatsSink();
			virtual ~FVideoSourceStatsSink() = default;
			virtual bool Wants(const webrtc::RTCStats& InStats) const override;
		};

		/*
		* ---------- FDataChannelStatsSink -------------------
		* Stats sink `data-channel` stats
		*/
		class FDataChannelStatsSink : public FStatsSink
		{
		public:
			FDataChannelStatsSink();
			virtual ~FDataChannelStatsSink() = default;
		};

	private:
		FPixelStreamingPlayerId AssociatedPlayerId;

		mutable int32 RefCount;

		uint64 LastCalculationCycles;

		bool bIsEnabled;

		// All sinks are given all webrtc::RTCStats contained in a report and they can choose which stats they wish to extract/process.
		TArray<TSharedPtr<FStatsSink>> RTCStatSinks;
	};
} // namespace UE::PixelStreaming
