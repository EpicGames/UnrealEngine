// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "PixelStreamingPrivate.h"
#include "Misc/DefaultValueHelper.h"
#include "Async/Async.h"
#include "IPixelStreamingModule.h"
#include "PixelStreamingDelegates.h"
#include "Utils.h"

namespace UE::PixelStreaming::Settings
{
	// Begin Encoder CVars
	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderTargetBitrate(
		TEXT("PixelStreaming.Encoder.TargetBitrate"),
		-1,
		TEXT("Target bitrate (bps). Ignore the bitrate WebRTC wants (not recommended). Set to -1 to disable. Default -1."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMaxBitrate(
		TEXT("PixelStreaming.Encoder.MaxBitrateVBR"),
		20000000,
		TEXT("Max bitrate (bps). Does not work in CBR rate control mode with NVENC."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarPixelStreamingDebugDumpFrame(
		TEXT("PixelStreaming.Encoder.DumpDebugFrames"),
		false,
		TEXT("Dumps frames from the encoder to a file on disk for debugging purposes."),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMinQP(
		TEXT("PixelStreaming.Encoder.MinQP"),
		0,
		TEXT("0-51, lower values result in better quality but higher bitrate. Default 0 - i.e. no limit on a minimum QP."),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMaxQP(
		TEXT("PixelStreaming.Encoder.MaxQP"),
		51,
		TEXT("0-51, lower values result in better quality but higher bitrate. Default 51 - i.e. no limit on a maximum QP."),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingEncoderRateControl(
		TEXT("PixelStreaming.Encoder.RateControl"),
		TEXT("CBR"),
		TEXT("PixelStreaming video encoder RateControl mode. Supported modes are `ConstQP`, `VBR`, `CBR`. Default: CBR, which we recommend."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingEnableFillerData(
		TEXT("PixelStreaming.Encoder.EnableFillerData"),
		false,
		TEXT("Maintains constant bitrate by filling with junk data. Note: Should not be required with CBR and MinQP = -1. Default: false."),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingEncoderMultipass(
		TEXT("PixelStreaming.Encoder.Multipass"),
		TEXT("FULL"),
		TEXT("PixelStreaming encoder multipass. Supported modes are `DISABLED`, `QUARTER`, `FULL`"),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingH264Profile(
		TEXT("PixelStreaming.Encoder.H264Profile"),
		TEXT("BASELINE"),
		TEXT("PixelStreaming encoder profile. Supported modes are `AUTO`, `BASELINE`, `MAIN`, `HIGH`, `HIGH444`, `STEREO`, `SVC_TEMPORAL_SCALABILITY`, `PROGRESSIVE_HIGH`, `CONSTRAINED_HIGH`"),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingEncoderPreset(
		TEXT("PixelStreaming.Encoder.Preset"),
		TEXT("ULTRA_LOW_LATENCY"),
		TEXT("PixelStreaming encoder presets. Supported modes are `ULTRA_LOW_LATENCY`, `LOW_QUALITY`, `DEFAULT`, `HIGH_QUALITY`, `LOSSLESS`"),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderKeyframeInterval(
		TEXT("PixelStreaming.Encoder.KeyframeInterval"),
		300,
		TEXT("How many frames before a key frame is sent. Default: 300. Values <=0 will disable sending of periodic key frames. Note: NVENC does not support changing this after encoding has started."),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderIntraRefreshPeriodFrames(
		TEXT("PixelStreaming.Encoder.IntraRefreshPeriodFrames"),
		0,
		TEXT("The total number of frames between each intra refresh. Smallers values will cause intra refresh more often. Default: 0. Values <= 0 will disable intra refresh."),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderIntraRefreshCountFrames(
		TEXT("PixelStreaming.Encoder.IntraRefreshCountFrames"),
		0,
		TEXT("The total number of frames within the intra refresh period that should be used as 'intra refresh' frames. Smaller values make stream recovery quicker at the cost of more bandwidth usage. Default: 0."),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingEncoderCodec(
		TEXT("PixelStreaming.Encoder.Codec"),
		TEXT("H264"),
		TEXT("PixelStreaming encoder codec. Supported values are `H264`, `AV1`, `VP8`, `VP9`"),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMaxSessions(
		TEXT("PixelStreaming.Encoder.MaxSessions"),
		-1,
		TEXT("Maximum number of concurrent hardware encoder sessions for Pixel Streaming."),
		ECVF_Default);
	// End Encoder CVars

	// Begin WebRTC CVars
	TAutoConsoleVariable<FString> CVarPixelStreamingDegradationPreference(
		TEXT("PixelStreaming.WebRTC.DegradationPreference"),
		TEXT("MAINTAIN_FRAMERATE"),
		TEXT("PixelStreaming degradation preference. Supported modes are `BALANCED`, `MAINTAIN_FRAMERATE`, `MAINTAIN_RESOLUTION`"),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCFps(
		TEXT("PixelStreaming.WebRTC.Fps"),
		60,
		TEXT("Framerate for WebRTC encoding. Default: 60"),
		ECVF_Default);

	// Note: 1 megabit is the maximum allowed in WebRTC for a start bitrate.
	TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCStartBitrate(
		TEXT("PixelStreaming.WebRTC.StartBitrate"),
		1000000,
		TEXT("Start bitrate (bps) that WebRTC will try begin the stream with. Must be between Min/Max bitrates. Default: 1000000"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCMinBitrate(
		TEXT("PixelStreaming.WebRTC.MinBitrate"),
		100000,
		TEXT("Min bitrate (bps) that WebRTC will not request below. Careful not to set too high otherwise WebRTC will just drop frames. Default: 100000"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCMaxBitrate(
		TEXT("PixelStreaming.WebRTC.MaxBitrate"),
		100000000,
		TEXT("Max bitrate (bps) that WebRTC will not request above. Default: 100000000 aka 100 megabits/per second."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int> CVarPixelStreamingWebRTCLowQpThreshold(
		TEXT("PixelStreaming.WebRTC.LowQpThreshold"),
		25,
		TEXT("Only useful when MinQP=-1. Value between 1-51 (default: 25). If WebRTC is getting frames below this QP it will try to increase resolution when not in MAINTAIN_RESOLUTION mode."),
		ECVF_Default);

	TAutoConsoleVariable<int> CVarPixelStreamingWebRTCHighQpThreshold(
		TEXT("PixelStreaming.WebRTC.HighQpThreshold"),
		37,
		TEXT("Only useful when MinQP=-1. Value between 1-51 (default: 37). If WebRTC is getting frames above this QP it will decrease resolution when not in MAINTAIN_RESOLUTION mode."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableReceiveAudio(
		TEXT("PixelStreaming.WebRTC.DisableReceiveAudio"),
		false,
		TEXT("Disables receiving audio from the browser into UE."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableTransmitAudio(
		TEXT("PixelStreaming.WebRTC.DisableTransmitAudio"),
		false,
		TEXT("Disables transmission of UE audio to the browser."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableTransmitVideo(
		TEXT("PixelStreaming.WebRTC.DisableTransmitVideo"),
		false,
		TEXT("Disables transmission of UE video to the browser."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableAudioSync(
		TEXT("PixelStreaming.WebRTC.DisableAudioSync"),
		true,
		TEXT("Disables the synchronization of audio and video tracks in WebRTC. This can be useful in low latency usecases where synchronization is not required."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCUseLegacyAudioDevice(
		TEXT("PixelStreaming.WebRTC.UseLegacyAudioDevice"),
		false,
		TEXT("Whether put audio and video in the same stream (which will make WebRTC try to sync them)."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCEnableFlexFec(
		TEXT("PixelStreaming.WebRTC.EnableFlexFec"),
		false,
		TEXT("Signals support for Flexible Forward Error Correction to WebRTC."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableStats(
		TEXT("PixelStreaming.WebRTC.DisableStats"),
		false,
		TEXT("Disables the collection of WebRTC stats."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCNegotiateCodecs(
		TEXT("PixelStreaming.WebRTC.NegotiateCodecs"),
		false,
		TEXT("Whether PS should send all its codecs during sdp handshake so peers can negotiate or just send a single selected codec."),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingWebRTCAudioGain(
		TEXT("PixelStreaming.WebRTC.AudioGain"),
		1.0f,
		TEXT("Sets the amount of gain to apply to audio. Default: 1.0"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableFrameDropper(
		TEXT("PixelStreaming.WebRTC.DisableFrameDropper"),
		false,
		TEXT("Disables the WebRTC internal frame dropper using the field trial WebRTC-FrameDropper/Disabled/"),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingWebRTCVideoPacingMaxDelay(
		TEXT("PixelStreaming.WebRTC.VideoPacing.MaxDelay"),
		-1.0f,
		TEXT("Enables the WebRTC-Video-Pacing field trial and sets the max delay (ms) parameter. Default: -1.0f (values below zero are discarded.)"),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingWebRTCVideoPacingFactor(
		TEXT("PixelStreaming.WebRTC.VideoPacing.Factor"),
		-1.0f,
		TEXT("Enables the WebRTC-Video-Pacing field trial and sets the video pacing factor parameter. Larger values are more lenient on larger bitrates. Default: -1.0f (values below zero are discarded.)"),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingWebRTCFieldTrials(
		TEXT("PixelStreaming.WebRTC.FieldTrials"),
		TEXT(""),
		TEXT("Sets the WebRTC field trials string. Format:\"TRIAL1/VALUE1/TRIAL2/VALUE2/\""),
		ECVF_Default);

	TAutoConsoleVariable<int> CVarPixelStreamingWebRTCMinPort(
		TEXT("PixelStreaming.WebRTC.MinPort"),
		49152, // Default according to RFC5766
		TEXT("Sets the minimum usable port for the WebRTC port allocator. Default: 49152"),
		ECVF_Default);

	TAutoConsoleVariable<int> CVarPixelStreamingWebRTCMaxPort(
		TEXT("PixelStreaming.WebRTC.MaxPort"),
		65535, // Default according to RFC5766
		TEXT("Sets the maximum usable port for the WebRTC port allocator. Default: 65535"),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingWebRTCPortAllocatorFlags(
		TEXT("PixelStreaming.WebRTC.PortAllocator.Flags"),
		TEXT(""),
		TEXT("Sets the WebRTC port allocator flags. Format:\"DISABLE_UDP,DISABLE_STUN,...\""),
		ECVF_Default);

	// End WebRTC CVars

	// Begin Pixel Streaming Plugin CVars

	TAutoConsoleVariable<bool> CVarPixelStreamingOnScreenStats(
		TEXT("PixelStreaming.HUDStats"),
		false,
		TEXT("Whether to show PixelStreaming stats on the in-game HUD (default: true)."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingLogStats(
		TEXT("PixelStreaming.LogStats"),
		false,
		TEXT("Whether to show PixelStreaming stats in the log (default: false)."),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarPixelStreamingFreezeFrameQuality(
		TEXT("PixelStreaming.FreezeFrameQuality"),
		100,
		TEXT("Compression quality of the freeze frame"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarSendPlayerIdAsInteger(
		TEXT("PixelStreaming.SendPlayerIdAsInteger"),
		false,
		TEXT("If true transmit the player id as an integer (for backward compatibility) or as a string."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingDisableLatencyTester(
		TEXT("PixelStreaming.DisableLatencyTester"),
		false,
		TEXT("If true disables latency tester being triggerable."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingVPXUseCompute(
		TEXT("PixelStreaming.VPXUseCompute"),
		false,
		TEXT("If true a compute shader will be used to process RGB -> I420. Otherwise libyuv will be used on the CPU."),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingInputController(
		TEXT("PixelStreaming.InputController"),
		TEXT("Any"),
		TEXT("Various modes of input control supported by Pixel Streaming, currently: \"Any\"  or \"Host\". Default: Any"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingSuppressICECandidateErrors(
		TEXT("PixelStreaming.SuppressICECandidateErrors"),
		false,
		TEXT("Silences ice candidate errors from the peer connection. This is used for testing and silencing a specific error that is expected but the number of errors is non-deterministic."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingExperimentalAudioInput(
		TEXT("PixelStreaming.ExperimentalAudioInput"),
		false,
		TEXT("Enables experimental mixing of arbitrary audio inputs which can fed in using the Pixel Streaming C++ API."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingCaptureUseFence(
		TEXT("PixelStreaming.CaptureUseFence"),
		false,
		TEXT("Whether the texture copy we do during image capture should use a fence or not (non-fenced is faster but unsafer)."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingDecoupleFramerate(
		TEXT("PixelStreaming.DecoupleFramerate"),
		false,
		TEXT("Whether we should only stream as fast as we render or at some fixed interval. Coupled means only stream what we render."),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingDecoupleWaitFactor(
		TEXT("PixelStreaming.DecoupleWaitFactor"),
		1.25f,
		TEXT("Frame rate factor to wait for a captured frame when streaming in decoupled mode. Higher factor waits longer but may also result in higher latency."),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingSignalingReconnectInterval(
		TEXT("PixelStreaming.SignalingReconnectInterval"),
		2.0f,
		TEXT("Changes the number of seconds between attempted reconnects to the signaling server. This is useful for reducing the log spam produced from attempted reconnects. A value <= 0 results in an immediate reconnect. Default: 2.0s"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingUseMediaCapture(
		TEXT("PixelStreaming.UseMediaCapture"),
		false,
		TEXT("Use Media Capture from MediaIOFramework to capture frames rather than Pixel Streamings internal backbuffer sources."),
		ECVF_Default);
	
	TAutoConsoleVariable<int32> CVarPixelStreamingQualityControllerMode(
		TEXT("PixelStreaming.QualityControllerMode"),
		EQualityControllerMode::GrantAllRequested,
		TEXT("Affect data channel requests for quality controller.\n"
			"0: allow all requests.\n"
			"1: only allow the first request. Subsequent requests are ignored until the player with quality control disconnects."
			)
		);

	FString DefaultStreamerID = TEXT("DefaultStreamer");
	FString DefaultSignallingURL = TEXT("ws://127.0.0.1:8888");

	void OnHudStatsToggled(IConsoleVariable* Var)
	{
		bool bHudStatsEnabled = Var->GetBool();

		if (!GEngine)
		{
			return;
		}

		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (WorldContext.WorldType == EWorldType::Game || WorldContext.WorldType == EWorldType::PIE)
			{
				UWorld* World = WorldContext.World();
				UGameViewportClient* ViewportClient = World->GetGameViewport();
				GEngine->SetEngineStat(World, ViewportClient, TEXT("PixelStreaming"), bHudStatsEnabled);
			}
		}
	}

	void OnKeyframeIntervalChanged(IConsoleVariable* Var)
	{
		AsyncTask(ENamedThreads::GameThread, [Var]() {
			IConsoleVariable* CVarNVENCKeyframeInterval = IConsoleManager::Get().FindConsoleVariable(TEXT("NVENC.KeyframeInterval"));
			if (CVarNVENCKeyframeInterval)
			{
				CVarNVENCKeyframeInterval->Set(Var->GetInt(), ECVF_SetByCommandline);
			}

			IConsoleVariable* CVarAMFKeyframeInterval = IConsoleManager::Get().FindConsoleVariable(TEXT("AMF.KeyframeInterval"));
			if (CVarNVENCKeyframeInterval)
			{
				CVarAMFKeyframeInterval->Set(Var->GetInt(), ECVF_SetByCommandline);
			}
		});
	}

	void OnWebRTCFpsChanged(IConsoleVariable* Var)
	{
		IPixelStreamingModule::Get().ForEachStreamer([Var](TSharedPtr<IPixelStreamingStreamer> Streamer) {
			Streamer->SetStreamFPS(Var->GetInt());
		});
	}

	void OnWebRTCBitrateRangeChanged(IConsoleVariable* Var)
	{
		IPixelStreamingModule::Get().ForEachStreamer([](TSharedPtr<IPixelStreamingStreamer> Streamer) {
			Streamer->RefreshStreamBitrate();
		});
	}

	// Ends Pixel Streaming Plugin CVars

	// Begin utility functions etc.
	std::map<FString, ERateControlMode> const RateControlCVarMap{
		{ "ConstQP", ERateControlMode::ConstQP },
		{ "VBR", ERateControlMode::VBR },
		{ "CBR", ERateControlMode::CBR },
	};

	std::map<FString, EMultipassMode> const MultipassCVarMap{
		{ "DISABLED", EMultipassMode::Disabled },
		{ "QUARTER", EMultipassMode::Quarter },
		{ "FULL", EMultipassMode::Full },
	};

	std::map<FString, EH264Profile> const H264ProfileMap{
		{ "AUTO", EH264Profile::Auto },
		{ "BASELINE", EH264Profile::Baseline },
		{ "MAIN", EH264Profile::Main },
		{ "HIGH", EH264Profile::High },
		{ "HIGH444", EH264Profile::High444 },
		{ "PROGRESSIVE_HIGH", EH264Profile::ProgressiveHigh },
		{ "CONSTRAINED_HIGH", EH264Profile::ConstrainedHigh },
	};

	std::map<FString, EAVPreset> const EncoderPresetMap{
		{ "ULTRA_LOW_LATENCY", EAVPreset::UltraLowQuality },
		{ "LOW_QUALITY", EAVPreset::LowQuality },
		{ "DEFAULT", EAVPreset::Default },
		{ "HIGH_QUALITY", EAVPreset::HighQuality },
		{ "LOSSLESS", EAVPreset::Lossless },
	};

	ERateControlMode GetRateControlCVar()
	{
		const FString EncoderRateControl = CVarPixelStreamingEncoderRateControl.GetValueOnAnyThread();
		auto const Iter = RateControlCVarMap.find(EncoderRateControl);
		if (Iter == std::end(RateControlCVarMap))
			return ERateControlMode::CBR;
		return Iter->second;
	}

	EMultipassMode GetMultipassCVar()
	{
		const FString EncoderMultipass = CVarPixelStreamingEncoderMultipass.GetValueOnAnyThread();
		auto const Iter = MultipassCVarMap.find(EncoderMultipass);
		if (Iter == std::end(MultipassCVarMap))
			return EMultipassMode::Full;
		return Iter->second;
	}

	webrtc::DegradationPreference GetDegradationPreference()
	{
		FString DegradationPreference = CVarPixelStreamingDegradationPreference.GetValueOnAnyThread();
		if (DegradationPreference == "MAINTAIN_FRAMERATE")
		{
			return webrtc::DegradationPreference::MAINTAIN_FRAMERATE;
		}
		else if (DegradationPreference == "MAINTAIN_RESOLUTION")
		{
			return webrtc::DegradationPreference::MAINTAIN_RESOLUTION;
		}
		// Everything else, return balanced.
		return webrtc::DegradationPreference::BALANCED;
	}

	EH264Profile GetH264Profile()
	{
		const FString H264Profile = CVarPixelStreamingH264Profile.GetValueOnAnyThread();
		auto const Iter = H264ProfileMap.find(H264Profile);
		if (Iter == std::end(H264ProfileMap))
			return EH264Profile::Baseline;
		return Iter->second;
	}

	EAVPreset GetEncoderPreset()
	{
		const FString EncoderPreset = CVarPixelStreamingEncoderPreset.GetValueOnAnyThread();
		auto const Iter = EncoderPresetMap.find(EncoderPreset);
		if (Iter == std::end(EncoderPresetMap))
			return EAVPreset::UltraLowQuality;
		return Iter->second;
	}

	// End utility functions etc.

	FPixelStreamingSimulcastParameters SimulcastParameters;

	void ReadSimulcastParameters()
	{
		SimulcastParameters.Layers.Empty();

		FString StringOptions;
		bool bPassedSimulcastParams = FParse::Value(FCommandLine::Get(), TEXT("SimulcastParameters="), StringOptions, false);

		// If no simulcast parameters are passed use some default values
		if (!bPassedSimulcastParams)
		{
			// StringOptions = FString(TEXT("1.0,5000000,20000000,2.0,1000000,5000000,4.0,50000,1000000"));
			StringOptions = FString(TEXT("1.0,0,1000000000"));
		}

		TArray<FString> ParameterArray;
		StringOptions.ParseIntoArray(ParameterArray, TEXT(","), true);
		const int OptionCount = ParameterArray.Num();
		bool bSuccess = OptionCount % 3 == 0;
		int NextOption = 0;
		while (bSuccess && ((OptionCount - NextOption) >= 3))
		{
			FPixelStreamingSimulcastParameters::FPixelStreamingSimulcastLayer Layer;
			bSuccess = FDefaultValueHelper::ParseFloat(ParameterArray[NextOption++], Layer.Scaling);
			bSuccess = FDefaultValueHelper::ParseInt(ParameterArray[NextOption++], Layer.MinBitrate);
			bSuccess = FDefaultValueHelper::ParseInt(ParameterArray[NextOption++], Layer.MaxBitrate);
			SimulcastParameters.Layers.Add(Layer);
		}

		if (!bSuccess)
		{
			// failed parsing the parameters. just ignore the parameters.
			UE_LOGF(LogPixelStreaming, Error, "Simulcast parameters malformed. Expected [Scaling_0, MinBitrate_0, MaxBitrate_0, ... , Scaling_N, MinBitrate_N, MaxBitrate_N] as [float, int, int, ... , float, int, int] etc.]");
			SimulcastParameters.Layers.Empty();
		}
	}

	void SetCodec(EPixelStreamingCodec Codec)
	{
		switch (Codec)
		{
			default:
			case EPixelStreamingCodec::H264:
				CVarPixelStreamingEncoderCodec.AsVariable()->Set(TEXT("H264"));
				break;
			case EPixelStreamingCodec::AV1:
				CVarPixelStreamingEncoderCodec.AsVariable()->Set(TEXT("AV1"));
				break;
			case EPixelStreamingCodec::VP8:
				CVarPixelStreamingEncoderCodec.AsVariable()->Set(TEXT("VP8"));
				break;
			case EPixelStreamingCodec::VP9:
				CVarPixelStreamingEncoderCodec.AsVariable()->Set(TEXT("VP9"));
				break;
		}
	}

	/*
	 * Selected Codec.
	 */
	EPixelStreamingCodec GetSelectedCodec()
	{
		const FString CodecStr = CVarPixelStreamingEncoderCodec.GetValueOnAnyThread();
		if (CodecStr == TEXT("H264"))
		{
			return EPixelStreamingCodec::H264;
		}
		else if (CodecStr == TEXT("AV1"))
		{
			return EPixelStreamingCodec::AV1;
		}
		else if (CodecStr == TEXT("VP8"))
		{
			return EPixelStreamingCodec::VP8;
		}
		else if (CodecStr == TEXT("VP9"))
		{
			return EPixelStreamingCodec::VP9;
		}
		else
		{
			return EPixelStreamingCodec::H264;
		}
	}

	bool IsCodecVPX()
	{
		EPixelStreamingCodec SelectedCodec = GetSelectedCodec();
		return SelectedCodec == EPixelStreamingCodec::VP8 || SelectedCodec == EPixelStreamingCodec::VP9;
	}

	bool IsCoupledFramerate()
	{
		return !CVarPixelStreamingDecoupleFramerate.GetValueOnAnyThread();
	}

	bool GetControlScheme(FString& OutControlScheme)
	{
		return FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingControlScheme="), OutControlScheme);
	}

	bool GetFastPan(float& OutFastPan)
	{
		return FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingFastPan="), OutFastPan);
	}

	bool GetSignallingServerUrl(FString& OutSignallingServerURL)
	{
		return FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingURL="), OutSignallingServerURL);
	}

	bool GetSignallingServerIP(FString& OutSignallingServerIP)
	{
		return FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingIP="), OutSignallingServerIP);
	}

	bool GetSignallingServerPort(uint16& OutSignallingServerPort)
	{
		return FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingPort="), OutSignallingServerPort);
	}

	bool GetVideoPacing(float& OutPacingFactor, float& OutMaxDelayMs)
	{
		OutPacingFactor = CVarPixelStreamingWebRTCVideoPacingFactor.GetValueOnAnyThread();
		OutMaxDelayMs = CVarPixelStreamingWebRTCVideoPacingMaxDelay.GetValueOnAnyThread();
		// Note: Use of OR operator here, field trial is enable if either of these is non-zero and positive.
		return OutPacingFactor >= 0.0 || OutMaxDelayMs >= 0.0;
	}

	EInputControllerMode GetInputControllerMode()
	{
		// Convert the current value to all lowercase and remove any whitespace.
		const FString InputControllerMode = CVarPixelStreamingInputController.GetValueOnAnyThread().ToLower().TrimStartAndEnd();

		if (InputControllerMode == TEXT("host"))
		{
			return EInputControllerMode::Host;
		}
		else
		{
			return EInputControllerMode::Any;
		}
	}

	FString GetDefaultStreamerID()
	{
		return DefaultStreamerID;
	}

	FString GetDefaultSignallingURL()
	{
		return DefaultSignallingURL;
	}

	/*
	 * Stats logger - as turned on/off by CVarPixelStreamingLogStats
	 */
	void ConsumeStat(FPixelStreamingPlayerId PlayerId, FName StatName, float StatValue)
	{
		UE_LOGF(LogPixelStreaming, Log, "[%ls](%ls) = %f", *PlayerId, *StatName.ToString(), StatValue);
	}

	FDelegateHandle LogStatsHandle;

	void OnLogStatsChanged(IConsoleVariable* Var)
	{
		bool bLogStats = Var->GetBool();

		UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates();
		if (!Delegates)
		{
			return;
		}

		if (bLogStats)
		{
			LogStatsHandle = Delegates->OnStatChangedNative.AddStatic(&ConsumeStat);
		}
		else
		{
			Delegates->OnStatChangedNative.Remove(LogStatsHandle);
		}
	}

	uint32 PortAllocatorParameters;

	void OnPortAllocatorParametersChanged(IConsoleVariable* Var)
	{
		PortAllocatorParameters = 0;

		FString StringOptions = Var->GetString();
		if(StringOptions.IsEmpty())
		{
			return;
		}

		TArray<FString> FlagArray;
		StringOptions.ParseIntoArray(FlagArray, TEXT(","), true);
		int OptionCount = FlagArray.Num();
		while (OptionCount > 0)
		{
			FString Flag = FlagArray[OptionCount - 1];	

			// Flags must match what's in Engine\Source\ThirdParty\WebRTC\xxxx\Include\p2p\base\port_allocator.h
			if(Flag == "DISABLE_UDP")
			{
				PortAllocatorParameters |= 0x01;
			}
			else if(Flag == "DISABLE_STUN")
			{
				PortAllocatorParameters |= 0x02;
			}
			else if(Flag == "DISABLE_RELAY")
			{
				PortAllocatorParameters |= 0x04;
			}
			else if(Flag == "DISABLE_TCP")
			{
				PortAllocatorParameters |= 0x08;
			}
			else if(Flag == "ENABLE_IPV6")
			{
				PortAllocatorParameters |= 0x40;
			}
			else if(Flag == "ENABLE_SHARED_SOCKET")
			{
				PortAllocatorParameters |= 0x100;
			}
			else if(Flag == "ENABLE_STUN_RETRANSMIT_ATTRIBUTE")
			{
				PortAllocatorParameters |= 0x200;
			}
			else if(Flag == "DISABLE_ADAPTER_ENUMERATION")
			{
				PortAllocatorParameters |= 0x400;
			}
			else if(Flag == "DISABLE_DEFAULT_LOCAL_CANDIDATE")
			{
				PortAllocatorParameters |= 0x800;
			}
			else if(Flag == "DISABLE_UDP_RELAY")
			{
				PortAllocatorParameters |= 0x1000;
			}
			else if(Flag == "ENABLE_IPV6_ON_WIFI")
			{
				PortAllocatorParameters |= 0x4000;
			}
			else if(Flag == "ENABLE_ANY_ADDRESS_PORTS")
			{
				PortAllocatorParameters |= 0x8000;
			}
			else if(Flag == "DISABLE_LINK_LOCAL_NETWORKS")
			{
				PortAllocatorParameters |= 0x10000;
			}
			else
			{
				UE_LOGF(LogPixelStreaming, Warning, "Unknown port allocator flag: %ls", *Flag);
			}
			OptionCount--;
		}
	}

	/*
	 * Settings parsing and initialization.
	 */

	// Some settings need to be set after streamer is initialized
	void OnStreamerReady(IPixelStreamingModule& Module)
	{
		CVarPixelStreamingLogStats.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnLogStatsChanged));
		CommandLineParseOption(TEXT("PixelStreamingLogStats"), CVarPixelStreamingLogStats);
	}

	void InitialiseSettings()
	{
		using namespace UE::PixelStreaming;
		UE_LOGF(LogPixelStreaming, Log, "Initialising Pixel Streaming settings.");

		FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingID="), DefaultStreamerID, false);

		CVarPixelStreamingOnScreenStats.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnHudStatsToggled));
		CVarPixelStreamingWebRTCFps.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnWebRTCFpsChanged));
		CVarPixelStreamingWebRTCMinBitrate.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnWebRTCBitrateRangeChanged));
		CVarPixelStreamingWebRTCMaxBitrate.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnWebRTCBitrateRangeChanged));
		CVarPixelStreamingEncoderKeyframeInterval.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnKeyframeIntervalChanged));
		CVarPixelStreamingWebRTCPortAllocatorFlags.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnPortAllocatorParametersChanged));

		// Values parse from commands line
		CommandLineParseValue(TEXT("PixelStreamingEncoderKeyframeInterval="), CVarPixelStreamingEncoderKeyframeInterval);
		CommandLineParseValue(TEXT("PixelStreamingEncoderTargetBitrate="), CVarPixelStreamingEncoderTargetBitrate);
		CommandLineParseValue(TEXT("PixelStreamingEncoderMaxBitrate="), CVarPixelStreamingEncoderMaxBitrate);
		CommandLineParseValue(TEXT("PixelStreamingEncoderMinQP="), CVarPixelStreamingEncoderMinQP);
		CommandLineParseValue(TEXT("PixelStreamingEncoderMaxQP="), CVarPixelStreamingEncoderMaxQP);
		CommandLineParseValue(TEXT("PixelStreamingEncoderRateControl="), CVarPixelStreamingEncoderRateControl);
		CommandLineParseValue(TEXT("PixelStreamingEncoderMultipass="), CVarPixelStreamingEncoderMultipass);
		CommandLineParseValue(TEXT("PixelStreamingEncoderCodec="), CVarPixelStreamingEncoderCodec);
		CommandLineParseValue(TEXT("PixelStreamingEncoderMaxSessions="), CVarPixelStreamingEncoderMaxSessions);
		CommandLineParseValue(TEXT("PixelStreamingH264Profile="), CVarPixelStreamingH264Profile);
		CommandLineParseValue(TEXT("PixelStreamingEncoderPreset="), CVarPixelStreamingEncoderPreset);
		CommandLineParseValue(TEXT("PixelStreamingDegradationPreference="), CVarPixelStreamingDegradationPreference);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCDegradationPreference="), CVarPixelStreamingDegradationPreference);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCFps="), CVarPixelStreamingWebRTCFps);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCStartBitrate="), CVarPixelStreamingWebRTCStartBitrate);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCMinBitrate="), CVarPixelStreamingWebRTCMinBitrate);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCMaxBitrate="), CVarPixelStreamingWebRTCMaxBitrate);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCLowQpThreshold="), CVarPixelStreamingWebRTCLowQpThreshold);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCHighQpThreshold="), CVarPixelStreamingWebRTCHighQpThreshold);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCVideoPacingFactor="), CVarPixelStreamingWebRTCVideoPacingFactor);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCVideoPacingMaxDelay="), CVarPixelStreamingWebRTCVideoPacingMaxDelay);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCFieldTrials="), CVarPixelStreamingWebRTCFieldTrials);
		CommandLineParseValue(TEXT("PixelStreamingFreezeFrameQuality"), CVarPixelStreamingFreezeFrameQuality);
		CommandLineParseValue(TEXT("PixelStreamingInputController="), CVarPixelStreamingInputController);
		CommandLineParseValue(TEXT("PixelStreamingSignalingReconnectInterval="), CVarPixelStreamingSignalingReconnectInterval);
		CommandLineParseValue(TEXT("PixelStreamingDecoupleWaitFactor="), CVarPixelStreamingDecoupleWaitFactor);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCMinPort="), CVarPixelStreamingWebRTCMinPort);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCMaxPort="), CVarPixelStreamingWebRTCMaxPort);
		CommandLineParseValue(TEXT("PixelStreamingWebRTCPortAllocatorFlags="), CVarPixelStreamingWebRTCPortAllocatorFlags);

		// Options parse (if these exist they are set to true)
		CommandLineParseOption(TEXT("PixelStreamingOnScreenStats"), CVarPixelStreamingOnScreenStats);
		CommandLineParseOption(TEXT("PixelStreamingHudStats"), CVarPixelStreamingOnScreenStats);
		CommandLineParseOption(TEXT("PixelStreamingDebugDumpFrame"), CVarPixelStreamingDebugDumpFrame);
		CommandLineParseOption(TEXT("PixelStreamingEnableFillerData"), CVarPixelStreamingEnableFillerData);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCDisableStats"), CVarPixelStreamingWebRTCDisableStats);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCDisableReceiveAudio"), CVarPixelStreamingWebRTCDisableReceiveAudio);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCDisableTransmitAudio"), CVarPixelStreamingWebRTCDisableTransmitAudio);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCDisableTransmitVideo"), CVarPixelStreamingWebRTCDisableTransmitVideo);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCDisableAudioSync"), CVarPixelStreamingWebRTCDisableAudioSync);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCDisableFrameDropper"), CVarPixelStreamingWebRTCDisableFrameDropper);
		CommandLineParseOption(TEXT("PixelStreamingSendPlayerIdAsInteger"), CVarSendPlayerIdAsInteger);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCUseLegacyAudioDevice"), CVarPixelStreamingWebRTCUseLegacyAudioDevice);
		CommandLineParseOption(TEXT("PixelStreamingWebRTCEnableFlexFec"), CVarPixelStreamingWebRTCEnableFlexFec);
		CommandLineParseOption(TEXT("PixelStreamingDisableLatencyTester"), CVarPixelStreamingDisableLatencyTester);
		CommandLineParseOption(TEXT("PixelStreamingVPXUseCompute"), CVarPixelStreamingVPXUseCompute);
		CommandLineParseOption(TEXT("PixelStreamingExperimentalAudioInput"), CVarPixelStreamingExperimentalAudioInput);
		CommandLineParseOption(TEXT("PixelStreamingNegotiateCodecs"), CVarPixelStreamingWebRTCNegotiateCodecs);
		CommandLineParseOption(TEXT("PixelStreamingCaptureUseFence"), CVarPixelStreamingCaptureUseFence);
		CommandLineParseOption(TEXT("PixelStreamingDecoupleFramerate"), CVarPixelStreamingDecoupleFramerate);
		CommandLineParseOption(TEXT("PixelStreamingUseMediaCapture"), CVarPixelStreamingUseMediaCapture);

		ReadSimulcastParameters();

		IPixelStreamingModule& Module = IPixelStreamingModule::Get();
		Module.OnReady().AddStatic(&OnStreamerReady);

		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("PixelStreaming.StartStreaming"),
			TEXT("Start all streaming sessions"),
			FConsoleCommandDelegate::CreateLambda([]() {
				IPixelStreamingModule::Get().StartStreaming();
			}));

		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("PixelStreaming.StopStreaming"),
			TEXT("End any existing streaming sessions."),
			FConsoleCommandDelegate::CreateLambda([]() {
				IPixelStreamingModule::Get().StopStreaming();
			}));
	}

} // namespace UE::PixelStreaming::Settings
