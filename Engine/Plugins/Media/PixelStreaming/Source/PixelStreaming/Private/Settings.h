// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingSettings.h"
#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "InputCoreTypes.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "WebRTCIncludes.h"
#include "PixelStreamingCodec.h"

namespace UE::PixelStreaming::Settings
{
	extern void InitialiseSettings();

	// Begin Encoder CVars
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderTargetBitrate;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMaxBitrate;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingDebugDumpFrame;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMinQP;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMaxQP;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingEncoderRateControl;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingEnableFillerData;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingEncoderMultipass;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingH264Profile;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingEncoderPreset;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderKeyframeInterval;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderIntraRefreshPeriodFrames;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderIntraRefreshCountFrames;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingEncoderCodec;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingEncoderMaxSessions;
	// End Encoder CVars

	// Begin WebRTC CVars
	extern TAutoConsoleVariable<FString> CVarPixelStreamingDegradationPreference;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCFps;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCStartBitrate;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCMinBitrate;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingWebRTCMaxBitrate;
	extern TAutoConsoleVariable<int> CVarPixelStreamingWebRTCLowQpThreshold;
	extern TAutoConsoleVariable<int> CVarPixelStreamingWebRTCHighQpThreshold;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableReceiveAudio;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableTransmitAudio;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableTransmitVideo;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableAudioSync;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCUseLegacyAudioDevice;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCEnableFlexFec;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableStats;
	extern TAutoConsoleVariable<float> CVarPixelStreamingWebRTCAudioGain;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCNegotiateCodecs;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingWebRTCFieldTrials;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingWebRTCDisableFrameDropper;
	extern TAutoConsoleVariable<float> CVarPixelStreamingWebRTCVideoPacingFactor;
	extern TAutoConsoleVariable<float> CVarPixelStreamingWebRTCVideoPacingMaxDelay;
	extern TAutoConsoleVariable<int> CVarPixelStreamingWebRTCMinPort;
	extern TAutoConsoleVariable<int> CVarPixelStreamingWebRTCMaxPort;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingWebRTCPortAllocatorFlags;
	// End WebRTC CVars

	// Begin Pixel Streaming Plugin CVars
	extern TAutoConsoleVariable<bool> CVarPixelStreamingOnScreenStats;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingLogStats;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingFreezeFrameQuality;
	extern TAutoConsoleVariable<bool> CVarSendPlayerIdAsInteger;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingDisableLatencyTester;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingVPXUseCompute;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingInputController;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingSuppressICECandidateErrors;
	extern TAutoConsoleVariable<float> CVarPixelStreamingSignalingReconnectInterval;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingExperimentalAudioInput;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingCaptureUseFence;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingDecoupleFramerate;
	extern TAutoConsoleVariable<float> CVarPixelStreamingDecoupleWaitFactor;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingUseMediaCapture;
	extern TAutoConsoleVariable<int32> CVarPixelStreamingQualityControllerMode;
	// Ends Pixel Streaming Plugin CVars

	/* Pixel Streaming can limit who can send input (keyboard, mouse, etc). */
	enum EInputControllerMode
	{
		/* Any - Any peer can control input. */
		Any,
		/* Host - Only the "host" peer can control input. */
		Host
	};
	
	enum EQualityControllerMode : int32
	{
		GrantAllRequested = 0,
		GrantOnlyFirst		= 1
	};

	// Begin utility functions etc.
	bool IsCoupledFramerate();
	bool IsCodecVPX();
	void SetCodec(EPixelStreamingCodec Codec);
	EPixelStreamingCodec GetSelectedCodec();
	ERateControlMode GetRateControlCVar();
	EMultipassMode GetMultipassCVar();
	webrtc::DegradationPreference GetDegradationPreference();
	EH264Profile GetH264Profile();
	EAVPreset GetEncoderPreset();
	EInputControllerMode GetInputControllerMode();
	FString GetDefaultStreamerID();
	FString GetDefaultSignallingURL();
	bool GetControlScheme(FString& OutControlScheme);
	bool GetFastPan(float& OutFastPan);
	bool GetSignallingServerUrl(FString& OutSignallingServerURL);
	bool GetSignallingServerIP(FString& OutSignallingServerIP);
	bool GetSignallingServerPort(uint16& OutSignallingServerPort);

	/**
	* @param OutPacingFactor The pacing factor to use for the field trial.
	* @param OutMaxDelayMs The max delay to use for the field trial.
	* @return True if WebRTC-Video-Pacing field trial is enabled.
	*/
	bool GetVideoPacing(float& OutPacingFactor, float& OutMaxDelayMs);
	// End utility functions etc.

	extern FPixelStreamingSimulcastParameters SimulcastParameters;
	extern uint32 PortAllocatorParameters;

} // namespace UE::PixelStreaming::Settings