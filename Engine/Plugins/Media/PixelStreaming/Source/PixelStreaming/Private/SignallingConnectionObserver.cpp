// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignallingConnectionObserver.h"
#include "PlayerContext.h"
#include "PixelStreamingPrivate.h"

namespace UE::PixelStreaming
{
	FPixelStreamingSignallingConnectionObserver::FPixelStreamingSignallingConnectionObserver(FStreamer& InStreamer)
		: Streamer(InStreamer)
	{
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingConnected()
	{
		Streamer.OnStreamingStarted().Broadcast(&Streamer);
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingDisconnected(int32 StatusCode, const FString& Reason, bool bWasClean)
	{
		Streamer.DeleteAllPlayerSessions();
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingError(const FString& ErrorMsg)
	{
		Streamer.DeleteAllPlayerSessions();
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingConfig(const webrtc::PeerConnectionInterface::RTCConfiguration& Config)
	{
		Streamer.PeerConnectionConfig = Config;

		// We want periodic bandwidth probing so ramping happens quickly
		Streamer.PeerConnectionConfig.media_config.video.periodic_alr_bandwidth_probing = true;
	}

	// Streamer-only
	void FPixelStreamingSignallingConnectionObserver::OnSignallingSessionDescription(FPixelStreamingPlayerId PlayerId, webrtc::SdpType Type, const FString& Sdp)
	{
		switch (Type)
		{
			case webrtc::SdpType::kOffer:
				Streamer.OnOffer(PlayerId, Sdp);
				break;
			case webrtc::SdpType::kAnswer:
			case webrtc::SdpType::kPrAnswer:
			{
				Streamer.OnAnswer(PlayerId, Sdp);
				break;
			}
			case webrtc::SdpType::kRollback:
				UE_LOGF(LogPixelStreaming, Error, "Rollback SDP is currently unsupported. SDP is: %ls", *Sdp);
				break;
		}
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingRemoteIceCandidate(FPixelStreamingPlayerId PlayerId, const FString& SdpMid, int SdpMLineIndex, const FString& Sdp)
	{
		if (FPlayerContext* PlayerContext = Streamer.Players->Find(PlayerId))
		{
			PlayerContext->PeerConnection->AddRemoteIceCandidate(SdpMid, SdpMLineIndex, Sdp);
		}
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer)
	{
		Streamer.OnPlayerConnected(PlayerId, PlayerConfig, bSendOffer);
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingPlayerDisconnected(FPixelStreamingPlayerId PlayerId)
	{
		UE_LOGF(LogPixelStreaming, Log, "player %ls disconnected", *PlayerId);
		Streamer.DeletePlayerSession(PlayerId);
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingSFUPeerDataChannels(FPixelStreamingPlayerId SFUId, FPixelStreamingPlayerId PlayerId, int32 SendStreamId, int32 RecvStreamId)
	{
		FPlayerContext* PlayerContext = Streamer.Players->Find(SFUId);
		if (PlayerContext == nullptr)
		{
			UE_LOGF(LogPixelStreaming, Error, "Trying to create data channels from SFU connection but no SFU connection found.");
			return;
		}

		TSharedPtr<FPixelStreamingDataChannel> NewChannel = FPixelStreamingDataChannel::Create(*PlayerContext->PeerConnection, SendStreamId, RecvStreamId);
		Streamer.AddNewDataChannel(PlayerId, NewChannel);
	}

	void FPixelStreamingSignallingConnectionObserver::OnPlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate)
	{
		Streamer.PlayerRequestsBitrate(PlayerId, MinBitrate, MaxBitrate);
	}

	// These are player only and will only be relevant when on the receiving side of pixel streaming, such as the player plugin.
	void FPixelStreamingSignallingConnectionObserver::OnSignallingStreamerList(const TArray<FString>& StreamerList)
	{
		unimplemented();
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingSessionDescription(webrtc::SdpType Type, const FString& Sdp)
	{
		unimplemented();
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp)
	{
		unimplemented();
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingPlayerCount(uint32 Count)
	{
		unimplemented();
	}

	void FPixelStreamingSignallingConnectionObserver::OnSignallingPeerDataChannels(int32 SendStreamId, int32 RecvStreamId)
	{
		unimplemented();
	}

} // namespace UE::PixelStreaming
