// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingSignallingConnectionObserver.h"
#include "Streamer.h"

namespace UE::PixelStreaming
{
	class FPixelStreamingSignallingConnectionObserver : public IPixelStreamingSignallingConnectionObserver
	{
	public:
		FPixelStreamingSignallingConnectionObserver(FStreamer& InStreamer);
		virtual ~FPixelStreamingSignallingConnectionObserver() = default;

		virtual void OnSignallingConnected() override;
		virtual void OnSignallingDisconnected(int32 StatusCode, const FString& Reason, bool bWasClean) override;
		virtual void OnSignallingError(const FString& ErrorMsg) override;
		virtual void OnSignallingConfig(const webrtc::PeerConnectionInterface::RTCConfiguration& Config) override;

		// Streamer-only
		virtual void OnSignallingSessionDescription(FPixelStreamingPlayerId PlayerId, webrtc::SdpType Type, const FString& Sdp) override;
		virtual void OnSignallingRemoteIceCandidate(FPixelStreamingPlayerId PlayerId, const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) override;
		virtual void OnSignallingPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer) override;
		virtual void OnSignallingPlayerDisconnected(FPixelStreamingPlayerId PlayerId) override;
		virtual void OnSignallingSFUPeerDataChannels(FPixelStreamingPlayerId SFUId, FPixelStreamingPlayerId PlayerId, int32 SendStreamId, int32 RecvStreamId) override;
		virtual void OnPlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate) override;

		// Player-only
		virtual void OnSignallingStreamerList(const TArray<FString>& StreamerList) override;
		virtual void OnSignallingSessionDescription(webrtc::SdpType Type, const FString& Sdp) override;
		virtual void OnSignallingRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) override;
		virtual void OnSignallingPlayerCount(uint32 Count) override;
		virtual void OnSignallingPeerDataChannels(int32 SendStreamId, int32 RecvStreamId) override;

	private:
		FStreamer& Streamer;
	};
} // namespace UE::PixelStreaming
