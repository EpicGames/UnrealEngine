// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats.h"
#include "Async/Async.h"
#include "CanvasTypes.h"
#include "Engine/GameViewportClient.h"
#include "PixelStreamingDelegates.h"
#include "Settings.h"
#include "Engine/Console.h"
#include "ConsoleSettings.h"
#include "PixelStreamingPeerConnection.h"
#include "UnrealClient.h"

// Complete the defintion for IPixelStreamingStats.h
IPixelStreamingStats& IPixelStreamingStats::Get()
{
	IPixelStreamingStats* Stats = UE::PixelStreaming::FStats::Get();
	return *Stats;
}

namespace UE::PixelStreaming
{
	FStats* FStats::Instance = nullptr;

	FStats* FStats::Get()
	{
		if (Instance == nullptr)
		{
			Instance = new FStats();
		}
		return Instance;
	}

	FStats::FStats()
	{
		checkf(Instance == nullptr, TEXT("There should only ever been one PixelStreaming stats object."));
		UConsole::RegisterConsoleAutoCompleteEntries.AddRaw(this, &FStats::UpdateConsoleAutoComplete);
	}

	void FStats::StorePeerStat(FPixelStreamingPlayerId PlayerId, FName StatCategory, FStatData Stat)
	{
		FName& StatName = Stat.Alias.IsSet() ? Stat.Alias.GetValue() : Stat.StatName;

		bool Updated = false;
		{
			FScopeLock Lock(&PeerStatsCS);

			if (!PeerStats.Contains(PlayerId))
			{
				PeerStats.Add(PlayerId, FPeerStats(PlayerId));
				Updated = true;
			}
			else
			{
				Updated = PeerStats[PlayerId].StoreStat(StatCategory, Stat);
			}
		}

		if (Updated)
		{
			if (Stat.ShouldGraph())
			{
				GraphValue(StatName, Stat.StatValue, 60, 0, Stat.StatValue * 10.0f, 0);
			}

			// If a stat has an alias, use that as the storage key, otherwise use its display name
			FireStatChanged(PlayerId, StatName, Stat.StatValue);
		}
	}

	bool FStats::QueryPeerStat(FPixelStreamingPlayerId PlayerId, FName StatCategory, FName StatToQuery, double& OutValue) const
	{
		FScopeLock Lock(&PeerStatsCS);

		if (const FPeerStats* SinglePeerStats = PeerStats.Find(PlayerId))
		{
			return SinglePeerStats->GetStat(StatCategory, StatToQuery, OutValue);
		}

		return false;
	}

	void FStats::RemovePeerStats(FPixelStreamingPlayerId PlayerId)
	{
		FScopeLock Lock(&PeerStatsCS);

		PeerStats.Remove(PlayerId);

		if (PlayerId == SFU_PLAYER_ID)
		{
			TArray<FPixelStreamingPlayerId> ToRemove;

			for (auto& Entry : PeerStats)
			{
				FPixelStreamingPlayerId PeerId = Entry.Key;
				if (PeerId.Contains(TEXT("Simulcast"), ESearchCase::IgnoreCase, ESearchDir::FromStart))
				{
					ToRemove.Add(PeerId);
				}
			}

			for (FPixelStreamingPlayerId SimulcastLayerId : ToRemove)
			{
				PeerStats.Remove(SimulcastLayerId);
			}
		}
	}

	double CalcMA(double PrevAvg, int NumSamples, double Value)
	{
		const double Result = NumSamples * PrevAvg + Value;
		return Result / (PrevAvg + 1.0);
	}

	double CalcEMA(double PrevAvg, int NumSamples, double Value)
	{
		const double Mult = 2.0 / (NumSamples + 1.0);
		const double Result = (Value - PrevAvg) * Mult + PrevAvg;
		return Result;
	}

	void FStats::StoreApplicationStat(FStatData Stat)
	{
		bool bUpdated = false;

		// If a stat has an alias, use that as the storage key, otherwise use its display name
		FName& StatName = Stat.Alias.IsSet() ? Stat.Alias.GetValue() : Stat.StatName;

		if (Stat.ShouldGraph())
		{
			GraphValue(StatName, Stat.StatValue, 60, 0, Stat.StatValue, 0);
		}

		{
			FScopeLock Lock(&ApplicationStatsCS);

			if (ApplicationStats.Contains(StatName))
			{
				FStoredStat* StoredStat = ApplicationStats.Find(StatName);

				if (Stat.bSmooth && StoredStat->Stat.StatValue != 0)
				{
					const int MaxSamples = 60;
					StoredStat->Stat.NumSamples = FGenericPlatformMath::Min(MaxSamples, StoredStat->Stat.NumSamples + 1);
					if (StoredStat->Stat.NumSamples < MaxSamples)
					{
						StoredStat->Stat.LastEMA = CalcMA(StoredStat->Stat.LastEMA, StoredStat->Stat.NumSamples - 1, Stat.StatValue);
					}
					else
					{
						StoredStat->Stat.LastEMA = CalcEMA(StoredStat->Stat.LastEMA, StoredStat->Stat.NumSamples - 1, Stat.StatValue);
					}
					StoredStat->Stat.StatValue = StoredStat->Stat.LastEMA;
					bUpdated = true;
				}
				else
				{
					bUpdated = StoredStat->Stat.StatValue != Stat.StatValue;
					StoredStat->Stat.StatValue = Stat.StatValue;
				}

				if (bUpdated && StoredStat->Renderable.IsSet())
				{
					FText TextToDisplay = FText::FromString(FString::Printf(TEXT("%s: %.*f"), *Stat.StatName.ToString(), Stat.NDecimalPlacesToPrint, StoredStat->Stat.StatValue));
					StoredStat->Renderable.GetValue().Text = TextToDisplay;
				}
			}
			else
			{
				FText TextToDisplay = FText::FromString(FString::Printf(TEXT("%s: %.*f"), *Stat.StatName.ToString(), Stat.NDecimalPlacesToPrint, Stat.StatValue));
				FStoredStat StoredStat(Stat);

				if (Stat.ShouldDisplayText())
				{
					FCanvasTextItem Text = FCanvasTextItem(FVector2D(0, 0), TextToDisplay, FSlateFontInfo(FSlateFontInfo(UEngine::GetSmallFont(), 12)), FLinearColor(0, 1, 0));
					Text.EnableShadow(FLinearColor::Black);
					StoredStat.Renderable = Text;
				}

				ApplicationStats.Add(StatName, StoredStat);
				bUpdated = true;
			}
		}

		if (bUpdated)
		{
			FireStatChanged(FPixelStreamingPlayerId(TEXT("Application")), Stat.StatName, Stat.StatValue);
		}
	}

	void FStats::FireStatChanged(FPixelStreamingPlayerId PlayerId, FName StatName, float StatValue)
	{
		// Broadcast must be done on the GameThread because the GameThread can remove the delegates.
		// If removing and broadcast happens simultaneously it causes a datarace failure.
		AsyncTask(ENamedThreads::GameThread, [PlayerId, StatName, StatValue]() {
			if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
			{
				Delegates->OnStatChangedNative.Broadcast(PlayerId, StatName, StatValue);
				Delegates->OnStatChanged.Broadcast(PlayerId, StatName, StatValue);
			}
		});
	}

	void FStats::UpdateConsoleAutoComplete(TArray<FAutoCompleteCommand>& AutoCompleteList)
	{
		// This *might* need to be on the game thread? I haven't seen issues not explicitly putting it on the game thread though.

		const UConsoleSettings* ConsoleSettings = GetDefault<UConsoleSettings>();

		AutoCompleteList.AddDefaulted();
		FAutoCompleteCommand& AutoCompleteCommand = AutoCompleteList.Last();
		AutoCompleteCommand.Command = TEXT("Stat PixelStreaming");
		AutoCompleteCommand.Desc = TEXT("Displays stats about Pixel Streaming on screen.");
		AutoCompleteCommand.Color = ConsoleSettings->AutoCompleteCommandColor;

		AutoCompleteList.AddDefaulted();
		FAutoCompleteCommand& AutoCompleteGraphCommand = AutoCompleteList.Last();
		AutoCompleteGraphCommand.Command = TEXT("Stat PixelStreamingGraphs");
		AutoCompleteGraphCommand.Desc = TEXT("Displays graphs about Pixel Streaming on screen.");
		AutoCompleteGraphCommand.Color = ConsoleSettings->AutoCompleteCommandColor;
	}

	int32 FStats::OnRenderStats(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		if (GAreScreenMessagesEnabled)
		{
			Y += 50;
			// Draw each peer's stats in a column, so we must recall where Y starts for each column
			int32 YStart = Y;

			// --------- Draw stats for this Pixel Streaming instance ----------

			{
				FScopeLock Lock(&ApplicationStatsCS);

				for (auto& ApplicationStatEntry : ApplicationStats)
				{
					FStoredStat& StatToDraw = ApplicationStatEntry.Value;
					if (!StatToDraw.Renderable.IsSet())
					{
						continue;
					}
					FCanvasTextItem& Text = StatToDraw.Renderable.GetValue();
					Text.Position.X = X;
					Text.Position.Y = Y;
					Canvas->DrawItem(Text);
					Y += Text.DrawnSize.Y;
				}
			}

			// --------- Draw stats for each peer ----------

			// increment X now we are done drawing application stats
			X += 435;

			{
				FScopeLock Lock(&PeerStatsCS);

				// <FPixelStreamingPlayerId, FPeerStats>
				for (auto& EachPeerEntry : PeerStats)
				{
					FPeerStats& SinglePeerStats = EachPeerEntry.Value;
					if (SinglePeerStats.GetStatGroups().Num() == 0)
					{
						continue;
					}

					// Reset Y for each peer as each peer gets it own column
					Y = YStart;

					SinglePeerStats.PlayerIdCanvasItem.Position.X = X;
					SinglePeerStats.PlayerIdCanvasItem.Position.Y = Y;
					Canvas->DrawItem(SinglePeerStats.PlayerIdCanvasItem);
					Y += SinglePeerStats.PlayerIdCanvasItem.DrawnSize.Y;

					// <FName, FStatGroup>
					for (auto& StatGroupEntry : SinglePeerStats.GetStatGroups())
					{
						const FStatGroup& StatGroup = StatGroupEntry.Value;

						// Draw StatGroup category name
						{
							const FCanvasTextItem& Text = StatGroup.CategoryCanvasItem;
							FCanvasTextItem& NonConstText = const_cast<FCanvasTextItem&>(Text);
							NonConstText.Position.X = X;
							NonConstText.Position.Y = Y;
							Canvas->DrawItem(NonConstText);
							Y += NonConstText.DrawnSize.Y;
						}

						// Draw the stat value
						for (auto& StatEntry : StatGroup.GetStoredStats())
						{
							const FStoredStat& Stat = StatEntry.Value;
							if (!Stat.Renderable.IsSet())
							{
								continue;
							}
							const FCanvasTextItem& Text = Stat.Renderable.GetValue();
							FCanvasTextItem& NonConstText = const_cast<FCanvasTextItem&>(Text);
							NonConstText.Position.X = X;
							NonConstText.Position.Y = Y;
							Canvas->DrawItem(NonConstText);
							Y += NonConstText.DrawnSize.Y;
						}
					}

					// Each peer's stats gets its own column
					X += 250;
				}
			}
		}
		return Y;
	}

	bool FStats::OnToggleStats(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		// todo: all about the toggle func
		return true;
	}

	bool FStats::OnToggleGraphs(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		return true;
	}

	int32 FStats::OnRenderGraphs(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		checkf(IsInGameThread(), TEXT("FStats::OnRenderGraphs must be called from the gamethread."));

		static const int XOffset = 50;
		static const int YOffset = 50;
		FVector2D GraphPos{ XOffset, YOffset };
		FVector2D GraphSize{ 200, 200 };
		float GraphSpacing = 5;

		for (auto& [GraphName, Graph] : Graphs)
		{
			Graph.Draw(Canvas, GraphPos, GraphSize);
			GraphPos.X += GraphSize.X + GraphSpacing;
			if ((GraphPos.X + GraphSize.X) > Canvas->GetRenderTarget()->GetSizeXY().X)
			{
				GraphPos.Y += GraphSize.Y + GraphSpacing;
				GraphPos.X = XOffset;
			}
		}

		for (auto& [TileName, Tile] : Tiles)
		{
			Tile.Position.X = GraphPos.X;
			Tile.Position.Y = GraphPos.Y;
			Tile.Size = GraphSize;
			Tile.Draw(Canvas);
			GraphPos.X += GraphSize.X + GraphSpacing;
			if ((GraphPos.X + GraphSize.X) > Canvas->GetRenderTarget()->GetSizeXY().X)
			{
				GraphPos.Y += GraphSize.Y + GraphSpacing;
				GraphPos.X = XOffset;
			}
		}

		return Y;
	}

	void FStats::PollPixelStreamingSettings()
	{
		double DeltaSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTimeSettingsPolledCycles);
		if (DeltaSeconds > 1)
		{
			StoreApplicationStat(FStatData(FName(TEXT("PixelStreaming.Encoder.MinQP")), Settings::CVarPixelStreamingEncoderMinQP.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStatData(FName(TEXT("PixelStreaming.Encoder.MaxQP")), Settings::CVarPixelStreamingEncoderMaxQP.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStatData(FName(TEXT("PixelStreaming.Encoder.KeyframeInterval (frames)")), Settings::CVarPixelStreamingEncoderKeyframeInterval.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStatData(FName(TEXT("PixelStreaming.WebRTC.Fps")), Settings::CVarPixelStreamingWebRTCFps.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStatData(FName(TEXT("PixelStreaming.WebRTC.StartBitrate")), Settings::CVarPixelStreamingWebRTCStartBitrate.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStatData(FName(TEXT("PixelStreaming.WebRTC.MinBitrate")), Settings::CVarPixelStreamingWebRTCMinBitrate.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStatData(FName(TEXT("PixelStreaming.WebRTC.MaxBitrate")), Settings::CVarPixelStreamingWebRTCMaxBitrate.GetValueOnAnyThread(), 0));

			LastTimeSettingsPolledCycles = FPlatformTime::Cycles64();
		}
	}

	void FStats::Tick(float DeltaTime)
	{
		RTCStatsPolledDelta += DeltaTime;

		// Note (Luke): If we want more metrics from WebRTC there is also the histogram counts.
		// For example:
		// RTC_HISTOGRAM_COUNTS("WebRTC.Video.NacksSent", nacks_sent, 1, 100000, 100);
		// webrtc::metrics::Histogram* Hist1 = webrtc::metrics::HistogramFactoryGetCounts("WebRTC.Video.NacksSent", 0, 100000, 100);
		// Will require calling webrtc::metrics::Enable();

		// We only poll WebRTC stats every 1s as this matches chrome://webrtc-internals
		// and more frequency does not seem useful as these stats are mostly used for visual inspection
		if (RTCStatsPolledDelta > 1.0f)
		{
			OnStatsPolled.Broadcast();
			RTCStatsPolledDelta = 0;
		}

		PollPixelStreamingSettings();

		if (!GEngine)
		{
			return;
		}

		if (!bRegisterEngineStats)
		{
			RegisterEngineHooks();
		}
	}

	void FStats::RemoveAllPeerStats()
	{
		FScopeLock LockPeers(&PeerStatsCS);
		PeerStats.Empty();
	}

	void FStats::ExecStatPS()
	{
		// Intetionally empty, registering this function is mostly about getting the stat comment to show up in autocomplete
	}

	void FStats::ExecStatPSGraphs()
	{
		// Intetionally empty, registering this function is mostly about getting the stat comment to show up in autocomplete
	}

	void FStats::RegisterEngineHooks()
	{
		GAreScreenMessagesEnabled = true;

		const FName StatName("STAT_PixelStreaming");
		const FName StatCategory("STATCAT_PixelStreaming");
		const FText StatDescription(FText::FromString("Pixel Streaming stats for all connected peers."));
		UEngine::FEngineStatRender RenderStatFunc = UEngine::FEngineStatRender::CreateRaw(this, &FStats::OnRenderStats);
		UEngine::FEngineStatToggle ToggleStatFunc = UEngine::FEngineStatToggle::CreateRaw(this, &FStats::OnToggleStats);
		GEngine->AddEngineStat(StatName, StatCategory, StatDescription, RenderStatFunc, ToggleStatFunc, false);

		// We register this console command so we get autocomplete on `Stat PixelStreaming`
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("Stat PixelStreaming"),
			TEXT("Stats for the Pixel Streaming plugin and its peers."),
			FConsoleCommandDelegate::CreateRaw(this, &FStats::ExecStatPS),
			ECVF_Default);

		const FName GraphName("STAT_PixelStreamingGraphs");
		const FText GraphDescription(FText::FromString("Pixel Streaming graphs showing frame pipeline timings."));
		UEngine::FEngineStatRender RenderGraphFunc = UEngine::FEngineStatRender::CreateRaw(this, &FStats::OnRenderGraphs);
		UEngine::FEngineStatToggle ToggleGraphFunc = UEngine::FEngineStatToggle::CreateRaw(this, &FStats::OnToggleGraphs);
		GEngine->AddEngineStat(GraphName, StatCategory, GraphDescription, RenderGraphFunc, ToggleGraphFunc, false);

		// We register this console command so we get autocomplete on `Stat PixelStreamingGraphs`
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("Stat PixelStreamingGraphs"),
			TEXT("Draws stats graphs for the Pixel Streaming plugin."),
			FConsoleCommandDelegate::CreateRaw(this, &FStats::ExecStatPSGraphs),
			ECVF_Default);

		bool StatsEnabled = Settings::CVarPixelStreamingOnScreenStats.GetValueOnAnyThread();
		if (StatsEnabled)
		{
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
			{
				if (WorldContext.WorldType == EWorldType::Game || WorldContext.WorldType == EWorldType::PIE)
				{
					UWorld* World = WorldContext.World();
					UGameViewportClient* ViewportClient = World->GetGameViewport();
					GEngine->SetEngineStat(World, ViewportClient, TEXT("PixelStreaming"), StatsEnabled);
				}
			}
		}

		bRegisterEngineStats = true;
	}

	//
	// ---------------- FStatGroup ---------------------------
	// A collection of stats grouped together by a category name
	//
	bool FStatGroup::StoreStat(FStatData& StatToStore)
	{
		bool bUpdated = false;

		// If a stat has an alias, use that as the storage key, otherwise use its display name
		FName& StatName = StatToStore.Alias.IsSet() ? StatToStore.Alias.GetValue() : StatToStore.StatName;

		if (!StoredStats.Contains(StatName))
		{
			FStoredStat NewStat(StatToStore);

			// If we are displaying the stat, add a renderable for it
			if (StatToStore.ShouldDisplayText())
			{
				FText TextToDisplay = FText::FromString(FString::Printf(TEXT("%s: %.*f"), *StatToStore.StatName.ToString(), StatToStore.NDecimalPlacesToPrint, StatToStore.StatValue));
				FCanvasTextItem Text = FCanvasTextItem(FVector2D(0, 0), TextToDisplay, FSlateFontInfo(FSlateFontInfo(UEngine::GetSmallFont(), 12)), FLinearColor(0, 1, 0));
				Text.EnableShadow(FLinearColor::Black);
				NewStat.Renderable = Text;
			}

			// Actually store the stat
			StoredStats.Add(StatName, NewStat);

			// first time this stat has been stored, so we also need to sort our stats so they render in consistent order
			StoredStats.KeySort([](const FName& A, const FName& B) {
				return A.FastLess(B);
			});

			return true;
		}
		else
		{
			// We already have this stat, so just update it

			FStoredStat* StoredStat = StoredStats.Find(StatName);
			if (!StoredStat)
			{
				return false;
			}

			if (StoredStat->Stat.bSmooth && StoredStat->Stat.StatValue != 0)
			{
				const int MaxSamples = 60;
				StoredStat->Stat.NumSamples = FGenericPlatformMath::Min(MaxSamples, StoredStat->Stat.NumSamples + 1);
				if (StoredStat->Stat.NumSamples < MaxSamples)
				{
					StoredStat->Stat.LastEMA = CalcMA(StoredStat->Stat.LastEMA, StoredStat->Stat.NumSamples - 1, StatToStore.StatValue);
				}
				else
				{
					StoredStat->Stat.LastEMA = CalcEMA(StoredStat->Stat.LastEMA, StoredStat->Stat.NumSamples - 1, StatToStore.StatValue);
				}
				StoredStat->Stat.StatValue = StoredStat->Stat.LastEMA;
				bUpdated = true;
			}
			else
			{
				bUpdated = StoredStat->Stat.StatValue != StatToStore.StatValue;
				StoredStat->Stat.StatValue = StatToStore.StatValue;
			}

			if (!bUpdated)
			{
				return false;
			}
			else if (StoredStat->Stat.ShouldDisplayText() && StoredStat->Renderable.IsSet())
			{
				FText TextToDisplay = FText::FromString(FString::Printf(TEXT("%s: %.*f"), *StatToStore.StatName.ToString(), StatToStore.NDecimalPlacesToPrint, StoredStat->Stat.StatValue));
				StoredStat->Renderable.GetValue().Text = TextToDisplay;
			}
			return bUpdated;
		}
	}

	//
	// ---------------- FPeerStats ---------------------------
	// Stats specific to a particular peer, as opposed to the entire app.
	//

	bool FPeerStats::StoreStat(FName StatCategory, FStatData& StatToStore)
	{
		if (!StatGroups.Contains(StatCategory))
		{
			StatGroups.Add(StatCategory, FStatGroup(StatCategory));
		}
		return StatGroups[StatCategory].StoreStat(StatToStore);
	}

	bool UE::PixelStreaming::FPeerStats::GetStat(FName StatCategory, FName StatToQuery, double& OutValue) const
	{
		const FStatGroup* Group = GetStatGroups().Find(StatCategory);
		if (!Group)
		{
			return false;
		}
		const FStoredStat* StoredStat = Group->GetStoredStats().Find(StatToQuery);
		if (!StoredStat)
		{
			return false;
		}
		OutValue = StoredStat->Stat.StatValue;
		return true;
	}

	void FStats::GraphValue(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue)
	{
		if (IsInGameThread())
		{
			GraphValue_GameThread(InName, Value, InSamples, InMinRange, InMaxRange, InRefValue);
		}
		else
		{
			AsyncTask(ENamedThreads::Type::GameThread, [this, InName, Value, InSamples, InMinRange, InMaxRange, InRefValue]() {
				GraphValue_GameThread(InName, Value, InSamples, InMinRange, InMaxRange, InRefValue);
			});
		}
	}

	void FStats::GraphValue_GameThread(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue)
	{
		checkf(IsInGameThread(), TEXT("FStats::GraphValue_GameThread must be called from the gamethread."));

		if (!Graphs.Contains(InName))
		{
			auto& Graph = Graphs.Add(InName, FDebugGraph(InName, InSamples, InMinRange, InMaxRange, InRefValue));
			Graph.AddValue(Value);
		}
		else
		{
			Graphs[InName].AddValue(Value);
		}
	}

	double FStats::AddTimeStat(uint64 Millis, const FString& Label)
	{
		const double	DeltaMs = Millis;
		const FStatData TimeData{ FName(*Label), DeltaMs, 2, true };
		StoreApplicationStat(TimeData);
		return DeltaMs;
	}

	double FStats::AddTimeDeltaStat(uint64 Millis1, uint64 Millis2, const FString& Label)
	{
		const uint64 MaxMillis = FGenericPlatformMath::Max(Millis1, Millis2);
		const uint64 MinMillis = FGenericPlatformMath::Min(Millis1, Millis2);
		const double DeltaMs = (MaxMillis - MinMillis) * ((Millis1 > Millis2) ? 1.0 : -1.0);
		const FStatData TimeData{ FName(*Label), DeltaMs, 2, true };
		StoreApplicationStat(TimeData);
		return DeltaMs;
	}

	void FStats::AddFrameTimingStats(const FPixelCaptureFrameMetadata& FrameMetadata)
	{
		const double TimeCapture = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureEndCyles - FrameMetadata.CaptureStartCyles), FString::Printf(TEXT("%s Layer %d Frame Capture Time"), *FrameMetadata.ProcessName, FrameMetadata.Layer));
		const double TimeCPU = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureProcessCPUEndCycles - FrameMetadata.CaptureProcessCPUStartCycles), FString::Printf(TEXT("%s Layer %d Frame Capture CPU Time"), *FrameMetadata.ProcessName, FrameMetadata.Layer));
		const double TimeGPUDelay = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureProcessGPUEnqueueEndCycles - FrameMetadata.CaptureProcessGPUEnqueueStartCycles), FString::Printf(TEXT("%s Layer %d Frame Capture GPU Delay Time"), *FrameMetadata.ProcessName, FrameMetadata.Layer));
		const double TimeGPU = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureProcessGPUEndCycles - FrameMetadata.CaptureProcessGPUStartCycles), FString::Printf(TEXT("%s Layer %d Frame Capture GPU Time"), *FrameMetadata.ProcessName, FrameMetadata.Layer));
		const double TimeEncode = AddTimeDeltaStat(FrameMetadata.LastEncodeEndTime, FrameMetadata.LastEncodeStartTime, FString::Printf(TEXT("%s Layer %d Frame Encode Time"), *FrameMetadata.ProcessName, FrameMetadata.Layer));
		const double TimePacketize = AddTimeDeltaStat(FrameMetadata.LastPacketizationEndTime, FrameMetadata.LastPacketizationStartTime, FString::Printf(TEXT("%s Layer %d Frame Packetization Time"), *FrameMetadata.ProcessName, FrameMetadata.Layer));

		const FStatData UseData{ FName(*FString::Printf(TEXT("%s Layer %d Frame Uses"), *FrameMetadata.ProcessName, FrameMetadata.Layer)), static_cast<double>(FrameMetadata.UseCount), 0, false };
		StoreApplicationStat(UseData);

		const int Samples = 100;
		GraphValue(*FString::Printf(TEXT("%d Capture Time"), FrameMetadata.Layer), StaticCast<float>(TimeCapture), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("%d CPU Time"), FrameMetadata.Layer), StaticCast<float>(TimeCPU), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("%d GPU Delay Time"), FrameMetadata.Layer), StaticCast<float>(TimeGPUDelay), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("%d GPU Time"), FrameMetadata.Layer), StaticCast<float>(TimeGPU), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("%d Encode Time"), FrameMetadata.Layer), StaticCast<float>(TimeEncode), Samples, 0.0f, 10.0f);
		GraphValue(*FString::Printf(TEXT("%d Packetization Time"), FrameMetadata.Layer), StaticCast<float>(TimePacketize), Samples, 0.0f, 10.0f);
		GraphValue(*FString::Printf(TEXT("%d Frame Uses"), FrameMetadata.Layer), StaticCast<float>(FrameMetadata.UseCount), Samples, 0.0f, 10.0f);
	}

	void FStats::AddCanvasTile(FName Name, const FCanvasTileItem& Tile)
	{
		if (IsInGameThread())
		{
			AddCanvasTile_GameThread(Name, Tile);
		}
		else
		{
			AsyncTask(ENamedThreads::GameThread, [this, Name, Tile]() {
				AddCanvasTile_GameThread(Name, Tile);
			});
		}
	}

	void FStats::AddCanvasTile_GameThread(FName Name, const FCanvasTileItem& Tile)
	{
		checkf(IsInGameThread(), TEXT("FStats::AddCanvasTile_GameThread must be called from the gamethread."));
		Tiles.FindOrAdd(Name, Tile);
	}
} // namespace UE::PixelStreaming
