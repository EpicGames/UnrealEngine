// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"
#include "PixelStreamingPlayerId.h"
#include "CanvasItem.h"
#include "UnrealEngine.h"
#include "ConsoleSettings.h"
#include "DebugGraph.h"
#include "Misc/Optional.h"
#include "PixelCaptureFrameMetadata.h"
#include "IPixelStreamingStats.h"

namespace UE::PixelStreaming
{
	// An interface that allows us to collect webrtc stats from anything that implements it
	class IStatsSource
	{
	public:
		virtual ~IStatsSource() = default;
		virtual void PollWebRTCStats() const = 0;
	};

	struct FStatData
	{
	public:
		enum EDisplayFlags: uint8
		{
			HIDDEN	   = 0,
			TEXT	   = 1 << 0,
			GRAPH	   = 1 << 1,
		};
	public:
		FStatData(FName InStatName, double InStatValue, int InNDecimalPlacesToPrint, bool bInSmooth = false)
			: StatName(InStatName)
			, StatValue(InStatValue)
			, NDecimalPlacesToPrint(InNDecimalPlacesToPrint)
			, bSmooth(bInSmooth)
		{
		}

		bool operator==(const FStatData& Other) const
		{
			return Equals(Other);
		}

		bool Equals(const FStatData& Other) const
		{
			return StatName == Other.StatName;
		}

		bool IsHidden() { return DisplayFlags == EDisplayFlags::HIDDEN; }
		bool ShouldGraph() { return DisplayFlags & EDisplayFlags::GRAPH; }
		bool ShouldDisplayText() { return DisplayFlags & EDisplayFlags::TEXT; }

		FName StatName;
		double StatValue;
		int NDecimalPlacesToPrint;
		bool bSmooth;
		double LastEMA = 0;
		int NumSamples = 0;
		uint8 DisplayFlags = EDisplayFlags::TEXT; /* Some stats we only wish to store or broadcast, but not display */
		TOptional<FName> Alias; /* Some stats need an alias that they are stored by/queried by to disambiguate them from other stats */
	};

	FORCEINLINE uint32 GetTypeHash(const FStatData& Obj)
	{
		// From UnrealString.h
		return GetTypeHash(Obj.StatName);
	}

	// Stat that can be optionally rendered
	struct FStoredStat
	{
	public:
		FStatData Stat;
		TOptional<FCanvasTextItem> Renderable;
	public:
		FStoredStat(FStatData& InStat) : Stat(InStat) {}
	};

	// A grouping of stats by some category name
	class FStatGroup
	{
	public:
		FStatGroup(FName CategoryName)
			: GroupName(CategoryName)
			, CategoryCanvasItem(FVector2D(0, 0), FText::FromString(FString::Printf(TEXT("---%s---"), *CategoryName.ToString())), FSlateFontInfo(FSlateFontInfo(UEngine::GetSmallFont(), 15)), FLinearColor(0, 0.9f, 0.1f))
		{
			CategoryCanvasItem.EnableShadow(FLinearColor::Black);
		}

		virtual ~FStatGroup() = default;
		bool StoreStat(FStatData& StatToStore);
		const TMap<FName, FStoredStat>& GetStoredStats() const { return StoredStats; }
	private:
		FName GroupName;
		TMap<FName, FStoredStat> StoredStats;
	public:
		FCanvasTextItem CategoryCanvasItem;
	};

	// Pixel Streaming stats that are associated with a specific peer.
	class FPeerStats
	{

	public:
		FPeerStats(FPixelStreamingPlayerId InAssociatedPlayer)
			: AssociatedPlayer(InAssociatedPlayer)
			, PlayerIdCanvasItem(FVector2D(0, 0), FText::FromString(FString::Printf(TEXT("[Peer Stats(%s)]"), *AssociatedPlayer)), FSlateFontInfo(FSlateFontInfo(UEngine::GetSmallFont(), 15)), FLinearColor(0, 1, 0))
		{
			PlayerIdCanvasItem.EnableShadow(FLinearColor::Black);
		};

		bool StoreStat(FName StatCategory, FStatData& StatToStore);
		bool GetStat(FName StatCategory, FName StatToQuery, double& OutValue) const;
		const TMap<FName, FStatGroup>& GetStatGroups() const { return StatGroups; }

	private:
		int DisplayId = 0;
		FPixelStreamingPlayerId AssociatedPlayer;
		TMap<FName, FStatGroup> StatGroups;

	public:
		FCanvasTextItem PlayerIdCanvasItem;
	};

	// Stats about Pixel Streaming that can displayed either in the in-application HUD, in the log, or simply reported to some subscriber.
	// Stats can be enabled to draw on screen with:
	// `stat pixelstreaming`
	// `stat pixelstreaminggraphs`
	class FStats : public FTickableGameObject, public IPixelStreamingStats
	{
	public:
		virtual bool IsTickableInEditor() const override { return true; }

		static constexpr double SmoothingPeriod = 3.0 * 60.0;
		static constexpr double SmoothingFactor = 10.0 / 100.0;
		static FStats* Get();

		FStats(const FStats&) = delete;
		bool QueryPeerStat(FPixelStreamingPlayerId PlayerId, FName StatCategory, FName StatToQuery, double& OutValue) const;
		void RemovePeerStats(FPixelStreamingPlayerId PlayerId);
		void RemoveAllPeerStats();
		void StorePeerStat(FPixelStreamingPlayerId PlayerId, FName StatCategory, FStatData Stat);
		void StoreApplicationStat(FStatData PeerStat);
		void Tick(float DeltaTime);

		DECLARE_MULTICAST_DELEGATE(FOnStatsPolled);
		FOnStatsPolled OnStatsPolled;

		void ExecStatPS();
		bool OnToggleStats(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream);
		int32 OnRenderStats(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);

		void ExecStatPSGraphs();
		bool OnToggleGraphs(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream);
		int32 OnRenderGraphs(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);

		FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(PixelStreamingStats, STATGROUP_Tickables); }

		double AddTimeStat(uint64 Millis, const FString& Label);
		double AddTimeDeltaStat(uint64 Millis1, uint64 Millis2, const FString& Label);
		void AddFrameTimingStats(const FPixelCaptureFrameMetadata& FrameMetadata);
		void AddCanvasTile(FName Name, const FCanvasTileItem& Tile);

		// Begin IPixelStreamingStats interface
		friend class IPixelStreamingStats;
		void GraphValue(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue = 0.0f) override;
		// End IPixelStreamingStats interface

	private:
		FStats();
		void RegisterEngineHooks();
		void PollPixelStreamingSettings();
		void FireStatChanged(FPixelStreamingPlayerId PlayerId, FName StatName, float StatValue);
		void UpdateConsoleAutoComplete(TArray<FAutoCompleteCommand>& AutoCompleteList);
		void GraphValue_GameThread(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue);
		void AddCanvasTile_GameThread(FName Name, const FCanvasTileItem& Tile);

	private:
		static FStats* Instance;

		bool bRegisterEngineStats = false;

		mutable FCriticalSection PeerStatsCS;
		TMap<FPixelStreamingPlayerId, FPeerStats> PeerStats;

		mutable FCriticalSection ApplicationStatsCS;
		TMap<FName, FStoredStat> ApplicationStats;

		int64 LastTimeSettingsPolledCycles = 0;
		float RTCStatsPolledDelta = 0.0f;

		TMap<FName, FDebugGraph> Graphs;
		TMap<FName, FCanvasTileItem> Tiles;

		FCriticalSection StatNotificationCS;
	};
} // namespace UE::PixelStreaming
