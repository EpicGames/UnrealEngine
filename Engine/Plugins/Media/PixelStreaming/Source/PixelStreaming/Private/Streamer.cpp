// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streamer.h"
#include "IPixelStreamingModule.h"
#include "PixelStreamingDelegates.h"
#include "PixelStreamingSignallingConnection.h"
#include "PixelStreamingAudioDeviceModule.h"
#include "TextureResource.h"
#include "WebRTCIncludes.h"
#include "WebSocketsModule.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingDataChannel.h"
#include "Settings.h"
#include "AudioSink.h"
#include "Stats.h"
#include "PixelStreamingStatNames.h"
#include "Async/Async.h"
#include "Engine/Texture2D.h"
#include "RTCStatsCollector.h"
#include "Framework/Application/SlateApplication.h"
#include "PixelStreamingCodec.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Dom/JsonObject.h"
#include "PixelStreamingModule.h"
#include "PixelStreamingInputProtocol.h"
#include "IPixelStreamingModule.h"
#include "IPixelStreamingInputModule.h"
#include "SignallingConnectionObserver.h"
#include "ToStringExtensions.h"
#include "Settings.h"
#include "CoreGlobals.h"

namespace UE::PixelStreaming
{
	TSharedPtr<FStreamer> FStreamer::Create(const FString& StreamerId)
	{
		TSharedPtr<FStreamer> Streamer = TSharedPtr<FStreamer>(new FStreamer(StreamerId));
		IPixelStreamingInputModule::Get().OnProtocolUpdated.AddSP(Streamer.ToSharedRef(), &FStreamer::OnProtocolUpdated);

		return Streamer;
	}

	FStreamer::FStreamer(const FString& InStreamerId)
		: StreamerId(InStreamerId)
		, InputHandler(IPixelStreamingInputModule::Get().CreateInputHandler())
		, Players(new TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>())
		, Module(IPixelStreamingModule::Get())
		, FreezeFrame(FFreezeFrame::Create(Players))
	{
		VideoSourceGroup = FVideoSourceGroup::Create();
		Observer = MakeShared<FPixelStreamingSignallingConnectionObserver>(*this);

		SignallingServerConnection = MakeShared<FPixelStreamingSignallingConnection>(Observer, InStreamerId);
		SignallingServerConnection->SetAutoReconnect(true);

		InputHandler->SetElevatedCheck([this](FString PlayerId) {
			return QualityControllingId == INVALID_PLAYER_ID || PlayerId == QualityControllingId;
		});
	}

	FStreamer::~FStreamer()
	{
		StopStreaming();
	}

	void FStreamer::OnProtocolUpdated()
	{
		Players->Apply([this](FPixelStreamingPlayerId DataPlayerId, FPlayerContext& PlayerContext) {
			if (PlayerContext.DataChannel)
			{
				SendProtocol(DataPlayerId);
			}
		});
	}

	void FStreamer::SetStreamFPS(int32 InFramesPerSecond)
	{
		VideoSourceGroup->SetFPS(InFramesPerSecond);
	}

	int32 FStreamer::GetStreamFPS()
	{
		return VideoSourceGroup->GetFPS();
	}

	void FStreamer::SetCoupleFramerate(bool bCouple)
	{
		VideoSourceGroup->SetCoupleFramerate(bCouple);
	}

	void FStreamer::SetVideoInput(TSharedPtr<FPixelStreamingVideoInput> Input)
	{
		VideoSourceGroup->SetVideoInput(Input);
		FreezeFrame->SetVideoInput(Input);

		// Users may update the video input after a player has connected. If this is the case,
		// we need to run through all existing players and update their video source
		Players->Apply([this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
			if (PlayerContext.PeerConnection)
			{
				PlayerContext.PeerConnection->SetVideoSource(VideoSourceGroup->CreateVideoSource(PlayerContext.Config.IsSFU, [this, PlayerId]() { return ShouldPeerGenerateFrames(PlayerId); }));
			}
		});
	}

	TWeakPtr<FPixelStreamingVideoInput> FStreamer::GetVideoInput()
	{
		return VideoSourceGroup->GetVideoInput();
	}

	void FStreamer::SetTargetViewport(TWeakPtr<SViewport> InTargetViewport)
	{
		InputHandler->SetTargetViewport(InTargetViewport);
	}

	TWeakPtr<SViewport> FStreamer::GetTargetViewport()
	{
		return InputHandler ? InputHandler->GetTargetViewport() : nullptr;
	}

	void FStreamer::SetTargetWindow(TWeakPtr<SWindow> InTargetWindow)
	{
		InputHandler->SetTargetWindow(InTargetWindow);
	}

	TWeakPtr<SWindow> FStreamer::GetTargetWindow()
	{
		return InputHandler->GetTargetWindow();
	}

	void FStreamer::SetTargetScreenRect(TWeakPtr<FIntRect> InTargetScreenRect)
	{
		InputHandler->SetTargetScreenRect(InTargetScreenRect);
	}

	TWeakPtr<FIntRect> FStreamer::GetTargetScreenRect()
	{
		return InputHandler->GetTargetScreenRect();
	}

	TWeakPtr<IPixelStreamingSignallingConnection> FStreamer::GetSignallingConnection()
	{
		return SignallingServerConnection;
	}

	void FStreamer::SetSignallingConnection(TSharedPtr<IPixelStreamingSignallingConnection> InSignallingConnection)
	{
		SignallingServerConnection = InSignallingConnection;
	}

	TWeakPtr<IPixelStreamingSignallingConnectionObserver> FStreamer::GetSignallingConnectionObserver()
	{
		return Observer;
	}

	void FStreamer::SetSignallingServerURL(const FString& InSignallingServerURL)
	{
		CurrentSignallingServerURL = InSignallingServerURL;
	}

	FString FStreamer::GetSignallingServerURL()
	{
		return CurrentSignallingServerURL;
	}

	bool FStreamer::IsSignallingConnected()
	{
		return SignallingServerConnection && SignallingServerConnection->IsConnected();
	}

	void FStreamer::StartStreaming()
	{
		if (CurrentSignallingServerURL.IsEmpty())
		{
			UE_LOGF(LogPixelStreaming, Log, "Attempted to start streamer (%ls) but no signalling server URL has been set. Use Streamer->SetSignallingServerURL(URL) or -PixelStreamingURL=", *StreamerId);
			return;
		}

		StopStreaming();

		if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
		{
			ConsumeStatsHandle = Delegates->OnStatChangedNative.AddSP(AsShared(), &FStreamer::ConsumeStats);
			AllConnectionsClosedHandle = Delegates->OnAllConnectionsClosedNative.AddSP(AsShared(), &FStreamer::TriggerMouseLeave);
		}

		// Broadcast the preconnection event just before we do `TryConnect`
		StreamingPreConnectionEvent.Broadcast(this);

		VideoSourceGroup->Start();
		SignallingServerConnection->TryConnect(CurrentSignallingServerURL);
		bStreamingStarted = true;
	}

	void FStreamer::StopStreaming()
	{
		if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
		{
			Delegates->OnStatChangedNative.Remove(ConsumeStatsHandle);
			Delegates->OnAllConnectionsClosedNative.Remove(AllConnectionsClosedHandle);
		}

		if (SignallingServerConnection)
		{
			SignallingServerConnection->Disconnect(TEXT("Streamer stopped streaming"));
		}
		VideoSourceGroup->Stop();
		TriggerMouseLeave(StreamerId);

		if (bStreamingStarted)
		{
			OnStreamingStopped().Broadcast(this);
		}

		DeleteAllPlayerSessions();
		bStreamingStarted = false;
	}

	IPixelStreamingStreamer::FPreConnectionEvent& FStreamer::OnPreConnection()
	{
		return StreamingPreConnectionEvent;
	}

	IPixelStreamingStreamer::FStreamingStartedEvent& FStreamer::OnStreamingStarted()
	{
		return StreamingStartedEvent;
	}

	IPixelStreamingStreamer::FStreamingStoppedEvent& FStreamer::OnStreamingStopped()
	{
		return StreamingStoppedEvent;
	}

	void FStreamer::ForceKeyFrame()
	{
		FPixelStreamingPeerConnection::ForceVideoKeyframe();
	}

	void FStreamer::PushFrame()
	{
		VideoSourceGroup->Tick();
	}

	void FStreamer::FreezeStream(UTexture2D* Texture)
	{
		FreezeFrame->StartFreeze(Texture);
	}

	void FStreamer::UnfreezeStream()
	{
		// Force a keyframe so when stream unfreezes if player has never received a frame before they can still connect.
		ForceKeyFrame();
		FreezeFrame->StopFreeze();
	}

	void FStreamer::SendPlayerMessage(uint8 Type, const FString& Descriptor)
	{
		Players->Apply([&Type, &Descriptor](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
			if (PlayerContext.DataChannel)
			{
				PlayerContext.DataChannel->SendMessage(Type, Descriptor);
			}
		});
	}

	void FStreamer::SendPlayerMessage(FPixelStreamingPlayerId PlayerId, uint8 Type, const FString& Descriptor)
	{
		if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			if (PlayerContext->DataChannel)
			{
				PlayerContext->DataChannel->SendMessage(Type, Descriptor);
			}
		}
	}

	void FStreamer::SendFileData(const TArray64<uint8>& ByteData, FString& MimeType, FString& FileExtension)
	{
		// TODO this should be dispatched as an async task, but because we lock when we visit the data
		// channels it might be a bad idea. At some point it would be good to take a snapshot of the
		// keys in the map when we start, then one by one get the channel and send the data

		Players->Apply([&ByteData, &MimeType, &FileExtension, this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
			if (PlayerContext.DataChannel)
			{
				// Send the mime type first
				PlayerContext.DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("FileMimeType")->GetID(), MimeType);

				// Send the extension next
				PlayerContext.DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("FileExtension")->GetID(), FileExtension);

				// Send the contents of the file. Note to callers: consider running this on its own thread, it can take a while if the file is big.
				if (!PlayerContext.DataChannel->SendArbitraryData(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("FileContents")->GetID(), ByteData))
				{
					UE_LOGF(LogPixelStreaming, Error, "Unable to send file data over the data channel for player %ls.", *PlayerId);
				}
			}
		});
	}

	void FStreamer::KickPlayer(FPixelStreamingPlayerId PlayerId)
	{
		SignallingServerConnection->SendDisconnectPlayer(PlayerId, TEXT("Player was kicked"));
		// TODO Delete player session?
	}

	void FStreamer::SetPlayerLayerPreference(FPixelStreamingPlayerId PlayerId, int SpatialLayerId, int TemporalLayerId)
	{
		TSharedPtr<FJsonObject> LayerJson = MakeShared<FJsonObject>();
		LayerJson->SetStringField(TEXT("type"), TEXT("layerPreference"));
		int32 PlayerIdAsInt = PlayerIdToInt(PlayerId);
		LayerJson->SetNumberField(TEXT("playerId"), PlayerIdAsInt);
		LayerJson->SetNumberField(TEXT("spatialLayer"), SpatialLayerId);
		LayerJson->SetNumberField(TEXT("temporalLayer"), TemporalLayerId);
		SignallingServerConnection->SendMessage(UE::PixelStreaming::ToString(LayerJson, false));
	}

	TArray<FPixelStreamingPlayerId> FStreamer::GetConnectedPlayers()
	{
		TArray<FPixelStreamingPlayerId> ConnectedPlayerIds;
		Players->Apply([&ConnectedPlayerIds, this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
			ConnectedPlayerIds.Add(PlayerId);
		});
		return ConnectedPlayerIds;
	}

	void FStreamer::SetInputHandlerType(EPixelStreamingInputType InputType)
	{
		InputHandler->SetInputType(InputType);
	}

	IPixelStreamingAudioSink* FStreamer::GetPeerAudioSink(FPixelStreamingPlayerId PlayerId)
	{
		if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			return PlayerContext->PeerConnection->GetAudioSink().Get();
		}
		return nullptr;
	}

	IPixelStreamingAudioSink* FStreamer::GetUnlistenedAudioSink()
	{
		IPixelStreamingAudioSink* Result = nullptr;
		Players->ApplyUntil([&Result](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
			if (PlayerContext.PeerConnection)
			{
				if (!PlayerContext.PeerConnection->GetAudioSink()->HasAudioConsumers())
				{
					Result = PlayerContext.PeerConnection->GetAudioSink().Get();
					return true;
				}
			}
			return false;
		});
		return Result;
	}

	TSharedPtr<IPixelStreamingAudioInput> FStreamer::CreateAudioInput()
	{
		return FPixelStreamingPeerConnection::CreateAudioInput();
	}

	void FStreamer::RemoveAudioInput(TSharedPtr<IPixelStreamingAudioInput> AudioInput)
	{
		FPixelStreamingPeerConnection::RemoveAudioInput(AudioInput);
	}

	void FStreamer::SetConfigOption(const FName& OptionName, const FString& Value)
	{
		if (Value.IsEmpty())
		{
			ConfigOptions.Remove(OptionName);
		}
		else
		{
			ConfigOptions.Add(OptionName, Value);
		}
	}

	bool FStreamer::GetConfigOption(const FName& OptionName, FString& OutValue)
	{
		FString* OptionValue = ConfigOptions.Find(OptionName);
		if (OptionValue)
		{
			OutValue = *OptionValue;
			return true;
		}
		else
		{
			return false;
		}
	}

	void FStreamer::AddPlayerConfig(TSharedRef<FJsonObject>& JsonObject)
	{
		checkf(InputHandler.IsValid(), TEXT("No Input Device available when populating Player Config"));
		JsonObject->SetBoolField(TEXT("FakingTouchEvents"), InputHandler->IsFakingTouchEvents());
		FString PixelStreamingControlScheme;
		if (Settings::GetControlScheme(PixelStreamingControlScheme))
		{
			JsonObject->SetStringField(TEXT("ControlScheme"), PixelStreamingControlScheme);
		}
		float PixelStreamingFastPan;
		if (Settings::GetFastPan(PixelStreamingFastPan))
		{
			JsonObject->SetNumberField(TEXT("FastPan"), PixelStreamingFastPan);
		}
	}

	void FStreamer::PlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate)
	{
		if (QualityControllingId == INVALID_PLAYER_ID || PlayerId == QualityControllingId)
		{
			Settings::CVarPixelStreamingWebRTCMinBitrate->Set(MinBitrate, ECVF_SetByCode);
			Settings::CVarPixelStreamingWebRTCMaxBitrate->Set(MaxBitrate, ECVF_SetByCode);
		}
		else
		{
			UE_LOGF(LogPixelStreaming, Warning, "Unable to set bitrates from player %ls since they are not the Quality Controller.", *PlayerId);
		}
	}

	void FStreamer::RefreshStreamBitrate()
	{
		Players->Apply([this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
			if (PlayerContext.PeerConnection)
			{
				PlayerContext.PeerConnection->RefreshStreamBitrate();
			}
		});
	}

	void FStreamer::ForEachPlayer(const TFunction<void(FPixelStreamingPlayerId, FPlayerContext)>& Func)
	{
		Players->Apply(Func);
	}

	bool FStreamer::CreateSession(FPixelStreamingPlayerId PlayerId)
	{
		if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			if (PlayerContext->Config.IsSFU && SFUPlayerId != INVALID_PLAYER_ID)
			{
				UE_LOGF(LogPixelStreaming, Error, "SFU is connecting but we already have an SFU");
			}
			else
			{
				TUniquePtr<FPixelStreamingPeerConnection> NewConnection = FPixelStreamingPeerConnection::Create(PeerConnectionConfig, PlayerContext->Config.IsSFU);

				NewConnection->OnEmitIceCandidate.AddLambda([this, PlayerId](const webrtc::IceCandidateInterface* SDP) {
					SignallingServerConnection->SendIceCandidate(PlayerId, *SDP);
				});

				NewConnection->OnNewDataChannel.AddLambda([this, PlayerId](TSharedPtr<FPixelStreamingDataChannel> NewChannel) {
					AddNewDataChannel(PlayerId, NewChannel);
				});

#if WEBRTC_5414
				NewConnection->SetWebRTCStatsCallback(rtc::scoped_refptr<FRTCStatsCollector>(new FRTCStatsCollector(PlayerId)));
#else
				NewConnection->SetWebRTCStatsCallback(new rtc::RefCountedObject<FRTCStatsCollector>(PlayerId));
#endif

				PlayerContext->PeerConnection = MakeShareable(NewConnection.Release());

				if (PlayerContext->Config.IsSFU)
				{
					SFUPlayerId = PlayerId;
				}
				else if(QualityControllingId == INVALID_PLAYER_ID)
				{
					SetQualityController(PlayerId);
				}

				if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
				{
					Delegates->OnNewConnection.Broadcast(StreamerId, PlayerId, !PlayerContext->Config.IsSFU);
					Delegates->OnNewConnectionNative.Broadcast(StreamerId, PlayerId, !PlayerContext->Config.IsSFU);
				}

				return true;
			}
		}
		return false;
	}

	// Streams need to be added after the remote description when we get an offer to receive.
	void FStreamer::AddStreams(FPixelStreamingPlayerId PlayerId)
	{
		if (VideoSourceGroup->GetVideoInput() != nullptr)
		{
			if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
			{
				const bool AllowSimulcast = PlayerContext->Config.IsSFU;

				PlayerContext->PeerConnection->SetVideoSource(VideoSourceGroup->CreateVideoSource(PlayerContext->Config.IsSFU, [this, PlayerId]() { return ShouldPeerGenerateFrames(PlayerId); }));
				PlayerContext->PeerConnection->SetAudioSource(FPixelStreamingPeerConnection::GetApplicationAudioSource());
				PlayerContext->PeerConnection->SetAudioSink(MakeShared<FAudioSink>());
			}
		}
	}

	void FStreamer::OnPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer)
	{
		if (!bSendOffer)
		{
			// If we're not sending the offer, don't create the player session
			// we'll wait until the offer arrives to do that
			return;
		}

		FPlayerContext& PlayerContext = Players->GetOrAdd(PlayerId);
		PlayerContext.Config = PlayerConfig;

		// create peer connection
		if (CreateSession(PlayerId))
		{
			AddStreams(PlayerId);

			// Create a datachannel, if needed.
			if (PlayerContext.Config.SupportsDataChannel)
			{
				const bool Negotiated = PlayerContext.Config.IsSFU;
				const int Id = PlayerContext.Config.IsSFU ? 1023 : 0;
				TSharedPtr<FPixelStreamingDataChannel> NewChannel = PlayerContext.PeerConnection->CreateDataChannel(Id, Negotiated);
				AddNewDataChannel(PlayerId, NewChannel);
			}

			const FPixelStreamingPeerConnection::EReceiveMediaOption ReceiveOption = PlayerContext.Config.IsSFU
				? FPixelStreamingPeerConnection::EReceiveMediaOption::Nothing
				: FPixelStreamingPeerConnection::EReceiveMediaOption::Audio;

			PlayerContext.PeerConnection->CreateOffer(
				ReceiveOption,
				[this, PlayerId](const webrtc::SessionDescriptionInterface* SDP) {
					// on success
					SignallingServerConnection->SendOffer(PlayerId, *SDP);
				},
				[this, PlayerId](const FString& Error) {
					// on error
					DeletePlayerSession(PlayerId);
				});
		}
	}

	void FStreamer::ConsumeStats(FPixelStreamingPlayerId PlayerId, FName StatName, float StatValue)
	{
		if (StatName == PixelStreamingStatNames::MeanQPPerSecond)
		{
			if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
			{
				if (PlayerContext->DataChannel)
				{
					PlayerContext->DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("VideoEncoderAvgQP")->GetID(), FString::FromInt((int)StatValue));
				}
			}
		}
	}

	void FStreamer::OnOffer(FPixelStreamingPlayerId PlayerId, const FString& Sdp)
	{
		FPlayerContext& PlayerContext = Players->GetOrAdd(PlayerId);

		FPixelStreamingPlayerConfig Config;
		Config.SupportsDataChannel = true;
		Config.IsSFU = false;
		PlayerContext.Config = Config;

		if (CreateSession(PlayerId))
		{
			auto OnGeneralFailure = [this, PlayerId](const FString& Error) {
				DeletePlayerSession(PlayerId);
			};

			PlayerContext.PeerConnection->ReceiveOffer(
				Sdp,
				[this, PlayerContext, PlayerId, OnGeneralFailure]() {
					AddStreams(PlayerId);
					PlayerContext.PeerConnection->CreateAnswer(
						FPixelStreamingPeerConnection::EReceiveMediaOption::Audio,
						[this, PlayerId](const webrtc::SessionDescriptionInterface* LocalDescription) {
							SignallingServerConnection->SendAnswer(PlayerId, *LocalDescription);
							bStreamingStarted = true;
						},
						OnGeneralFailure);
				},
				OnGeneralFailure);
		}
	}

	void FStreamer::OnAnswer(FPixelStreamingPlayerId PlayerId, const FString& Sdp)
	{
		if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			PlayerContext->PeerConnection->ReceiveAnswer(
				Sdp,
				[this]() {
					bStreamingStarted = true;
				},
				[this, PlayerId](const FString& Error) {
					DeletePlayerSession(PlayerId);
				});
		}
		else
		{
			UE_LOGF(LogPixelStreaming, Error, "Failed to find player id %ls for incoming answer.", *PlayerId);
		}
	}

	void FStreamer::DeletePlayerSession(FPixelStreamingPlayerId PlayerId)
	{
		// We dont want to allow this to be deleted within Players.Remove because
		// we lock the players map and the delete could dispatch a webrtc object
		// delete on the signalling thread which might be waiting for the players
		// lock.
		FPlayerContext PendingDeletePlayer;
		if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			// when a sfu is connected we only get disconnect messages.
			// we dont get connect messages but we might get datachannel requests which can result
			// in players with no PeerConnection but a datachannel
			if (PlayerContext->PeerConnection)
			{
				VideoSourceGroup->RemoveVideoSource(PlayerContext->PeerConnection->GetVideoSource().get());
			}
			PendingDeletePlayer = *PlayerContext;
		}

		Players->Remove(PlayerId);

		// delete webrtc objects here outside the lock
		PendingDeletePlayer.DataChannel.Reset();
		PendingDeletePlayer.PeerConnection.Reset();

		if (PlayerId == SFUPlayerId)
		{
			SFUPlayerId = INVALID_PLAYER_ID;
		}

		bool bWasQualityController = PlayerId == QualityControllingId;
		if (bWasQualityController)
		{
			SetQualityController(INVALID_PLAYER_ID);

			// find the first non sfu peer and give it quality controller status
			Players->ApplyUntil([this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
				if (PlayerContext.PeerConnection)
				{
					if (PlayerId != SFUPlayerId)
					{
						SetQualityController(PlayerId);
						return true;
					}
				}
				return false;
			});
		}

		if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
		{
			Delegates->OnClosedConnection.Broadcast(StreamerId, PlayerId, bWasQualityController);
			Delegates->OnClosedConnectionNative.Broadcast(StreamerId, PlayerId, bWasQualityController);
			if (Players->IsEmpty())
			{
				Delegates->OnAllConnectionsClosed.Broadcast(StreamerId);
				Delegates->OnAllConnectionsClosedNative.Broadcast(StreamerId);
			}
		}

		if(FStats* PSStats = FStats::Get())
		{
			PSStats->RemovePeerStats(PlayerId);
		}
	}

	void FStreamer::DeleteAllPlayerSessions()
	{
		if(FStats* PSStats = FStats::Get())
		{
			PSStats->RemoveAllPeerStats();
		}

		VideoSourceGroup->RemoveAllVideoSources();
		Players->Empty();
		SFUPlayerId = INVALID_PLAYER_ID;
		QualityControllingId = INVALID_PLAYER_ID;
		InputControllingId = INVALID_PLAYER_ID;
		if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
		{
			Delegates->OnAllConnectionsClosed.Broadcast(StreamerId);
			Delegates->OnAllConnectionsClosedNative.Broadcast(StreamerId);
		}
	}

	void FStreamer::AddNewDataChannel(FPixelStreamingPlayerId PlayerId, TSharedPtr<FPixelStreamingDataChannel> NewChannel)
	{
		FPlayerContext& PlayerContext = Players->GetOrAdd(PlayerId);
		PlayerContext.DataChannel = NewChannel;

		TWeakPtr<FStreamer> WeakStreamer = AsShared();
		PlayerContext.DataChannel->OnOpen.AddLambda([WeakStreamer, PlayerId](FPixelStreamingDataChannel& Channel) {
			if (TSharedPtr<FStreamer> Streamer = WeakStreamer.Pin())
			{
				Streamer->OnDataChannelOpen(PlayerId);
			}
		});

		PlayerContext.DataChannel->OnClosed.AddLambda([WeakStreamer, PlayerId](FPixelStreamingDataChannel& Channel) {
			if (TSharedPtr<FStreamer> Streamer = WeakStreamer.Pin())
			{
				Streamer->OnDataChannelClosed(PlayerId);
			}
		});

		PlayerContext.DataChannel->OnMessageReceived.AddLambda([WeakStreamer, PlayerId](uint8 Type, const webrtc::DataBuffer& RawBuffer) {
			if (TSharedPtr<FStreamer> Streamer = WeakStreamer.Pin())
			{

				Streamer->OnDataChannelMessage(PlayerId, Type, RawBuffer);
			}
		});
	}

	void FStreamer::OnDataChannelOpen(FPixelStreamingPlayerId PlayerId)
	{
		// Only time we automatically make a new peer the input controlling host is if they are the first peer (and not the SFU).
		bool HostControlsInput = Settings::GetInputControllerMode() == Settings::EInputControllerMode::Host;
		if (HostControlsInput && PlayerId != SFUPlayerId && InputControllingId == INVALID_PLAYER_ID)
		{
			InputControllingId = PlayerId;
		}

		if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
		{
			FPlayerContext* PlayerContext = Players->Find(PlayerId);
			Delegates->OnDataChannelOpenNative.Broadcast(StreamerId, PlayerId, PlayerContext->DataChannel.Get());
		}

		// When data channel is open
		SendProtocol(PlayerId);
		// Try to send cached freeze frame (if we have one)
		FreezeFrame->SendCachedFreezeFrameTo(PlayerId);
		SendInitialSettings(PlayerId);
		SendPeerControllerMessages(PlayerId);
	}

	void FStreamer::OnDataChannelClosed(FPixelStreamingPlayerId PlayerId)
	{
		if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			PlayerContext->DataChannel = nullptr;

			if (InputControllingId == PlayerId)
			{
				InputControllingId = INVALID_PLAYER_ID;
				// just get the first channel we have and give it input control.
				Players->ApplyUntil([this](FPixelStreamingPlayerId PlayerId, FPlayerContext& PlayerContext) {
					if (PlayerContext.DataChannel)
					{
						if (PlayerId != SFUPlayerId)
						{
							InputControllingId = PlayerId;
							return true;
						}
					}
					return false;
				});
			}

			if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
			{
				Delegates->OnDataChannelClosedNative.Broadcast(StreamerId, PlayerId);
			}
		}
	}

	void FStreamer::OnDataChannelMessage(FPixelStreamingPlayerId PlayerId, uint8 Type, const webrtc::DataBuffer& RawBuffer)
	{
		if (Type == FPixelStreamingInputProtocol::ToStreamerProtocol.Find("RequestQualityControl")->GetID())
		{
			HandleSetQualityControllerRequest(PlayerId);
		}
		else if (Type == FPixelStreamingInputProtocol::ToStreamerProtocol.Find("LatencyTest")->GetID())
		{
			SendLatencyReport(PlayerId);
		}
		else if (Type == FPixelStreamingInputProtocol::ToStreamerProtocol.Find("RequestInitialSettings")->GetID())
		{
			SendInitialSettings(PlayerId);
		}
		else if (Type == FPixelStreamingInputProtocol::ToStreamerProtocol.Find("IFrameRequest")->GetID())
		{
			ForceKeyFrame();
		}
		else if (Type == FPixelStreamingInputProtocol::ToStreamerProtocol.Find("TestEcho")->GetID())
		{
			if (FPlayerContext* PlayerContext = Players->Find(PlayerId))
			{
				if (PlayerContext->DataChannel)
				{
					const size_t DescriptorSize = (RawBuffer.data.size() - 1) / sizeof(TCHAR);
					const TCHAR* DescPtr = reinterpret_cast<const TCHAR*>(RawBuffer.data.data() + 1);
					const FString Message = FString::ConstructFromPtrSize(DescPtr, DescriptorSize);
					PlayerContext->DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("TestEcho")->GetID(), Message);
				}
			}
		}
		else if (!IsEngineExitRequested())
		{
			// If we are in "Host" mode and the current peer is not the host, then discard this input.
			if (Settings::GetInputControllerMode() == Settings::EInputControllerMode::Host
				&& InputControllingId != PlayerId)
			{
				return;
			}

			TArray<uint8> MessageData(RawBuffer.data.data(), RawBuffer.data.size());
			OnInputReceived.Broadcast(PlayerId, Type, MessageData);

			if (InputHandler)
			{
				InputHandler->OnMessage(PlayerId, MessageData);
			}
		}
	}

	void FStreamer::SendInitialSettings(FPixelStreamingPlayerId PlayerId) const
	{
		static const IConsoleVariable* PixelStreamingInputAllowConsoleCommands = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.AllowPixelStreamingCommands"));
		const FString PixelStreamingPayload = FString::Printf(TEXT("{ \"AllowPixelStreamingCommands\": %s, \"DisableLatencyTest\": %s }"),
			PixelStreamingInputAllowConsoleCommands->GetBool() ? TEXT("true") : TEXT("false"),
			Settings::CVarPixelStreamingDisableLatencyTester.GetValueOnAnyThread() ? TEXT("true") : TEXT("false"));

		const FString WebRTCPayload = FString::Printf(TEXT("{ \"DegradationPref\": \"%s\", \"FPS\": %d, \"MinBitrate\": %d, \"MaxBitrate\": %d, \"LowQP\": %d, \"HighQP\": %d }"),
			*Settings::CVarPixelStreamingDegradationPreference.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingWebRTCFps.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingWebRTCMinBitrate.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingWebRTCMaxBitrate.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingWebRTCLowQpThreshold.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingWebRTCHighQpThreshold.GetValueOnAnyThread());

		const FString EncoderPayload = FString::Printf(TEXT("{ \"TargetBitrate\": %d, \"MaxBitrate\": %d, \"MinQP\": %d, \"MaxQP\": %d, \"RateControl\": \"%s\", \"FillerData\": %d, \"MultiPass\": \"%s\" }"),
			Settings::CVarPixelStreamingEncoderTargetBitrate.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingEncoderMaxBitrate.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingEncoderMinQP.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingEncoderMaxQP.GetValueOnAnyThread(),
			*Settings::CVarPixelStreamingEncoderRateControl.GetValueOnAnyThread(),
			Settings::CVarPixelStreamingEnableFillerData.GetValueOnAnyThread() ? 1 : 0,
			*Settings::CVarPixelStreamingEncoderMultipass.GetValueOnAnyThread());

		FString ConfigPayload = TEXT("{ ");
		bool bComma = false; // Simplest way to avoid complaints from pedantic JSON parsers
		for (const TPair<FName, FString>& Option : ConfigOptions)
		{
			if (bComma)
			{
				ConfigPayload.Append(TEXT(", "));
			}
			ConfigPayload.Append(FString::Printf(TEXT("\"%s\": \"%s\""), *Option.Key.ToString(), *Option.Value));
			bComma = true;
		}
		ConfigPayload.Append(TEXT("}"));

		const FString FullPayload = FString::Printf(TEXT("{ \"PixelStreaming\": %s, \"Encoder\": %s, \"WebRTC\": %s, \"ConfigOptions\": %s }"), *PixelStreamingPayload, *EncoderPayload, *WebRTCPayload, *ConfigPayload);

		if (const FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			if (PlayerContext->DataChannel)
			{
				if (!PlayerContext->DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("InitialSettings")->GetID(), FullPayload))
				{
					UE_LOGF(LogPixelStreaming, Log, "Failed to send initial Pixel Streaming settings to player %ls.", *PlayerId);
				}
			}
		}
	}

	void FStreamer::SendProtocol(FPixelStreamingPlayerId PlayerId) const
	{
		const TArray<EPixelStreamingMessageDirection> PixelStreamingMessageDirections = { EPixelStreamingMessageDirection::ToStreamer, EPixelStreamingMessageDirection::FromStreamer };
		for (EPixelStreamingMessageDirection MessageDirection : PixelStreamingMessageDirections)
		{
			TSharedPtr<FJsonObject> ProtocolJson = FPixelStreamingInputProtocol::ToJson(MessageDirection);
			FString body;
			TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&body);
			if (!ensure(FJsonSerializer::Serialize(ProtocolJson.ToSharedRef(), JsonWriter)))
			{
				UE_LOGF(LogPixelStreaming, Warning, "Cannot serialize protocol json object");
				return;
			}

			if (const FPlayerContext* PlayerContext = Players->Find(PlayerId))
			{
				if (PlayerContext->DataChannel)
				{
					// Log a warning if we are unable to send our updated protocol
					if (!PlayerContext->DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("Protocol")->GetID(), body))
					{
						UE_LOGF(LogPixelStreaming, Warning, "Failed to send Pixel Streaming protocol to player %ls. This player will use the default protocol specified in the front end", *PlayerId);
					}
				}
			}
		}
	}

	void FStreamer::SendPeerControllerMessages(FPixelStreamingPlayerId PlayerId) const
	{
		if (const FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			if (PlayerContext->DataChannel)
			{
				const uint8 ControlsInput = (Settings::GetInputControllerMode() == Settings::EInputControllerMode::Host) ? (PlayerId == InputControllingId) : 1;
				const uint8 ControlsQuality = PlayerId == QualityControllingId ? 1 : 0;
				PlayerContext->DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("InputControlOwnership")->GetID(), ControlsInput);
				PlayerContext->DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("QualityControlOwnership")->GetID(), ControlsQuality);
			}
		}
	}

	void FStreamer::SendLatencyReport(FPixelStreamingPlayerId PlayerId) const
	{
		if (Settings::CVarPixelStreamingDisableLatencyTester.GetValueOnAnyThread())
		{
			return;
		}

		double ReceiptTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

		AsyncTask(ENamedThreads::GameThread, [this, PlayerId, ReceiptTimeMs]() {
			FString ReportToTransmitJSON;

			if (!Settings::CVarPixelStreamingWebRTCDisableStats.GetValueOnAnyThread())
			{
				double EncodeMs = -1.0;
				double CaptureToSendMs = 0.0;

				FStats* Stats = FStats::Get();
				if (Stats)
				{
					Stats->QueryPeerStat(PlayerId, RTCStatTypes::OutboundRTP, PixelStreamingStatNames::MeanEncodeTime, EncodeMs);
					Stats->QueryPeerStat(PlayerId, RTCStatTypes::OutboundRTP, PixelStreamingStatNames::AvgSendDelay, CaptureToSendMs);
				}

				double TransmissionTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
				ReportToTransmitJSON = FString::Printf(
					TEXT("{ \"ReceiptTimeMs\": %.2f, \"EncodeMs\": %.2f, \"CaptureToSendMs\": %.2f, \"TransmissionTimeMs\": %.2f }"),
					ReceiptTimeMs,
					EncodeMs,
					CaptureToSendMs,
					TransmissionTimeMs);
			}
			else
			{
				double TransmissionTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
				ReportToTransmitJSON = FString::Printf(
					TEXT("{ \"ReceiptTimeMs\": %.2f, \"EncodeMs\": \"Pixel Streaming stats are disabled\", \"CaptureToSendMs\": \"Pixel Streaming stats are disabled\", \"TransmissionTimeMs\": %.2f }"),
					ReceiptTimeMs,
					TransmissionTimeMs);
			}

			if (const FPlayerContext* PlayerContext = Players->Find(PlayerId))
			{
				if (PlayerContext->DataChannel)
				{
					PlayerContext->DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("LatencyTest")->GetID(), ReportToTransmitJSON);
				}
			}
		});
	}

	bool FStreamer::ShouldPeerGenerateFrames(FPixelStreamingPlayerId PlayerId) const
	{
		if (const FPlayerContext* PlayerContext = Players->Find(PlayerId))
		{
			EPixelStreamingCodec Codec = PlayerContext->PeerConnection->GetNegotiatedVideoCodec();

			// This could happen if SDP is malformed, oddly ordered, or if offer is made before codecs are registered
			// In any case, we should fall back to our default setting.
			if (Codec == EPixelStreamingCodec::Invalid)
			{
				UE_LOGF(LogPixelStreaming, Log, "Was unable to extract a negotiated video codec from the SDP falling back to -PixelStreamingEncoderCodec");
				Codec = Settings::GetSelectedCodec();
			}

			// Connected peer is invalid, we should not encode anything
			if (PlayerId == INVALID_PLAYER_ID)
			{
				return false;
			}

			switch (Codec)
			{
				case EPixelStreamingCodec::H264:
				case EPixelStreamingCodec::AV1:
					return (PlayerId == QualityControllingId || PlayerId == SFUPlayerId);
				case EPixelStreamingCodec::VP8:
				case EPixelStreamingCodec::VP9:
					return true;
				default:
					// There should be a case for every Codec type, so this should never happen.
					checkNoEntry();
					break;
			}
		}
		return false;
	}

	void FStreamer::HandleSetQualityControllerRequest(const FPixelStreamingPlayerId& PlayerId)
	{
		const Settings::EQualityControllerMode QualityControlMode = static_cast<Settings::EQualityControllerMode>(
			Settings::CVarPixelStreamingQualityControllerMode.GetValueOnAnyThread()
			);
		UE_LOGF(LogPixelStreaming, Verbose, "Player %ls has requested quality control through the data channel.", *PlayerId);
		
		switch (QualityControlMode)
		{
		case Settings::GrantAllRequested: SetQualityController(PlayerId); break;
		case Settings::GrantOnlyFirst:
			if (QualityControllingId == INVALID_PLAYER_ID || QualityControllingId == PlayerId)
			{
				SetQualityController(PlayerId);
			}
			else
			{
				UE_LOGF(LogPixelStreaming, Verbose, "Dropped quality control request from player %ls.", *PlayerId);
			}
			break;
		}
	}

	void FStreamer::SetQualityController(FPixelStreamingPlayerId PlayerId)
	{
		QualityControllingId = PlayerId;
		Players->Apply([this](FPixelStreamingPlayerId DataPlayerId, FPlayerContext& PlayerContext) {
			if (PlayerContext.DataChannel)
			{
				const uint8 IsController = DataPlayerId == QualityControllingId ? 1 : 0;
				PlayerContext.DataChannel->SendMessage(FPixelStreamingInputProtocol::FromStreamerProtocol.Find("QualityControlOwnership")->GetID(), IsController);
			}
		});
		if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
		{
			Delegates->OnQualityControllerChangedNative.Broadcast(StreamerId, QualityControllingId);
		}
	}

	void FStreamer::TriggerMouseLeave(FString InStreamerId)
	{
		if (!IsEngineExitRequested() && StreamerId == InStreamerId)
		{
			// Force a MouseLeave event. This prevents the PixelStreamingApplicationWrapper from
			// still wrapping the base FSlateApplication after we stop streaming
			const auto MouseLeaveFunction = [this]() 
			{
				TArray<uint8> EmptyArray;
				TFunction<void(FString, FMemoryReader)> MouseLeaveHandler = InputHandler->FindMessageHandler("MouseLeave");	
				MouseLeaveHandler("", FMemoryReader(EmptyArray));
			};

			if(IsInGameThread())
			{
				MouseLeaveFunction();
			}
			else
			{
				TWeakPtr<FStreamer> WeakStreamer = AsShared();
				AsyncTask(ENamedThreads::GameThread, [this, MouseLeaveFunction, WeakStreamer]() {
					if (!WeakStreamer.IsValid())
					{
						return;
					}

					MouseLeaveFunction();
				});
			}
		}
	}
} // namespace UE::PixelStreaming
