// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingStreamer.h"
#include "IPixelStreamingSignallingConnectionObserver.h"
#include "PixelStreamingPeerConnection.h"
#include "VideoSourceGroup.h"
#include "ThreadSafeMap.h"
#include "Dom/JsonObject.h"
#include "IPixelStreamingInputHandler.h"
#include "PixelStreamingSignallingConnection.h"
#include "Templates/SharedPointer.h"
#include "PlayerContext.h"
#include "FreezeFrame.h"

class IPixelStreamingModule;

namespace UE::PixelStreaming
{
	class FStreamer : public IPixelStreamingStreamer, public TSharedFromThis<FStreamer>
	{
		friend class FPixelStreamingSignallingConnectionObserver;

	public:
		static TSharedPtr<FStreamer> Create(const FString& StreamerId);
		virtual ~FStreamer();

		virtual void SetStreamFPS(int32 InFramesPerSecond) override;
		virtual int32 GetStreamFPS() override;
		virtual void SetCoupleFramerate(bool bCouple) override;

		virtual void SetVideoInput(TSharedPtr<FPixelStreamingVideoInput> Input) override;
		virtual TWeakPtr<FPixelStreamingVideoInput> GetVideoInput() override;
		virtual void SetTargetViewport(TWeakPtr<SViewport> InTargetViewport) override;
		virtual TWeakPtr<SViewport> GetTargetViewport() override;
		virtual void SetTargetWindow(TWeakPtr<SWindow> InTargetWindow) override;
		virtual TWeakPtr<SWindow> GetTargetWindow() override;
		virtual void SetTargetScreenRect(TWeakPtr<FIntRect> InTargetScreenRect) override;
		virtual TWeakPtr<FIntRect> GetTargetScreenRect() override;

		virtual TWeakPtr<IPixelStreamingSignallingConnection> GetSignallingConnection() override;
		virtual void SetSignallingConnection(TSharedPtr<IPixelStreamingSignallingConnection> InSignallingConnection) override;
		virtual TWeakPtr<IPixelStreamingSignallingConnectionObserver> GetSignallingConnectionObserver() override;
		virtual void SetSignallingServerURL(const FString& InSignallingServerURL) override;
		virtual FString GetSignallingServerURL() override;
		virtual FString GetId() override { return StreamerId; };
		virtual bool IsSignallingConnected() override;
		virtual void StartStreaming() override;
		virtual void StopStreaming() override;
		virtual bool IsStreaming() const override { return bStreamingStarted; }

		virtual FPreConnectionEvent& OnPreConnection() override;
		virtual FStreamingStartedEvent& OnStreamingStarted() override;
		virtual FStreamingStoppedEvent& OnStreamingStopped() override;

		virtual void ForceKeyFrame() override;
		void PushFrame();

		virtual void FreezeStream(UTexture2D* Texture) override;
		virtual void UnfreezeStream() override;

		virtual void SendPlayerMessage(uint8 Type, const FString& Descriptor) override;
		virtual void SendPlayerMessage(FPixelStreamingPlayerId PlayerId, uint8 Type, const FString& Descriptor) override;
		virtual void SendFileData(const TArray64<uint8>& ByteData, FString& MimeType, FString& FileExtension) override;
		virtual void KickPlayer(FPixelStreamingPlayerId PlayerId) override;
		virtual void SetPlayerLayerPreference(FPixelStreamingPlayerId PlayerId, int SpatialLayerId, int TemporalLayerId) override;
		virtual TArray<FPixelStreamingPlayerId> GetConnectedPlayers() override;

		virtual void SetInputHandler(TSharedPtr<IPixelStreamingInputHandler> InInputHandler) override {	InputHandler = InInputHandler; }
		virtual TWeakPtr<IPixelStreamingInputHandler> GetInputHandler() override { return InputHandler; }
		virtual void SetInputHandlerType(EPixelStreamingInputType InputType) override;

		virtual IPixelStreamingAudioSink* GetPeerAudioSink(FPixelStreamingPlayerId PlayerId) override;
		virtual IPixelStreamingAudioSink* GetUnlistenedAudioSink() override;
		TSharedPtr<IPixelStreamingAudioInput> CreateAudioInput() override;
		void RemoveAudioInput(TSharedPtr<IPixelStreamingAudioInput> AudioInput) override;

		virtual void SetConfigOption(const FName& OptionName, const FString& Value) override;
		virtual bool GetConfigOption(const FName& OptionName, FString& OutValue) override;

		// TODO(Luke) hook this back up so that the Engine can change how the interface is working browser side
		void AddPlayerConfig(TSharedRef<FJsonObject>& JsonObject);

		virtual void PlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate) override;

		virtual void RefreshStreamBitrate() override;

		void ForEachPlayer(const TFunction<void(FPixelStreamingPlayerId, FPlayerContext)>& Func);

	private:
		FStreamer(const FString& StreamerId);

		bool CreateSession(FPixelStreamingPlayerId PlayerId);
		void AddStreams(FPixelStreamingPlayerId PlayerId);

		// own methods
		void OnProtocolUpdated();
		void ConsumeStats(FPixelStreamingPlayerId PlayerId, FName StatName, float StatValue);
		void OnOffer(FPixelStreamingPlayerId PlayerId, const FString& Sdp);
		void OnAnswer(FPixelStreamingPlayerId PlayerId, const FString& Sdp);
		void OnPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer);
		void DeletePlayerSession(FPixelStreamingPlayerId PlayerId);
		void DeleteAllPlayerSessions();
		void AddNewDataChannel(FPixelStreamingPlayerId PlayerId, TSharedPtr<FPixelStreamingDataChannel> NewChannel);
		void OnDataChannelOpen(FPixelStreamingPlayerId PlayerId);
		void OnDataChannelClosed(FPixelStreamingPlayerId PlayerId);
		void OnDataChannelMessage(FPixelStreamingPlayerId PlayerId, uint8 Type, const webrtc::DataBuffer& RawBuffer);
		void SendInitialSettings(FPixelStreamingPlayerId PlayerId) const;
		void SendProtocol(FPixelStreamingPlayerId PlayerId) const;
		void SendPeerControllerMessages(FPixelStreamingPlayerId PlayerId) const;
		void SendLatencyReport(FPixelStreamingPlayerId PlayerId) const;
		bool ShouldPeerGenerateFrames(FPixelStreamingPlayerId PlayerId) const;

		void HandleSetQualityControllerRequest(const FPixelStreamingPlayerId& PlayerId);
		void SetQualityController(FPixelStreamingPlayerId PlayerId);
		void TriggerMouseLeave(FString InStreamerId);

	private:
		FString StreamerId;
		FString CurrentSignallingServerURL;

		TSharedPtr<IPixelStreamingInputHandler> InputHandler;
		TSharedPtr<IPixelStreamingSignallingConnection> SignallingServerConnection;
		TSharedPtr<IPixelStreamingSignallingConnectionObserver> Observer;

		double LastSignallingServerConnectionAttemptTimestamp = 0;

		webrtc::PeerConnectionInterface::RTCConfiguration PeerConnectionConfig;

		TSharedPtr<TThreadSafeMap<FPixelStreamingPlayerId, FPlayerContext>> Players;

		FPixelStreamingPlayerId QualityControllingId = INVALID_PLAYER_ID;
		FPixelStreamingPlayerId SFUPlayerId = INVALID_PLAYER_ID;
		FPixelStreamingPlayerId InputControllingId = INVALID_PLAYER_ID;

		bool bStreamingStarted = false;

		FPreConnectionEvent StreamingPreConnectionEvent;
		FStreamingStartedEvent StreamingStartedEvent;
		FStreamingStoppedEvent StreamingStoppedEvent;

		TUniquePtr<webrtc::SessionDescriptionInterface> SFULocalDescription;
		TUniquePtr<webrtc::SessionDescriptionInterface> SFURemoteDescription;

		TSharedPtr<FVideoSourceGroup> VideoSourceGroup;

		FDelegateHandle ConsumeStatsHandle;
		FDelegateHandle AllConnectionsClosedHandle;

		IPixelStreamingModule& Module;

		TSharedPtr<FFreezeFrame> FreezeFrame;

		TMap<FName, FString> ConfigOptions;
	};
} // namespace UE::PixelStreaming
