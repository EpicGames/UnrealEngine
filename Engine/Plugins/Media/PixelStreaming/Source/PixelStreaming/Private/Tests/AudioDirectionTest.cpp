// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Misc/AutomationTest.h"
#include "TestUtils.h"
#include "DirectionUtils.h"

namespace UE::PixelStreaming
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioSendRecvTest, "System.Plugins.PixelStreaming.FAudioSendRecvTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FAudioSendRecvTest::RunTest(const FString& Parameters)
	{
        SetDisableTransmitAudio(false);
		SetDisableReceiveAudio(false);
		DoDirectionTest(cricket::MediaType::MEDIA_TYPE_AUDIO, webrtc::RtpTransceiverDirection::kSendRecv);

        return true;
	}

    IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioSendOnlyTest, "System.Plugins.PixelStreaming.FAudioSendOnlyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FAudioSendOnlyTest::RunTest(const FString& Parameters)
	{
        SetDisableTransmitAudio(false);
		SetDisableReceiveAudio(true);
		DoDirectionTest(cricket::MediaType::MEDIA_TYPE_AUDIO, webrtc::RtpTransceiverDirection::kSendOnly);

        return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioRecvOnlyTest, "System.Plugins.PixelStreaming.FAudioRecvOnlyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FAudioRecvOnlyTest::RunTest(const FString& Parameters)
	{
        SetDisableTransmitAudio(true);
		SetDisableReceiveAudio(false);
		DoDirectionTest(cricket::MediaType::MEDIA_TYPE_AUDIO, webrtc::RtpTransceiverDirection::kRecvOnly);

        return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioInactiveTest, "System.Plugins.PixelStreaming.FAudioInactiveTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FAudioInactiveTest::RunTest(const FString& Parameters)
	{
        SetDisableTransmitAudio(true);
		SetDisableReceiveAudio(true);
		DoDirectionTest(cricket::MediaType::MEDIA_TYPE_AUDIO, webrtc::RtpTransceiverDirection::kInactive);

        return true;
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS