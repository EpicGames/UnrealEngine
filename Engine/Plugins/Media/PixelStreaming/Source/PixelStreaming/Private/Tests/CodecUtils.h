// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"
#include "TestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	using namespace UE::PixelStreaming;

	void DoFrameReceiveTest()
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		FMockVideoFrameConfig FrameConfig = { 128 /*Width*/, 128 /*Height*/, 255 /*Y*/, 137 /*U*/, 216 /*V*/ };

		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		TSharedPtr<IPixelStreamingStreamer> Streamer = CreateStreamer(StreamerPort);
		TSharedPtr<FPixelStreamingVideoInputI420> VideoInput = MakeShared<FPixelStreamingVideoInputI420>();
		Streamer->SetVideoInput(VideoInput);

		TSharedPtr<FMockPlayer> Player = CreatePlayer(FMockPlayer::EMode::AcceptOffers);
		TSharedPtr<FMockVideoSink> VideoSink = MakeShared<FMockVideoSink>();
		Player->VideoSink = VideoSink;

		Streamer->StartStreaming();

		ADD_LATENT_AUTOMATION_COMMAND(FConnectPlayerAfterStreamerConnectedOrTimeout(5.0, Streamer, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForPlayerConnectedOrTimeout(5.0, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForICEConnectedOrTimeout(5.0, Player))

		// Send 30 frames
		for(int i = 0; i < 30; i++)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FSendSolidColorFrame(VideoInput, FrameConfig))
			ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(0.033)) // send at 30fps interval
		}

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForFrameReceived(5.0, VideoSink, FrameConfig))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAll(SignallingServer, Streamer, Player))
	}

	void DoFrameResizeMultipleTimesTest()
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		TSharedPtr<IPixelStreamingStreamer> Streamer = CreateStreamer(StreamerPort);
		TSharedPtr<FPixelStreamingVideoInputI420> VideoInput = MakeShared<FPixelStreamingVideoInputI420>();
		Streamer->SetVideoInput(VideoInput);

		TSharedPtr<FMockPlayer> Player = CreatePlayer(FMockPlayer::EMode::AcceptOffers);
		TSharedPtr<FMockVideoSink> VideoSink = MakeShared<FMockVideoSink>();
		Player->VideoSink = VideoSink;

		// Note: Important to couple framerate as we are manually passing frames and don't want any cached frames
		Streamer->SetCoupleFramerate(true);
		Streamer->StartStreaming();

		ADD_LATENT_AUTOMATION_COMMAND(FConnectPlayerAfterStreamerConnectedOrTimeout(5.0, Streamer, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForPlayerConnectedOrTimeout(5.0, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForICEConnectedOrTimeout(5.0, Player))

		for (int Res = 2; Res < 512; Res *= 2)
		{
			FMockVideoFrameConfig FrameConfig = { Res /*Width*/, Res /*Height*/, 255 /*Y*/, 0 /*U*/, 255 /*V*/ };

			// Send 30 frames
			for(int i = 0; i < 30; i++)
			{
				ADD_LATENT_AUTOMATION_COMMAND(FSendSolidColorFrame(VideoInput, FrameConfig))
				ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(0.033)) // send at 30fps interval
			}

			ADD_LATENT_AUTOMATION_COMMAND(FWaitForFrameReceived(5.0, VideoSink, FrameConfig))
		}

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAll(SignallingServer, Streamer, Player))
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS
