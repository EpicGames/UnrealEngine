// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"
#include "TestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	using namespace UE::PixelStreaming;

	void DoDirectionTest(cricket::MediaType ExpectedMediaType, webrtc::RtpTransceiverDirection ExpectedDirection)
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		TSharedPtr<IPixelStreamingStreamer> Streamer = CreateStreamer(StreamerPort);
		TSharedPtr<FPixelStreamingVideoInputI420> VideoInput = MakeShared<FPixelStreamingVideoInputI420>();
		Streamer->SetVideoInput(VideoInput);

		TSharedPtr<FMockPlayer> Player = CreatePlayer(FMockPlayer::EMode::AcceptOffers);
		TSharedPtr<FMockVideoSink> VideoSink = MakeShared<FMockVideoSink>();
		Player->VideoSink = VideoSink;

		Streamer->StartStreaming();

		const TFunction<bool(rtc::scoped_refptr<webrtc::RtpTransceiverInterface>)> CheckFunc = [ExpectedMediaType, ExpectedDirection](rtc::scoped_refptr<webrtc::RtpTransceiverInterface> Transceiver) {
			return ((Transceiver->media_type() == ExpectedMediaType) && (Transceiver->direction() == ExpectedDirection));
		};

		ADD_LATENT_AUTOMATION_COMMAND(FConnectPlayerAfterStreamerConnectedOrTimeout(5.0, Streamer, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForPlayerConnectedOrTimeout(5.0, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForICEConnectedOrTimeout(5.0, Player))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckTransceivers(Streamer, CheckFunc))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAll(SignallingServer, Streamer, Player))
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS
