// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "TestUtils.h"
#include "IPixelStreamingStreamer.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	void DoHandShake(FAutomationTestBase* ParentTest, FMockPlayer::EMode PlayerOfferMode, bool bUseLegacySignallingServer)
	{
		// need to be able to accept codec to handshake otherwise setting local description fails when generating an answer
		SetCodec(EPixelStreamingCodec::VP8, false /*bUseComputeShader*/);

		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreamingServers::IServer> OutSignallingServer = bUseLegacySignallingServer ? CreateLegacySignallingServer(StreamerPort, PlayerPort) : CreateSignallingServer(StreamerPort, PlayerPort);
		TSharedPtr<IPixelStreamingStreamer> OutStreamer = CreateStreamer(StreamerPort);
		TSharedPtr<FMockPlayer> OutPlayer = CreatePlayer(PlayerOfferMode);

		OutPlayer->OnConnectionEstablished.AddLambda([ParentTest, OutPlayer]() {
			ParentTest->TestTrue(TEXT("Expected streamer and peer to establish RTC connection when streamer offered first."), OutPlayer->Completed);
		});

		OutStreamer->StartStreaming();

		ADD_LATENT_AUTOMATION_COMMAND(FConnectPlayerAfterStreamerConnectedOrTimeout(5.0, OutStreamer, OutPlayer, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForPlayerConnectedOrTimeout(5.0, OutPlayer, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForICEConnectedOrTimeout(5.0, OutPlayer))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAll(OutSignallingServer, OutStreamer, OutPlayer))
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHandshakeTestStreamerOffer, "System.Plugins.PixelStreaming.HandshakeStreamerOffer", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FHandshakeTestStreamerOffer::RunTest(const FString& Parameters)
	{
		DoHandShake(this, FMockPlayer::EMode::AcceptOffers, false /*bUseLegacySignallingServer*/);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHandshakeTestPlayerOffer, "System.Plugins.PixelStreaming.HandshakePlayerOffer", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FHandshakeTestPlayerOffer::RunTest(const FString& Parameters)
	{
		DoHandShake(this, FMockPlayer::EMode::CreateOffers, false /*bUseLegacySignallingServer*/);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLegacyHandshakeTest, "System.Plugins.PixelStreaming.LegacyHandshake", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FLegacyHandshakeTest::RunTest(const FString& Parameters)
	{
		// Note: This test represents compatibility with the signalling protocol that existed in UE 4.2X versions of Pixel Streaming
		// As long as this test passes we can be somewhat confident Pixel Streaming is backwards compatible with those old signalling servers.
		DoHandShake(this, FMockPlayer::EMode::CreateOffers, true /*bUseLegacySignallingServer*/);
		return true;
	}

} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS
