// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPixelStreamingSignallingConnectionObserver.h"
#include "Settings.h"
#include "Utils.h"
#include "PixelStreamingPrivate.h"
#include "ToStringExtensions.h"
#include "WebSocketsModule.h"
#include "HAL/ThreadSafeBool.h"
#include "PixelStreamingDataChannel.h"
#include "WebRTCIncludes.h"
#include "PixelStreamingPeerConnection.h"
#include "PixelStreamingSignallingConnection.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	struct FMockVideoFrameConfig
	{
		int Height;
		int Width;
		uint8 Y;
		uint8 U;
		uint8 V;
	};

	class FMockVideoSink : public rtc::VideoSinkInterface<webrtc::VideoFrame>
	{
	public:
		bool HasReceivedFrame() const { return bReceivedFrame; }
		void ResetReceivedFrame()
		{
			bReceivedFrame = false;
			ReceivedBuffer = nullptr;
			ReceivedHeight = 0;
			ReceivedWidth = 0;
		}
		rtc::scoped_refptr<webrtc::VideoFrameBuffer> GetReceivedBuffer() { return ReceivedBuffer; }
		int GetReceivedWidth() { return ReceivedWidth; }
		int GetReceivedHeight() { return ReceivedHeight; }

	protected:
		virtual void OnFrame(const webrtc::VideoFrame& Frame) override
		{
			if (!bReceivedFrame)
			{
				bReceivedFrame = true;
				ReceivedBuffer = Frame.video_frame_buffer();
				ReceivedWidth = Frame.width();
				ReceivedHeight = Frame.height();

				OnFrameReceived.Broadcast(Frame);
			}
			return;
		}

	public:
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnFrameReceived, const webrtc::VideoFrame&);
		FOnFrameReceived OnFrameReceived;

	private:
		FThreadSafeBool bReceivedFrame = false;
		rtc::scoped_refptr<webrtc::VideoFrameBuffer> ReceivedBuffer;
		int ReceivedHeight;
		int ReceivedWidth;
	};

	class FMockPlayer : public IPixelStreamingSignallingConnectionObserver
	{
	public:
		virtual ~FMockPlayer()
		{
			Disconnect(TEXT("Mock player being destroyed"));

			UE::PixelStreaming::DoOnGameThread([]() {
				Settings::CVarPixelStreamingSuppressICECandidateErrors->Set(false, ECVF_SetByCode);
			});
		}

		enum class EMode
		{
			Unknown,
			AcceptOffers,
			CreateOffers,
		};

		void SetMode(EMode InMode) { Mode = InMode; }

		void Connect(int Port)
		{
			FString Url = FString::Printf(TEXT("ws://127.0.0.1:%d"), Port);
			if(Mode == EMode::CreateOffers)
			{
				Url += "/?OfferToReceive=true";
			}
			SignallingServerConnection->TryConnect(Url);
		}

		void Disconnect(FString Reason)
		{
			SignallingServerConnection->Disconnect(Reason);
		}

		bool IsSignallingConnected()
		{
			return SignallingServerConnection->IsConnected();
		}

		virtual void OnSignallingConnected() override
		{
		}

		virtual void OnSignallingDisconnected(int32 StatusCode, const FString& Reason, bool bWasClean) override
		{
		}

		virtual void OnSignallingError(const FString& ErrorMsg) override
		{
		}

		virtual void OnSignallingConfig(const webrtc::PeerConnectionInterface::RTCConfiguration& Config) override
		{
			PeerConnection = FPixelStreamingPeerConnection::Create(Config);

			PeerConnection->OnEmitIceCandidate.AddLambda([this](const webrtc::IceCandidateInterface* Candidate) {
				SignallingServerConnection->SendIceCandidate(*Candidate);
			});

			PeerConnection->OnNewDataChannel.AddLambda([this](TSharedPtr<FPixelStreamingDataChannel> NewChannel) {
				UE_LOGF(LogPixelStreaming, Log, "Player OnNewDataChannel");
				DataChannel = NewChannel;

				DataChannel->OnMessageReceived.AddLambda([this](uint8 Type, const webrtc::DataBuffer& RawBuffer) {
					OnMessageReceived.Broadcast(Type, RawBuffer);
				});

				HaveDataChannel = true;
			});

			PeerConnection->OnIceStateChanged.AddLambda([this](webrtc::PeerConnectionInterface::IceConnectionState NewState) {
				UE_LOGF(LogPixelStreaming, Log, "Player OnIceStateChanged: %ls", UE::PixelStreaming::ToString(NewState));
				if (NewState == webrtc::PeerConnectionInterface::IceConnectionState::kIceConnectionConnected)
				{
					Completed = true;
					OnConnectionEstablished.Broadcast();
				}
			});

			if (VideoSink)
			{
				PeerConnection->SetVideoSink(VideoSink.Get());
			}

			if (Mode == EMode::CreateOffers)
			{
				PeerConnection->CreateOffer(
					FPixelStreamingPeerConnection::EReceiveMediaOption::All,
					[this](const webrtc::SessionDescriptionInterface* SDP) {
						SignallingServerConnection->SendOffer(*SDP);
					},
					[](const FString& Error) {
					});
			}
			else if (Mode == EMode::AcceptOffers)
			{
				SignallingServerConnection->RequestStreamerList();
			}
		}

		virtual void OnSignallingSessionDescription(webrtc::SdpType Type, const FString& Sdp) override
		{
			if (Type == webrtc::SdpType::kOffer && Mode == EMode::AcceptOffers)
			{
				const auto OnFailure = [](const FString& Error) {
					// fail
				};
				const auto OnSuccess = [this, &OnFailure]() {
					const auto OnSuccess = [this](const webrtc::SessionDescriptionInterface* Sdp) {
						SignallingServerConnection->SendAnswer(*Sdp);
					};
					PeerConnection->CreateAnswer(FPixelStreamingPeerConnection::EReceiveMediaOption::All, OnSuccess, OnFailure);
				};
				PeerConnection->ReceiveOffer(Sdp, OnSuccess, OnFailure);
			}
			else if (Type == webrtc::SdpType::kAnswer && Mode == EMode::CreateOffers)
			{
				const auto OnFailure = [](const FString& Error) {
					// fail
				};
				const auto OnSuccess = []() {
					// nothing
				};
				PeerConnection->ReceiveAnswer(Sdp, OnSuccess, OnFailure);
			}
		}

		virtual void OnSignallingRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) override
		{
			PeerConnection->AddRemoteIceCandidate(SdpMid, SdpMLineIndex, Sdp);
		}

		virtual void OnSignallingPlayerCount(uint32 Count) override {}
		virtual void OnSignallingPeerDataChannels(int32 SendStreamId, int32 RecvStreamId) override {}
		virtual void OnSignallingSessionDescription(FPixelStreamingPlayerId PlayerId, webrtc::SdpType Type, const FString& Sdp) override {}
		virtual void OnSignallingRemoteIceCandidate(FPixelStreamingPlayerId PlayerId, const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) override {}
		virtual void OnSignallingPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer) override {}
		virtual void OnSignallingPlayerDisconnected(FPixelStreamingPlayerId PlayerId) override {}
		virtual void OnSignallingSFUPeerDataChannels(FPixelStreamingPlayerId SFUId, FPixelStreamingPlayerId PlayerId, int32 SendStreamId, int32 RecvStreamId) override {}
		virtual void OnSignallingStreamerList(const TArray<FString>& StreamerList) override 
		{
			if(StreamerList.Num() != 1)
			{
				UE_LOGF(LogPixelStreaming, Error, "Invalid number streamers connected (%d). Expected just one!", StreamerList.Num());
			}

			SignallingServerConnection->SendSubscribe(StreamerList[0]);
		}
		virtual void OnPlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate) override {}

		DECLARE_MULTICAST_DELEGATE(FOnConnectionEstablished);
		FOnConnectionEstablished OnConnectionEstablished;

		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMessageReceived, uint8, const webrtc::DataBuffer&);
		FOnMessageReceived OnMessageReceived;

		EMode Mode = EMode::Unknown;
		TUniquePtr<IPixelStreamingSignallingConnection> SignallingServerConnection;
		TUniquePtr<FPixelStreamingPeerConnection> PeerConnection;
		TSharedPtr<FPixelStreamingDataChannel> DataChannel;
		bool Completed = false;
		bool HaveDataChannel = false;
		TSharedPtr<rtc::VideoSinkInterface<webrtc::VideoFrame>> VideoSink;
	};

} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS
