// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "PixelStreamingServers.h"
#include "Misc/Paths.h"
#include "IPixelStreamingStreamer.h"
#include "IPixelStreamingSignallingConnection.h"
#include "PixelStreamingSignallingConnection.h"
#include "PixelStreamingVideoInputBackBuffer.h"
#include "IPixelStreamingModule.h"
#include "TestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FWaitForSignalling, double, TimeoutSeconds, TSharedPtr<IPixelStreamingStreamer>, OutStreamer, bool, bIsExpectedSignalling);
	bool FWaitForSignalling::Update()
	{
		if (!OutStreamer.IsValid())
		{
			UE_LOGF(LogPixelStreaming, Error, "Streamer not found");
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for streamer to dis/connect to signalling server.");
			return true;
		}

		if (bIsExpectedSignalling && OutStreamer->IsSignallingConnected())
		{
			UE_LOGF(LogPixelStreaming, Log, "Signalling Connected as expected");
			return true;
		}
		else if (!bIsExpectedSignalling && !OutStreamer->IsSignallingConnected())
		{
			UE_LOGF(LogPixelStreaming, Log, "Signalling Disconnected as expected");
			return true;
		}
		return false;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitForServerOrTimeout, TSharedPtr<UE::PixelStreamingServers::IServer>, Server);
	bool FWaitForServerOrTimeout::Update()
	{
		return Server->IsTimedOut() || Server->IsReady();
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDisconnectStreamer, TSharedPtr<IPixelStreamingStreamer>, Streamer);
	bool FDisconnectStreamer::Update()
	{
		Streamer->StopStreaming();
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCleanupServer, TSharedPtr<UE::PixelStreamingServers::IServer>, Server);
	bool FCleanupServer::Update()
	{
		Server->Stop();
		Server.Reset();
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCleanupStreamer, TSharedPtr<IPixelStreamingStreamer>, Streamer);
	bool FCleanupStreamer::Update()
	{
		Streamer->StopStreaming();
		Streamer.Reset();
		return true;
	}

	class FCheckNumConnected: public IAutomationLatentCommand
	{
		public:
			FCheckNumConnected(double InTimeoutSeconds,
					TSharedPtr<UE::PixelStreamingServers::IServer> InSignallingServer,
					uint16 InNumExpected)
				: TimeoutSeconds(InTimeoutSeconds)
				, SignallingServer(InSignallingServer)
				, NumExpected(InNumExpected)
				, bRequestedNumStreamers(false)
				, bHasNumStreamers(false)
				, NumStreamers(0)
			{}
			virtual ~FCheckNumConnected() = default;
			virtual bool Update() override;

		private:
			double TimeoutSeconds;
			TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer;
			uint16 NumExpected;
			bool bRequestedNumStreamers;
			bool bHasNumStreamers;
			uint16 NumStreamers;
	};

	bool FCheckNumConnected::Update()
	{
		if (!bRequestedNumStreamers)
		{
			bRequestedNumStreamers = true;
			SignallingServer->GetNumStreamers([this](uint16 InNumStreamers){
				NumStreamers = InNumStreamers;
				bHasNumStreamers = true;
			});
		}

		if (bHasNumStreamers)
		{
			if (NumStreamers == NumExpected)
			{
				UE_LOGF(LogPixelStreaming, Log, "Expected %d streamers and found %d", NumExpected, NumStreamers);
				return true;
			}
			else
			{
				bRequestedNumStreamers = false;
			}
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			if (bHasNumStreamers)
			{
				UE_LOGF(LogPixelStreaming, Error, "Expected %d streamers but found %d", NumExpected, NumStreamers);
			}
			else
			{
				UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for number of streamers to be retrieved.");
			}
			return true;
		}

		return false;
	}

	TSharedPtr<IPixelStreamingStreamer> CreateStreamer(FString StreamerName, int StreamerPort)
	{
		TSharedPtr<IPixelStreamingStreamer> OutStreamer = IPixelStreamingModule::Get().CreateStreamer(StreamerName);
		OutStreamer->SetVideoInput(FPixelStreamingVideoInputBackBuffer::Create());
		OutStreamer->SetSignallingServerURL(FString::Printf(TEXT("ws://127.0.0.1:%d"), StreamerPort));

		TSharedPtr<IPixelStreamingSignallingConnection> SignallingConnection = MakeShared<FPixelStreamingSignallingConnection>(OutStreamer->GetSignallingConnectionObserver().Pin(), StreamerName);
		SignallingConnection->SetAutoReconnect(true);
		OutStreamer->SetSignallingConnection(SignallingConnection);
		return OutStreamer;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMultipleSignallingConnectionsTest, "System.Plugins.PixelStreaming.MultipleSignallingConnectionsTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FMultipleSignallingConnectionsTest::RunTest(const FString& Parameters)
	{
		UE_LOGF(LogPixelStreaming, Log, "----------- ConnectAndDisconnectMultipleStreamersEmbeddedCirrus -----------");

		int32 StreamerPort = TestUtils::NextStreamerPort();
		const int HttpPort = 85;

		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = UE::PixelStreamingServers::MakeSignallingServer();
		UE::PixelStreamingServers::FLaunchArgs LaunchArgs;
		LaunchArgs.bPollUntilReady = true;
		LaunchArgs.ReconnectionTimeoutSeconds = 30.0f;
		LaunchArgs.ReconnectionIntervalSeconds = 2.0f;
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--HttpPort=%d --StreamerPort=%d"), HttpPort, StreamerPort);

		SignallingServer->OnReady.AddLambda([this](TMap<UE::PixelStreamingServers::EEndpoint, FURL> Endpoints) {
			TestTrue("Got server OnReady.", true);
		});

		SignallingServer->OnFailedToReady.AddLambda([this]() {
			TestTrue("Server was not ready.", false);
		});

		bool bLaunched = SignallingServer->Launch(LaunchArgs);
		UE_LOGF(LogPixelStreaming, Log, "Embedded cirrus launched: %ls", bLaunched ? TEXT("true") : TEXT("false"));
		TestTrue("Embedded cirrus launched.", bLaunched);

		if(!bLaunched)
		{
			return false;
		}

		// make streamer and connect to signalling server websocket
		FString StreamerName1 = FString("Streamer1");
		FString StreamerName2 = FString("Streamer2");
		TSharedPtr<IPixelStreamingStreamer> Streamer1 = CreateStreamer(StreamerName1, StreamerPort);
		TSharedPtr<IPixelStreamingStreamer> Streamer2 = CreateStreamer(StreamerName2, StreamerPort);
		Streamer1->StartStreaming();
		Streamer2->StartStreaming();

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForServerOrTimeout(SignallingServer));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForSignalling(5.0f, Streamer1, true));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForSignalling(5.0f, Streamer2, true));
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumConnected(5.0f, SignallingServer, 2));

		ADD_LATENT_AUTOMATION_COMMAND(FDisconnectStreamer(Streamer1));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForSignalling(5.0f, Streamer1, false));
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumConnected(5.0f, SignallingServer, 1));

		ADD_LATENT_AUTOMATION_COMMAND(FDisconnectStreamer(Streamer2));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForSignalling(5.0f, Streamer2, false));
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumConnected(5.0f, SignallingServer, 0));

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupStreamer(Streamer1));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupStreamer(Streamer2));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupServer(SignallingServer));

		return true;
	}
} // UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS
