// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "TestUtils.h"
#include "PixelStreamingVideoInputBackBuffer.h"
#include "WebRTCIncludes.h"
#include "IPixelStreamingModule.h"
#include "IPixelStreamingStreamer.h"
#include "IPixelStreamingInputModule.h"
#include "PixelStreamingInputEnums.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FProtocolTestAddMessage, "System.Plugins.PixelStreaming.FProtocolTestAddMessage", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FProtocolTestAddMessage::RunTest(const FString& Parameters)
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		TSharedPtr<IPixelStreamingStreamer> Streamer = CreateStreamer(StreamerPort);
		TSharedPtr<FPixelStreamingVideoInputI420> VideoInput = MakeShared<FPixelStreamingVideoInputI420>();
		Streamer->SetVideoInput(VideoInput);

		// Define our message
		FPixelStreamingInputMessage Message = FPixelStreamingInputMessage({ EPixelStreamingMessageTypes::Uint16 } /* Structure */);
		// Add it to the protocol
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("CustomMessage", Message);
		// Define a handler function
		const TFunction<void(FString, FMemoryReader)> Handler = [this](FString, FMemoryReader Ar) { /* Do nothing */ };
		// Add it to the streamer
		Streamer->GetInputHandler().Pin()->RegisterMessageHandler("CustomMessage", Handler);

		TSharedPtr<FMockPlayer> Player = CreatePlayer(FMockPlayer::EMode::AcceptOffers);
		TSharedPtr<FMockVideoSink> VideoSink = MakeShared<FMockVideoSink>();
		Player->VideoSink = VideoSink;

		TSharedPtr<bool> bComplete = MakeShared<bool>(false);
		TFunction<void(uint8, const webrtc::DataBuffer&)> Callback = [bComplete](uint8 Type, const webrtc::DataBuffer& RawBuffer) {
			if (Type == 255)
			{
				const size_t DescriptorSize = (RawBuffer.data.size() - 1) / sizeof(TCHAR);
				const TCHAR* DescPtr = reinterpret_cast<const TCHAR*>(RawBuffer.data.data() + 1);
				const FString JsonRaw = FString::ConstructFromPtrSize(DescPtr, DescriptorSize);

				TSharedPtr<FJsonObject> JsonParsed;
				TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonRaw);
				if (FJsonSerializer::Deserialize(JsonReader, JsonParsed))
				{
					double Direction = JsonParsed->GetNumberField(TEXT("Direction"));
					if (!(Direction == static_cast<double>(EPixelStreamingMessageDirection::ToStreamer)))
					{
						return;
					}

					if (JsonParsed->HasField(TEXT("CustomMessage")))
					{
						*bComplete.Get() = true;
					}
					else
					{
						UE_LOGF(LogPixelStreaming, Error, "Expected custom message definition to be in the received protocol.");
					}
				}
			}
		};

		Streamer->StartStreaming();

		ADD_LATENT_AUTOMATION_COMMAND(FConnectPlayerAfterStreamerConnectedOrTimeout(5.0, Streamer, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForPlayerConnectedOrTimeout(5.0, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForICEConnectedOrTimeout(5.0, Player))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelMessageOrTimeout(15.0, Player, Callback, bComplete))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAll(SignallingServer, Streamer, Player))
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FProtocolTestUseCustomMessage, "System.Plugins.PixelStreaming.FProtocolTestUseCustomMessage", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FProtocolTestUseCustomMessage::RunTest(const FString& Parameters)
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		TSharedPtr<IPixelStreamingStreamer> Streamer = CreateStreamer(StreamerPort);
		TSharedPtr<FPixelStreamingVideoInputI420> VideoInput = MakeShared<FPixelStreamingVideoInputI420>();
		Streamer->SetVideoInput(VideoInput);

		// Define our message
		FPixelStreamingInputMessage Message = FPixelStreamingInputMessage({ EPixelStreamingMessageTypes::Uint16 } /* Structure */);
		// Add it to the protocol
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("CustomMessage", Message);
		// Define a handler function
		TSharedPtr<bool> bComplete = MakeShared<bool>(false);
		const TFunction<void(FString, FMemoryReader)> Handler = [this, bComplete](FString, FMemoryReader Ar) {
			*bComplete.Get() = true;
			uint16 Out;
			Ar << Out;
			TestTrue(TEXT("Expected message content to be 1337."), Out == 1337);
		};
		// Add it to the streamer
		Streamer->GetInputHandler().Pin()->RegisterMessageHandler("CustomMessage", Handler);

		
		TFunction<void(uint8, const webrtc::DataBuffer&)> Callback = [](uint8 Type, const webrtc::DataBuffer& RawBuffer) { /* Do nothing */ };


		TSharedPtr<FMockPlayer> Player = CreatePlayer(FMockPlayer::EMode::AcceptOffers);
		TSharedPtr<FMockVideoSink> VideoSink = MakeShared<FMockVideoSink>();
		Player->VideoSink = VideoSink;

		Streamer->StartStreaming();

		ADD_LATENT_AUTOMATION_COMMAND(FConnectPlayerAfterStreamerConnectedOrTimeout(5.0, Streamer, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForPlayerConnectedOrTimeout(5.0, Player, PlayerPort))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelOrTimeout(5.0, Player))
		ADD_LATENT_AUTOMATION_COMMAND(FSendCustomMessageToStreamer(Player, Message.GetID(), 1337))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelMessageOrTimeout(15.0, Player, Callback, bComplete))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAll(SignallingServer, Streamer, Player))
		return true;
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS