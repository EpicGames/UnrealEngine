// Copyright Epic Games, Inc. All Rights Reserved.
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "PixelStreamingSignallingConnection.h"
#include "Serialization/JsonSerializer.h"
#include "IWebSocket.h"
#include "Utils.h"
#include "ToStringExtensions.h"
#include "Dom/JsonObject.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMockPixelStreamingSS, Log, VeryVerbose);
DEFINE_LOG_CATEGORY(LogMockPixelStreamingSS);

namespace
{
	// using namespace UE::PixelStreaming;

	// For validating callbacks from the signalling server connection
	class FMockConnectionObserver : public IPixelStreamingSignallingConnectionObserver
	{
	public:
		virtual ~FMockConnectionObserver() = default;

		virtual void OnSignallingConnected() override {}
		virtual void OnSignallingDisconnected(int32 StatusCode, const FString& Reason, bool bWasClean) override {}
		virtual void OnSignallingError(const FString& ErrorMsg) override {}

		virtual void OnSignallingConfig(const webrtc::PeerConnectionInterface::RTCConfiguration& Config) override { SetConfig = Config; }

		//// Streamer-only
		virtual void OnSignallingSessionDescription(FPixelStreamingPlayerId PlayerId, webrtc::SdpType Type, const FString& Sdp) override
		{
			SetSDP = { Type, Sdp, PlayerId };
		}
		virtual void OnSignallingRemoteIceCandidate(FPixelStreamingPlayerId PlayerId, const FString& SdpMid, int SdpMLineIndex, const FString& Sdp)
		{
			SetIceCandidate = { SdpMid, SdpMLineIndex, Sdp, PlayerId };
		}
		virtual void OnSignallingPlayerConnected(FPixelStreamingPlayerId, const FPixelStreamingPlayerConfig&, bool bSendOffer) override {}
		virtual void OnSignallingPlayerDisconnected(FPixelStreamingPlayerId PlayerId) override {}
		virtual void OnSignallingSFUPeerDataChannels(FPixelStreamingPlayerId SFUId, FPixelStreamingPlayerId PlayerId, int32 SendStreamId, int32 RecvStreamId) override {}
		virtual void OnPlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate) override {}

		//// Player-only
		virtual void OnSignallingSessionDescription(webrtc::SdpType Type, const FString& Sdp) override
		{
			SetSDP = { Type, Sdp };
		}
		virtual void OnSignallingRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp)
		{
			SetIceCandidate = { SdpMid, SdpMLineIndex, Sdp };
		}
		virtual void OnSignallingPlayerCount(uint32 Count) override {}
		virtual void OnSignallingPeerDataChannels(int32 SendStreamId, int32 RecvStreamId) override {}
		virtual void OnSignallingStreamerList(const TArray<FString>& StreamerList) override {}

		void Reset()
		{
			SetConfig.Reset();
			SetSDP.Reset();
			SetIceCandidate.Reset();
		}

		TOptional<webrtc::PeerConnectionInterface::RTCConfiguration> SetConfig;

		struct FSessionDescriptionData
		{
			webrtc::SdpType Type;
			FString SDP;
			TOptional<FPixelStreamingPlayerId> PlayerId;
		};
		TOptional<FSessionDescriptionData> SetSDP;

		struct FIceCandidateData
		{
			FString SdpMid;
			int SdpMLineIndex;
			FString SDP;
			TOptional<FPixelStreamingPlayerId> PlayerId;
		};
		TOptional<FIceCandidateData> SetIceCandidate;
	};

	// For faking a web socket connection
	class FMockWebSocket : public IWebSocket
	{
	public:
		FMockWebSocket() = default;
		virtual ~FMockWebSocket() = default;
		virtual void Connect() override
		{
			bConnected = true;
			OnConnectedEvent.Broadcast();
		}
		virtual void Close(int32 Code = 1000, const FString& Reason = FString()) override { bConnected = false; }
		virtual bool IsConnected() override { return bConnected; }
		virtual void Send(const FString& Data) override { OnMessageSentEvent.Broadcast(Data); }
		virtual void Send(const void* Data, SIZE_T Size, bool bIsBinary = false) override {}
		virtual void SetTextMessageMemoryLimit(uint64 TextMessageMemoryLimit) override {}
		virtual FWebSocketConnectedEvent& OnConnected() override { return OnConnectedEvent; }
		virtual FWebSocketConnectionErrorEvent& OnConnectionError() override { return OnErrorEvent; }
		virtual FWebSocketClosedEvent& OnClosed() override { return OnClosedEvent; }
		virtual FWebSocketMessageEvent& OnMessage() override { return OnMessageEvent; }
		virtual FWebSocketBinaryMessageEvent& OnBinaryMessage() override { return OnBinaryMessageEvent; }
		virtual FWebSocketRawMessageEvent& OnRawMessage() override { return OnRawMessageEvent; }
		virtual FWebSocketMessageSentEvent& OnMessageSent() override { return OnMessageSentEvent; }

	private:
		FWebSocketConnectedEvent OnConnectedEvent;
		FWebSocketConnectionErrorEvent OnErrorEvent;
		FWebSocketClosedEvent OnClosedEvent;
		FWebSocketMessageEvent OnMessageEvent;
		FWebSocketBinaryMessageEvent OnBinaryMessageEvent;
		FWebSocketRawMessageEvent OnRawMessageEvent;
		FWebSocketMessageSentEvent OnMessageSentEvent;

		bool bConnected = false;
	};

	class FMockSessionDescriptionInterface : public webrtc::SessionDescriptionInterface
	{
	public:
		virtual ~FMockSessionDescriptionInterface() = default;
		virtual cricket::SessionDescription* description() override { return nullptr; }
		virtual const cricket::SessionDescription* description() const override { return nullptr; }
		virtual std::string session_id() const override { return ""; }
		virtual std::string session_version() const override { return ""; }
		virtual std::string type() const override { return ""; }
		virtual bool AddCandidate(const webrtc::IceCandidateInterface* candidate) override { return true; }
		virtual size_t RemoveCandidates(const std::vector<cricket::Candidate>& candidates) override { return 0; }
		virtual size_t number_of_mediasections() const override { return 0; }
		virtual const webrtc::IceCandidateCollection* candidates(size_t mediasection_index) const override { return nullptr; }
		virtual bool ToString(std::string* out) const override { return true; }
	};

	class FMockIceCandidateInterface : public webrtc::IceCandidateInterface
	{
	public:
		virtual ~FMockIceCandidateInterface() = default;
		virtual std::string sdp_mid() const override { return ""; }
		virtual int sdp_mline_index() const override { return 0; }
		virtual const cricket::Candidate& candidate() const override { return MockCandidate; }
		virtual std::string server_url() const override { return ""; }
		virtual bool ToString(std::string* out) const override { return true; }

	private:
		cricket::Candidate MockCandidate;
	};

	BEGIN_DEFINE_SPEC(FSignallingServerConnectionSpec, "System.Plugins.PixelStreaming.SignallingServerConnection", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	TUniquePtr<FPixelStreamingSignallingConnection> SignallingServerConnection; // the object being tested
	TSharedPtr<FMockConnectionObserver> Observer;
	TSharedPtr<FMockWebSocket> WebSocket;
	FDelegateHandle OnMessageDelegateHandle; // so we can remove the delegate after tests
	TSharedPtr<FJsonObject> LastJsonMsg;	 // the parsed JsonMsg of the last message received from the SignallingServerConnection
	const FString ServerID = "MOCK SERVER";
	const FPixelStreamingPlayerId MockPlayerId = "MOCK";
	const FMockSessionDescriptionInterface MockSDP;
	const FMockIceCandidateInterface MockCandidate;
	void OnMessageReceived(const FString& Response);
	END_DEFINE_SPEC(FSignallingServerConnectionSpec)
	void FSignallingServerConnectionSpec::Define()
	{
		Describe("Mocked connection tests", [this]() {
			BeforeEach([this]() {
				Observer = MakeShared<FMockConnectionObserver>();
				WebSocket = MakeShared<FMockWebSocket>();

				SignallingServerConnection = MakeUnique<FPixelStreamingSignallingConnection>(Observer, ServerID, WebSocket);
				SignallingServerConnection->TryConnect("Fake Url");
				OnMessageDelegateHandle = WebSocket->OnMessageSent().AddLambda([this](const FString& message) {
					OnMessageReceived(message);
				});
			});

			AfterEach([this]() {
				LastJsonMsg.Reset();
				WebSocket->OnMessageSent().Remove(OnMessageDelegateHandle);
				SignallingServerConnection.Reset();
			});

			It("should be connected", [this]() 
			{
				TestTrue("Connected", SignallingServerConnection->IsConnected());
			});

			It("should be able to send an offer", [this]() {
				// these calls should probably set some data on these mock objects
				// and then check that the received data is correct, like the mock
				// sends below
				SignallingServerConnection->SendOffer(MockPlayerId, MockSDP);
			});

			It("should be able to send an answer with no player it", [this]() {
				SignallingServerConnection->SendAnswer(MockSDP);
			});

			It("should be able to send an answer with a player id", [this]() {
				SignallingServerConnection->SendAnswer(MockPlayerId, MockSDP);
			});

			It("should be able to send an ice candidate with no player id", [this]() {
				SignallingServerConnection->SendIceCandidate(MockCandidate);
			});

			It("should be able to send an ice candidate with a player id", [this]() {
				SignallingServerConnection->SendIceCandidate(MockPlayerId, MockCandidate);
			});

			It("should be able to send a player disconnect message", [this]() {
				SignallingServerConnection->SendDisconnectPlayer(MockPlayerId, "No Reason");
			});

			It("should be able to send a custom message", [this]() {
				SignallingServerConnection->SendMessage(R"({ "type": "custom message"})");
			});

			It("should reply to an identity request", [this]() {
				// Simulate the signalling server sending this message to the application
				WebSocket->OnMessage().Broadcast(R"({"type" : "identify"})");
				TestEqual("Type", LastJsonMsg->GetStringField(TEXT("type")), "endpointId");
				TestEqual("Id", LastJsonMsg->GetStringField(TEXT("id")), ServerID);
			});

			It("should receive configurations", [this]() {
				// Simulate the signalling server sending this message to the application
				WebSocket->OnMessage().Broadcast(R"({ "type": "config", "peerConnectionOptions": {} })");
				TestTrue("Config IsSet", Observer->SetConfig.IsSet());
			});

			It("should be able to receive offers with no player id", [this]() {
				// Simulate the signalling server sending this message to the application
				WebSocket->OnMessage().Broadcast(R"({ "type": "offer", "sdp": "sdp goes here" })");
				if (TestTrue("SDP IsSet", Observer->SetSDP.IsSet()))
				{
					TestEqual("Offer type", Observer->SetSDP.GetValue().Type, webrtc::SdpType::kOffer);
					TestFalse("Player id IsSet", Observer->SetSDP.GetValue().PlayerId.IsSet());
					TestEqual("SDP value", Observer->SetSDP.GetValue().SDP, "sdp goes here");
				}
			});

			It("should be able to receive offers with a player id", [this]() {
				// Simulate the signalling server sending this message to the application
				WebSocket->OnMessage().Broadcast(R"({ "type": "offer", "playerId": "101", "sdp": "sdp goes here" })");
				if (TestTrue("SDP IsSet", Observer->SetSDP.IsSet()))
				{
					TestEqual("Offer type", Observer->SetSDP.GetValue().Type, webrtc::SdpType::kOffer);
					if (TestTrue("Player id IsSet", Observer->SetSDP.GetValue().PlayerId.IsSet()))
					{
						TestEqual("Player id", Observer->SetSDP.GetValue().PlayerId.GetValue(), "101");
					}
					TestEqual("SDP value", Observer->SetSDP.GetValue().SDP, "sdp goes here");
				}
			});

			It("should be able to receive an answer", [this]() {
				// Simulate the signalling server sending this message to the application
				WebSocket->OnMessage().Broadcast(R"({ "type": "answer", "playerId": "102", "sdp": "sdp goes here" })");
				if (TestTrue("SDP IsSet", Observer->SetSDP.IsSet()))
				{
					TestEqual("Answer type", Observer->SetSDP.GetValue().Type, webrtc::SdpType::kAnswer);
					TestEqual("SDP value", Observer->SetSDP.GetValue().SDP, "sdp goes here");
					if (TestTrue("Player id IsSet", Observer->SetSDP.GetValue().PlayerId.IsSet()))
					{
						TestEqual("Player id", Observer->SetSDP.GetValue().PlayerId.GetValue(), "102");
					}
				}
			});

			It("should be able to receive an ice candidate with no player id", [this]() {
				// Simulate the signalling server sending this message to the application
				WebSocket->OnMessage().Broadcast(R"({ "type": "iceCandidate", "candidate": { "sdpMid": "mid", "sdpMlineIndex": 10, "candidate": "sdp string" } })");
				if (TestTrue("Ice Candidate IsSet", Observer->SetIceCandidate.IsSet()))
				{
					TestEqual("sdp mid", Observer->SetIceCandidate.GetValue().SdpMid, "mid");
					TestEqual("sdp mline index", Observer->SetIceCandidate.GetValue().SdpMLineIndex, 10);
					TestEqual("sdp value", Observer->SetIceCandidate.GetValue().SDP, "sdp string");
					TestFalse("PlayerId IsSet", Observer->SetIceCandidate.GetValue().PlayerId.IsSet());
				}
			});

			It("should be able to receive an ice candidate with a player id", [this]() {
				// Simulate the signalling server sending this message to the application
				WebSocket->OnMessage().Broadcast(R"({ "type": "iceCandidate", "playerId": "103", "candidate": { "sdpMid": "mid", "sdpMlineIndex": 10, "candidate": "sdp string" } })");
				if (TestTrue("Ice Candidate IsSet", Observer->SetIceCandidate.IsSet()))
				{
					TestEqual("sdp mid", Observer->SetIceCandidate.GetValue().SdpMid, "mid");
					TestEqual("sdp mline index", Observer->SetIceCandidate.GetValue().SdpMLineIndex, 10);
					TestEqual("sdp value", Observer->SetIceCandidate.GetValue().SDP, "sdp string");
					if (TestTrue("Player id IsSet", Observer->SetIceCandidate.GetValue().PlayerId.IsSet()))
					{
						TestEqual("Player id", Observer->SetIceCandidate.GetValue().PlayerId.GetValue(), "103");
					}
				}
			});
		});
	}

	// used to store the parsed JsonMsg response from the signalling server connection
	void FSignallingServerConnectionSpec::OnMessageReceived(const FString& Response)
	{
		const auto JsonReader = TJsonReaderFactory<TCHAR>::Create(Response);
		TestTrue("Deserialised JsonMsg", FJsonSerializer::Deserialize(JsonReader, LastJsonMsg));
	}
} // namespace
