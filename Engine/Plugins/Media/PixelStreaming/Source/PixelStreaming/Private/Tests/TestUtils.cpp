// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestUtils.h"
#include "PixelStreamingVideoInputBackBuffer.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "PixelStreamingPrivate.h"
#include "IPixelStreamingModule.h"
#include "PixelCaptureInputFrameI420.h"
#include "SocketUtils.h"
#include "Streamer.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	int32 TestUtils::NextStreamerPort()
	{
		// Start of IANA un-registerable ports (49152 - 65535)
		static int NextStreamerPort = 49152;
		NextStreamerPort = GetNextAvailablePort(NextStreamerPort);
		return NextStreamerPort++;
	}

	int32 TestUtils::NextPlayerPort()
	{
		// Half of IANA un-registerable ports (49152 - 65535)
		static int NextPlayerPort = 57344;
		NextPlayerPort = GetNextAvailablePort(NextPlayerPort);
		return NextPlayerPort++;
	}

	/* ---------- Latent Automation Commands ----------- */

	bool FWaitSeconds::Update()
	{
		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > WaitSeconds)
		{
			return true;
		}
		return false;
	}

	bool FSendSolidColorFrame::Update()
	{
		TSharedPtr<FPixelCaptureBufferI420> Buffer = MakeShared<FPixelCaptureBufferI420>(FrameConfig.Width, FrameConfig.Height);

		uint8_t* yData = Buffer->GetMutableDataY();
		uint8_t* uData = Buffer->GetMutableDataU();
		uint8_t* vData = Buffer->GetMutableDataV();
		for (int y = 0; y < Buffer->GetHeight(); ++y)
		{
			for (int x = 0; x < Buffer->GetWidth(); ++x)
			{
				const int x2 = x / 2;
				const int y2 = y / 2;

				yData[x + (y * Buffer->GetStrideY())] = FrameConfig.Y;
				uData[x2 + (y2 * Buffer->GetStrideUV())] = FrameConfig.U;
				vData[x2 + (y2 * Buffer->GetStrideUV())] = FrameConfig.V;
			}
		}

		FPixelCaptureInputFrameI420 Frame(Buffer);
		VideoInput->OnFrame(Frame);
		return true;
	}

	bool FSendDataChannelMessageToPlayer::Update()
	{
		Streamer->SendPlayerMessage(Id, Body);
		return true;
	}

	bool FSendDataChannelMessageToStreamer::Update()
	{
		UE_LOGF(LogPixelStreaming, Log, "SendDataChannelMessageToStreamer: %d, %ls", Id, *Body);
		if (!Player->DataChannel)
		{
			UE_LOGF(LogPixelStreaming, Error, "No DataChannel on player.");
		}
		else
		{
			Player->DataChannel->SendMessage(Id, Body);
		}
		return true;
	}

	bool FSendCustomMessageToStreamer::Update()
	{
		UE_LOGF(LogPixelStreaming, Log, "SendDataChannelMessageToStreamer: %d, %d", Id, Body);
		if (!Player->DataChannel)
		{
			UE_LOGF(LogPixelStreaming, Error, "No DataChannel on player.");
		}
		else
		{
			Player->DataChannel->SendMessage(Id, Body);
		}
		return true;
	}

	bool FWaitForFrameReceived::Update()
	{
		if (VideoSink && VideoSink->HasReceivedFrame())
		{
			UE_LOGF(LogPixelStreaming, Log, "Successfully received streamed frame.");

			/* ----- Test resolution of frame is what we expect ---- */

			FString WidthTestString = FString::Printf(TEXT("Expected frame res=%dx%d, actual res=%dx%d"),
				FrameConfig.Width,
				FrameConfig.Height,
				VideoSink->GetReceivedWidth(),
				VideoSink->GetReceivedHeight());

			if (FrameConfig.Width != VideoSink->GetReceivedWidth() || FrameConfig.Height != VideoSink->GetReceivedHeight())
			{
				UE_LOGF(LogPixelStreaming, Error, "%ls", *WidthTestString);
			}
			else
			{
				UE_LOGF(LogPixelStreaming, Log, "%ls", *WidthTestString);
			}

			/* ----- Test the pixels of the received frame ---- */

			rtc::scoped_refptr<webrtc::VideoFrameBuffer> Buffer = VideoSink->GetReceivedBuffer();
			const webrtc::I420BufferInterface* I420Buffer = Buffer->GetI420();
			// Due this frame being a single solid color we "should" only need to look at a single element.

			FString PixelTestString = FString::Printf(TEXT("Expected solid color frame.| Expect: Y=%d, Actual: Y=%d | Expected: U=%d, Actual: U=%d | Expected: V=%d, Actual: V=%d"),
				FrameConfig.Y,
				*I420Buffer->DataY(),
				FrameConfig.U,
				*I420Buffer->DataU(),
				FrameConfig.V,
				*I420Buffer->DataV());

			const int YDelta = FMath::Max(FrameConfig.Y, *I420Buffer->DataY()) - FMath::Min(FrameConfig.Y, *I420Buffer->DataY());
			const int UDelta = FMath::Max(FrameConfig.U, *I420Buffer->DataU()) - FMath::Min(FrameConfig.U, *I420Buffer->DataU());
			const int VDelta = FMath::Max(FrameConfig.V, *I420Buffer->DataV()) - FMath::Min(FrameConfig.V, *I420Buffer->DataV());
			const int Tolerance = 5;

			// Match pixel values within a tolerance as compression can result in color variations, but not much as this is a solid color.
			if (YDelta > Tolerance || UDelta > Tolerance || VDelta > Tolerance)
			{
				UE_LOGF(LogPixelStreaming, Error, "%ls", *PixelTestString);
			}
			else
			{
				UE_LOGF(LogPixelStreaming, Log, "%ls", *PixelTestString);
			}

			// So we can use this sink for this test again if we want.
			VideoSink->ResetReceivedFrame();

			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming, Error, "Timed out waiting to receive a frame of video through the video sink.");
			return true;
		}
		return false;
	}

	bool FWaitForICEConnectedOrTimeout::Update()
	{
		// If no signalling we can early exit.
		if (!OutPlayer->IsSignallingConnected())
		{
			UE_LOGF(LogPixelStreaming, Error, "Early exiting waiting for ICE connected as player is not connected to signalling server.");
			return true;
		}

		if (OutPlayer)
		{
			double DeltaTime = FPlatformTime::Seconds() - StartTime;
			if (DeltaTime > TimeoutSeconds)
			{
				UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for RTC connection between streamer and player.");
				return true;
			}
			return OutPlayer->Completed;
		}
		return true;
	}

	bool FWaitForDataChannelOrTimeout::Update()
	{
		// If no signalling we can early exit.
		if (!OutPlayer->IsSignallingConnected())
		{
			UE_LOGF(LogPixelStreaming, Error, "Early exiting waiting for data channel as player is not connected to signalling server.");
			return true;
		}

		if (OutPlayer)
		{
			double DeltaTime = FPlatformTime::Seconds() - StartTime;
			if (DeltaTime > TimeoutSeconds)
			{
				UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for data channel between streamer and player.");
				return true;
			}
			return OutPlayer->HaveDataChannel;
		}
		return true;
	}

	bool FWaitForStreamerConnectedOrTimeout::Update()
	{
		if (OutStreamer->IsSignallingConnected())
		{
			UE_LOGF(LogPixelStreaming, Log, "Streamer connected to signalling server.");
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for streamer to connect to signalling server.");
			return true;
		}
		return false;
	}

	bool FConnectPlayerAfterStreamerConnectedOrTimeout::Update()
	{
		if (OutStreamer->IsSignallingConnected())
		{
			UE_LOGF(LogPixelStreaming, Log, "Streamer connected to signalling server. Attempting to connect player...");
			OutPlayer->Connect(PlayerPort);
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for streamer to connect to signalling server.");
			return true;
		}
		return false;
	}

	bool FWaitForPlayerConnectedOrTimeout::Update()
	{
		if (OutPlayer->IsSignallingConnected())
		{
			UE_LOGF(LogPixelStreaming, Log, "Player connected to signalling server.");
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for player to connect to signalling server.");
			return true;
		}
		return false;
	}

	bool FWaitForDataChannelMessageOrTimeout::Update()
	{
		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming, Error, "Timed out waiting for a datachannel message.");
			return true;
		}
		return *bComplete.Get();
	}

	bool FCheckTransceivers::Update()
	{
		if(!Streamer)
		{
			UE_LOGF(LogPixelStreaming, Error, "Streamer is invalid!");
			return true;
		}

        FStreamer* StreamerPtr = (FStreamer*)Streamer.Get();

        bool bTransceiverExists = false;
        const TFunction<void(rtc::scoped_refptr<webrtc::RtpTransceiverInterface>)> TranceiverFunc = [&bTransceiverExists = bTransceiverExists, this](rtc::scoped_refptr<webrtc::RtpTransceiverInterface> Transceiver) 
        {
            bTransceiverExists |= CheckFunc(Transceiver);
		};

        const TFunction<void(FPixelStreamingPlayerId, FPlayerContext)> PlayerFunc = [&TranceiverFunc = TranceiverFunc, this](FPixelStreamingPlayerId PlayerId, FPlayerContext PlayerContext) 
        {
            PlayerContext.PeerConnection->ForEachTransceiver(TranceiverFunc);
        };

        StreamerPtr->ForEachPlayer(PlayerFunc);

        if(!bTransceiverExists)
        {
            UE_LOGF(LogPixelStreaming, Error, "Transceivers don't exist!");
        }

		return true;
	}

	bool FCleanupAll::Update()
	{
		if (OutPlayer)
		{
			OutPlayer->Disconnect(TEXT("Cleaning up tests"));
			OutPlayer.Reset();
		}

		if (OutStreamer)
		{
			OutStreamer->StopStreaming();
			OutStreamer.Reset();
		}

		if (OutSignallingServer)
		{
			OutSignallingServer->Stop();
			OutSignallingServer.Reset();
		}
		return true;
	}

	/* ---------- Utility functions ----------- */

	void SetCodec(EPixelStreamingCodec Codec, bool bUseComputeShaders)
	{
		// Set codec and whether to use compute shaders
		UE::PixelStreaming::DoOnGameThreadAndWait(MAX_uint32, [Codec, bUseComputeShaders]() {
			Settings::SetCodec(Codec);
			Settings::CVarPixelStreamingVPXUseCompute->Set(bUseComputeShaders, ECVF_SetByCode);
		});
	}

	void SetDisableTransmitAudio(bool bDisableTransmitAudio)
	{
		// Set whether to transmit audio
		UE::PixelStreaming::DoOnGameThreadAndWait(MAX_uint32, [bDisableTransmitAudio]() {
			Settings::CVarPixelStreamingWebRTCDisableTransmitAudio->Set(bDisableTransmitAudio, ECVF_SetByCode);
		});
	}

	void SetDisableReceiveAudio(bool bDisableReceiveAudio)
	{
		// Set whether to receive audio
		UE::PixelStreaming::DoOnGameThreadAndWait(MAX_uint32, [bDisableReceiveAudio]() {
			Settings::CVarPixelStreamingWebRTCDisableReceiveAudio->Set(bDisableReceiveAudio, ECVF_SetByCode);
		});
	}

	void SetDisableTransmitVideo(bool bDisableTransmitVideo)
	{
		// Set whether to transmit video
		UE::PixelStreaming::DoOnGameThreadAndWait(MAX_uint32, [bDisableTransmitVideo]() {
			Settings::CVarPixelStreamingWebRTCDisableTransmitVideo->Set(bDisableTransmitVideo, ECVF_SetByCode);
		});
	}

	TSharedPtr<IPixelStreamingStreamer> CreateStreamer(int StreamerPort)
	{
		FString StreamerName("Mock Streamer");
		TSharedPtr<IPixelStreamingStreamer> OutStreamer = IPixelStreamingModule::Get().CreateStreamer(StreamerName);
		OutStreamer->SetVideoInput(FPixelStreamingVideoInputBackBuffer::Create());
		OutStreamer->SetSignallingServerURL(FString::Printf(TEXT("ws://127.0.0.1:%d"), StreamerPort));

		TSharedPtr<IPixelStreamingSignallingConnection> SignallingConnection = MakeShared<FPixelStreamingSignallingConnection>(OutStreamer->GetSignallingConnectionObserver().Pin(), StreamerName);
		SignallingConnection->SetAutoReconnect(true);
		OutStreamer->SetSignallingConnection(SignallingConnection);
		return OutStreamer;
	}

	TSharedPtr<FMockPlayer> CreatePlayer(FMockPlayer::EMode OfferMode)
	{
		TSharedPtr<FMockPlayer> OutPlayer = MakeShared<FMockPlayer>();

		// We have to pass an existing shared ptr to the mockplayer or else we will end up with competing reference controllers
		OutPlayer->SignallingServerConnection = MakeUnique<FPixelStreamingSignallingConnection>(OutPlayer, TEXT("FMockPlayer"));
		UE::PixelStreaming::DoOnGameThreadAndWait(MAX_uint32, []() {
			Settings::CVarPixelStreamingSuppressICECandidateErrors->Set(true, ECVF_SetByCode);
		});

		OutPlayer->SetMode(OfferMode);
		return OutPlayer;
	}

	TSharedPtr<UE::PixelStreamingServers::IServer> CreateSignallingServer(int StreamerPort, int PlayerPort, bool bCreateLegacySignallingServer)
	{
		// Make signalling server
		TSharedPtr<UE::PixelStreamingServers::IServer> OutSignallingServer = bCreateLegacySignallingServer ? UE::PixelStreamingServers::MakeLegacySignallingServer() : UE::PixelStreamingServers::MakeSignallingServer();

		UE::PixelStreamingServers::FLaunchArgs LaunchArgs;
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--StreamerPort=%d --HttpPort=%d"), StreamerPort, PlayerPort);
		bool bLaunchedSignallingServer = OutSignallingServer->Launch(LaunchArgs);
		if (!bLaunchedSignallingServer)
		{
			UE_LOGF(LogPixelStreaming, Error, "Failed to launch signalling server.");
		}
		UE_LOGF(LogPixelStreaming, Log, "Signalling server launched=%ls", bLaunchedSignallingServer ? TEXT("true") : TEXT("false"));
		return OutSignallingServer;
	}

	TSharedPtr<UE::PixelStreamingServers::IServer> CreateSignallingServer(int StreamerPort, int PlayerPort)
	{
		return CreateSignallingServer(StreamerPort, PlayerPort, false /*bCreateLegacySignallingServer*/);
	}

	TSharedPtr<UE::PixelStreamingServers::IServer> CreateLegacySignallingServer(int StreamerPort, int PlayerPort)
	{
		return CreateSignallingServer(StreamerPort, PlayerPort, true /*bCreateLegacySignallingServer*/);
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS
