// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"
#include "MockPlayer.h"
#include "PixelStreamingServers.h"
#include "PixelStreamingVideoInputI420.h"
#include "IPixelStreamingStreamer.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	namespace TestUtils
	{
		int32 NextStreamerPort();
		int32 NextPlayerPort();
	}

	// Equivalent to DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER, but instead we define a custom constructor
	class FWaitForDataChannelMessageOrTimeout : public IAutomationLatentCommand
	{
	public:
		FWaitForDataChannelMessageOrTimeout(double InTimeoutSeconds, TSharedPtr<FMockPlayer> InPlayer, TFunction<void(uint8, const webrtc::DataBuffer&)>& InCallback, TSharedPtr<bool> InbComplete)
		: TimeoutSeconds(InTimeoutSeconds)
		, Player(InPlayer)
		, Callback(InCallback)
		, bComplete(InbComplete)
		{
			MessageReceivedHandle = Player->OnMessageReceived.AddLambda([this](uint8 Type, const webrtc::DataBuffer& RawBuffer) {
				(Callback)(Type, RawBuffer);
			});
		}

		virtual ~FWaitForDataChannelMessageOrTimeout() 
		{
			Player->OnMessageReceived.Remove(MessageReceivedHandle);
		}
		virtual bool Update() override; 
	private:
		double TimeoutSeconds;
		TSharedPtr<FMockPlayer> Player;
		TFunction<void(uint8, const webrtc::DataBuffer&)> Callback;
		TSharedPtr<bool> bComplete;
		FDelegateHandle MessageReceivedHandle;
	};

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitSeconds, double, WaitSeconds);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FSendDataChannelMessageToPlayer, TSharedPtr<IPixelStreamingStreamer>, Streamer, uint8, Id, const FString, Body);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FSendDataChannelMessageToStreamer, TSharedPtr<FMockPlayer>, Player, uint8, Id, const FString, Body);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FSendCustomMessageToStreamer, TSharedPtr<FMockPlayer>, Player, uint8, Id, uint16, Body);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FSendSolidColorFrame, TSharedPtr<FPixelStreamingVideoInputI420>, VideoInput, FMockVideoFrameConfig, FrameConfig);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FWaitForFrameReceived, double, TimeoutSeconds, TSharedPtr<FMockVideoSink>, VideoSink, FMockVideoFrameConfig, FrameConfig);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FWaitForICEConnectedOrTimeout, double, TimeoutSeconds, TSharedPtr<FMockPlayer>, OutPlayer);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FWaitForDataChannelOrTimeout, double, TimeoutSeconds, TSharedPtr<FMockPlayer>, OutPlayer);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FWaitForStreamerConnectedOrTimeout, double, TimeoutSeconds, TSharedPtr<IPixelStreamingStreamer>, OutStreamer);
	DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(FConnectPlayerAfterStreamerConnectedOrTimeout, double, TimeoutSeconds, TSharedPtr<IPixelStreamingStreamer>, OutStreamer, TSharedPtr<FMockPlayer>, OutPlayer, int, PlayerPort);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FWaitForPlayerConnectedOrTimeout, double, TimeoutSeconds, TSharedPtr<FMockPlayer>, OutPlayer, int, PlayerPort);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCheckTransceivers, TSharedPtr<IPixelStreamingStreamer>, Streamer, const TFunction<bool(rtc::scoped_refptr<webrtc::RtpTransceiverInterface>)>, CheckFunc);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FCleanupAll, TSharedPtr<UE::PixelStreamingServers::IServer>, OutSignallingServer, TSharedPtr<IPixelStreamingStreamer>, OutStreamer, TSharedPtr<FMockPlayer>, OutPlayer);

	TSharedPtr<IPixelStreamingStreamer> CreateStreamer(int StreamerPort);
	TSharedPtr<FMockPlayer> CreatePlayer(FMockPlayer::EMode OfferMode);
	TSharedPtr<UE::PixelStreamingServers::IServer> CreateSignallingServer(int StreamerPort, int PlayerPort);
	TSharedPtr<UE::PixelStreamingServers::IServer> CreateLegacySignallingServer(int StreamerPort, int PlayerPort);
	void SetCodec(EPixelStreamingCodec Codec, bool bUseComputeShaders);
	void SetDisableTransmitAudio(bool bDisableTransmitAudio);
	void SetDisableReceiveAudio(bool bDisableReceiveAudio);
	void SetDisableTransmitVideo(bool bDisableTransmitVideo);
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS