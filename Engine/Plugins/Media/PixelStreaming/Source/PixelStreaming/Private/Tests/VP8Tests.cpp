// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodecUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP8FrameReceivedTest, "System.Plugins.PixelStreaming.FVP8FrameReceivedTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP8FrameReceivedTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP8, false /*bUseComputeShader*/);
		DoFrameReceiveTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP8ComputeShaderFrameReceivedTest, "System.Plugins.PixelStreaming.FVP8ComuteShaderFrameReceivedTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP8ComputeShaderFrameReceivedTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP8, true /*bUseComputeShader*/);
		DoFrameReceiveTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP8FrameResizeTest, "System.Plugins.PixelStreaming.FVP8FrameResizeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP8FrameResizeTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP8, false /*bUseComputeShader*/);
		DoFrameResizeMultipleTimesTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP8ComputeShaderFrameResizeTest, "System.Plugins.PixelStreaming.FVP8ComputeShaderFrameResizeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP8ComputeShaderFrameResizeTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP8, true /*bUseComputeShader*/);
		DoFrameResizeMultipleTimesTest();
		return true;
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS