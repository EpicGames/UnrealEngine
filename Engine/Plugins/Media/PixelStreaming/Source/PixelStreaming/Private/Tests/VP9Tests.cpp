// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodecUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP9FrameReceivedTest, "System.Plugins.PixelStreaming.FVP9FrameReceivedTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP9FrameReceivedTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP9, false /*bUseComputeShader*/);
		DoFrameReceiveTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP9ComputeShaderFrameReceivedTest, "System.Plugins.PixelStreaming.FVP9ComuteShaderFrameReceivedTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP9ComputeShaderFrameReceivedTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP9, true /*bUseComputeShader*/);
		DoFrameReceiveTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP9FrameResizeTest, "System.Plugins.PixelStreaming.FVP9FrameResizeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP9FrameResizeTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP9, false /*bUseComputeShader*/);
		DoFrameResizeMultipleTimesTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVP9ComputeShaderFrameResizeTest, "System.Plugins.PixelStreaming.FVP9ComputeShaderFrameResizeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVP9ComputeShaderFrameResizeTest::RunTest(const FString& Parameters)
	{
		SetCodec(EPixelStreamingCodec::VP9, false /*bUseComputeShader*/);
		DoFrameResizeMultipleTimesTest();
		return true;
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS