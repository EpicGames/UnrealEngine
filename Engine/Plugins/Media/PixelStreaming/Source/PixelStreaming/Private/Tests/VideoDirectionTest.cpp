// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Misc/AutomationTest.h"
#include "TestUtils.h"
#include "DirectionUtils.h"

namespace UE::PixelStreaming
{
    IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVideoSendOnlyTest, "System.Plugins.PixelStreaming.FVideoSendOnlyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVideoSendOnlyTest::RunTest(const FString& Parameters)
	{
        SetDisableTransmitVideo(false);
		DoDirectionTest(cricket::MediaType::MEDIA_TYPE_VIDEO, webrtc::RtpTransceiverDirection::kSendOnly);

        return true;
	}

    IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVideoInactiveTest, "System.Plugins.PixelStreaming.FVideoInactiveTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FVideoInactiveTest::RunTest(const FString& Parameters)
	{
        SetDisableTransmitVideo(true);
		DoDirectionTest(cricket::MediaType::MEDIA_TYPE_VIDEO, webrtc::RtpTransceiverDirection::kInactive);

        return true;
	}
} // namespace UE::PixelStreaming

#endif // WITH_DEV_AUTOMATION_TESTS