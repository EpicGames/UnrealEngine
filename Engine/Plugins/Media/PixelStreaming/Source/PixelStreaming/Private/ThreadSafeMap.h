// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"

namespace UE::PixelStreaming
{
	template <typename KeyType, typename ValueType>
	class TThreadSafeMap
	{
	private:
		// We add a level of indirection here so when the map changes the location of the ValueType
		// does not change. This allows Find to return a pointer to the ValueType and release the lock
		// and not worry that changes to the map will change the location of the ValueType pointed to.
		using InternalElementType = TSharedPtr<ValueType>;
		TMap<KeyType, InternalElementType> InnerMap;
		mutable FCriticalSection Mutex;

		// Helpers for manipulating internal elements
		InternalElementType NewEmptyElement() { return MakeShared<ValueType>(); }
		InternalElementType NewElement(const ValueType& Value) { return MakeShared<ValueType>(Value); }
		ValueType* GetValuePtr(const InternalElementType& Element) const { return Element.Get(); }
		ValueType& GetValue(const InternalElementType& Element) const { return *GetValuePtr(Element); }

	public:
		bool Add(KeyType Key, const ValueType& Value)
		{
			FScopeLock Lock(&Mutex);
			if (!InnerMap.Contains(Key))
			{
				InnerMap.Add(Key, NewElement(Value));
				return true;
			}
			return false;
		}

		bool Remove(KeyType Key)
		{
			FScopeLock Lock(&Mutex);
			return InnerMap.Remove(Key) != 0;
		}

		ValueType& GetOrAdd(KeyType Key)
		{
			FScopeLock Lock(&Mutex);
			if (InnerMap.Contains(Key))
			{
				return *InnerMap[Key].Get();
			}
			InternalElementType& NewElement = InnerMap.Add(Key, NewEmptyElement());
			return GetValue(NewElement);
		}

		ValueType* Find(KeyType Key)
		{
			FScopeLock Lock(&Mutex);
			if (InternalElementType* Found = InnerMap.Find(Key))
			{
				return GetValuePtr(*Found);
			}
			return nullptr;
		}

		const ValueType* Find(KeyType Key) const
		{
			FScopeLock Lock(&Mutex);
			if (const InternalElementType* Found = InnerMap.Find(Key))
			{
				return GetValuePtr(*Found);
			}
			return nullptr;
		}

		void Empty()
		{
			FScopeLock Lock(&Mutex);
			InnerMap.Empty();
		}

		bool IsEmpty() const
		{
			FScopeLock Lock(&Mutex);
			return InnerMap.IsEmpty();
		}

		// Note: Do not call a map method inside a visitor or you will deadlock
		template <typename T>
		void Apply(T&& Visitor)
		{
			// work on a copy of the map so we dont lock while calling the visitor
			TMap<KeyType, InternalElementType> MapCopy;
			{
				FScopeLock Lock(&Mutex);
				MapCopy = InnerMap;
			}
			for (auto&& [Key, Element] : MapCopy)
			{
				Visitor(Key, GetValue(Element));
			}
		}

		// Note: Do not call a map method inside a visitor or you will deadlock
		template <typename T>
		void ApplyUntil(T&& Visitor)
		{
			// work on a copy of the map so we dont lock while calling the visitor
			TMap<KeyType, InternalElementType> MapCopy;
			{
				FScopeLock Lock(&Mutex);
				MapCopy = InnerMap;
			}
			for (auto&& [Key, Element] : MapCopy)
			{
				if (Visitor(Key, GetValue(Element)))
				{
					break;
				}
			}
		}
	};
} // namespace UE::PixelStreaming
