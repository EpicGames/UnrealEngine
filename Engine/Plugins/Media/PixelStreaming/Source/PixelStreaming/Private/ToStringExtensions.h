// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <string>
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Dom/JsonObject.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/Atomic.h"
#include "WebRTCIncludes.h"

namespace UE::PixelStreaming
{
	inline FString ToString(const std::string& Str)
	{
		return UTF8_TO_TCHAR(Str.c_str());
	}

	inline std::string ToString(const FString& Str)
	{
		return TCHAR_TO_UTF8(*Str);
	}

	inline FString ToString(const TSharedPtr<FJsonObject>& JsonObj, bool bPretty = true)
	{
		FString Res;
		if (bPretty)
		{
			auto JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&Res);
			FJsonSerializer::Serialize(JsonObj.ToSharedRef(), JsonWriter);
		}
		else
		{
			auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Res);
			FJsonSerializer::Serialize(JsonObj.ToSharedRef(), JsonWriter);
		}
		return Res;
	}

	inline const TCHAR* ToString(webrtc::PeerConnectionInterface::SignalingState Val)
	{
		TCHAR const* SignallingStatesStr[] = {
			TEXT("Stable"),
			TEXT("HaveLocalOffer"),
			TEXT("HaveLocalPrAnswer"),
			TEXT("HaveRemoteOffer"),
			TEXT("HaveRemotePrAnswer"),
			TEXT("Closed")
		};

		return ensureMsgf(0 <= Val && Val <= webrtc::PeerConnectionInterface::kClosed, TEXT("Invalid `webrtc::PeerConnectionInterface::SignalingState` value: %d"), static_cast<uint32>(Val)) ? SignallingStatesStr[Val] : TEXT("Unknown");
	}

	inline const TCHAR* ToString(webrtc::PeerConnectionInterface::IceConnectionState Val)
	{
		TCHAR const* IceConnectionStatsStr[] = {
			TEXT("IceConnectionNew"),
			TEXT("IceConnectionChecking"),
			TEXT("IceConnectionConnected"),
			TEXT("IceConnectionCompleted"),
			TEXT("IceConnectionFailed"),
			TEXT("IceConnectionDisconnected"),
			TEXT("IceConnectionClosed")
		};

		return ensureMsgf(
				   0 <= Val && Val < webrtc::PeerConnectionInterface::kIceConnectionMax,
				   TEXT("Invalid `webrtc::PeerConnectionInterface::IceConnectionState` value: %d"),
				   static_cast<uint32>(Val))
			? IceConnectionStatsStr[Val]
			: TEXT("Unknown");
	}

	inline FString ToString(const webrtc::SessionDescriptionInterface& Sdp)
	{
		std::string SdpAnsi;
		verifyf(Sdp.ToString(&SdpAnsi), TEXT("Failed to serialize IceCandidate"));
		return ToString(SdpAnsi);
	}

	inline FString ToString(const webrtc::IceCandidateInterface& IceCandidate)
	{
		std::string CandidateAnsi;
		verifyf(IceCandidate.ToString(&CandidateAnsi), TEXT("Failed to serialize IceCandidate"));
		return ToString(CandidateAnsi);
	}

	inline const TCHAR* ToString(webrtc::PeerConnectionInterface::IceGatheringState Val)
	{
		TCHAR const* IceGatheringStatsStr[] = {
			TEXT("IceGatheringNew"),
			TEXT("IceGatheringGathering"),
			TEXT("IceGatheringComplete")
		};

		return ensureMsgf(
				   0 <= Val && Val <= webrtc::PeerConnectionInterface::kIceGatheringComplete,
				   TEXT("Invalid `webrtc::PeerConnectionInterface::IceGatheringState` value: %d"),
				   static_cast<uint32>(Val))
			? IceGatheringStatsStr[Val]
			: TEXT("Unknown");
	}

	inline const TCHAR* ToString(webrtc::VideoFrameType FrameType)
	{
		TCHAR const* FrameTypesStr[] = {
			TEXT("EmptyFrame"),
			TEXT("AudioFrameSpeech"),
			TEXT("AudioFrameCN"),
			TEXT("VideoFrameKey"),
			TEXT("VideoFrameDelta")
		};
		int FrameTypeInt = (int)FrameType;
		return ensureMsgf(
				   0 <= FrameTypeInt && FrameTypeInt <= (int)webrtc::VideoFrameType::kVideoFrameDelta,
				   TEXT("Invalid `webrtc::FrameType`: %d"), static_cast<uint32>(FrameType))
			? FrameTypesStr[FrameTypeInt]
			: TEXT("Unknown");
	}
} // namespace UE::PixelStreaming
