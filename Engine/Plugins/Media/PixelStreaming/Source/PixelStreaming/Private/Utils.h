// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebRTCIncludes.h"
#include "Async/Async.h"
#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"

namespace UE::PixelStreaming
{
	template <typename T>
	inline void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<T>& CVar)
	{
		T Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value))
			CVar->Set(Value, ECVF_SetByCommandline);
	};

	inline void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<FString>& CVar, bool bStopOnSeparator = false)
	{
		FString Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value, bStopOnSeparator))
			CVar->Set(*Value, ECVF_SetByCommandline);
	};

	inline void CommandLineParseOption(const TCHAR* Match, TAutoConsoleVariable<bool>& CVar)
	{
		FString ValueMatch(Match);
		ValueMatch.Append(TEXT("="));
		FString Value;
		if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value))
		{
			if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
			{
				CVar->Set(true, ECVF_SetByCommandline);
			}
			else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
			{
				CVar->Set(false, ECVF_SetByCommandline);
			}
		}
		else if (FParse::Param(FCommandLine::Get(), Match))
		{
			CVar->Set(true, ECVF_SetByCommandline);
		}
	}

	template <typename T>
	void DoOnGameThread(T&& Func)
	{
		if (IsInGameThread())
		{
			Func();
		}
		else
		{
			AsyncTask(ENamedThreads::GameThread, [Func]() { Func(); });
		}
	}

	template <typename T>
	void DoOnGameThreadAndWait(uint32 Timeout, T&& Func)
	{
		if (IsInGameThread())
		{
			Func();
		}
		else
		{
			FEvent* TaskEvent = FPlatformProcess::GetSynchEventFromPool();
			AsyncTask(ENamedThreads::GameThread, [Func, TaskEvent]() {
				Func();
				TaskEvent->Trigger();
			});
			TaskEvent->Wait(Timeout);
			FPlatformProcess::ReturnSynchEventToPool(TaskEvent);
		}
	}

	inline webrtc::SdpVideoFormat CreateH264Format(webrtc::H264Profile profile, webrtc::H264Level level)
	{
		const absl::optional<std::string> ProfileString =
			webrtc::H264ProfileLevelIdToString(webrtc::H264ProfileLevelId(profile, level));
		check(ProfileString);
		return webrtc::SdpVideoFormat(
			cricket::kH264CodecName,
			{ { cricket::kH264FmtpProfileLevelId, *ProfileString },
				{ cricket::kH264FmtpLevelAsymmetryAllowed, "1" },
				{ cricket::kH264FmtpPacketizationMode, "1" } });
	}

	inline void MemCpyStride(void* Dest, const void* Src, size_t DestStride, size_t SrcStride, size_t Height)
	{
		char* DestPtr = static_cast<char*>(Dest);
		const char* SrcPtr = static_cast<const char*>(Src);
		size_t Row = Height;
		while (Row--)
		{
			FMemory::Memcpy(DestPtr + DestStride * Row, SrcPtr + SrcStride * Row, DestStride);
		}
	}

	inline size_t SerializeToBuffer(rtc::CopyOnWriteBuffer& Buffer, size_t Pos, const void* Data, size_t DataSize)
	{
		FMemory::Memcpy(Buffer.MutableData() + Pos, Data, DataSize);
		return Pos + DataSize;
	}

} // namespace UE::PixelStreaming
