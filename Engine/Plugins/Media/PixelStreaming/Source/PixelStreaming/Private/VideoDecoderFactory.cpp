// Copyright Epic Games, Inc. All Rights Reserved.
#include "VideoDecoderFactory.h"
#include "Utils.h"
#include "VideoDecoderStub.h"
#include "VideoDecoderSoftware.h"
#include "VideoDecoderHardware.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingCoderUtils.h"
#include "Settings.h"

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "absl/strings/match.h"
#include "PostWebRTCApi.h"
// End WebRTC Includes

namespace UE::PixelStreaming
{
	// the list of each individual codec we have decoder support for (the order of this array is preference order after the selected codec)
	const TArray<EPixelStreamingCodec> SupportedDecoderCodecList{ EPixelStreamingCodec::VP8, EPixelStreamingCodec::VP9, EPixelStreamingCodec::H264, EPixelStreamingCodec::AV1 };

	// mapping of codec to a list of video formats
	// done this way so we can order the list of formats based on selected codec in GetSupportedFormats
	TMap<EPixelStreamingCodec, std::vector<webrtc::SdpVideoFormat>> CreateSupportedDecoderFormatMap()
	{
		TMap<EPixelStreamingCodec, std::vector<webrtc::SdpVideoFormat>> Codecs;
		for (auto& Codec : SupportedDecoderCodecList)
		{
			Codecs.Add(Codec);
		}

		Codecs[EPixelStreamingCodec::VP8].push_back(webrtc::SdpVideoFormat(cricket::kVp8CodecName));
		Codecs[EPixelStreamingCodec::VP9].push_back(webrtc::SdpVideoFormat(cricket::kVp9CodecName));
		
		if(IsDecoderSupported<FVideoDecoderConfigH264>())
		{
			Codecs[EPixelStreamingCodec::H264].push_back(UE::PixelStreaming::CreateH264Format(webrtc::H264Profile::kProfileConstrainedBaseline, webrtc::H264Level::kLevel3_1));
			Codecs[EPixelStreamingCodec::H264].push_back(UE::PixelStreaming::CreateH264Format(webrtc::H264Profile::kProfileBaseline, webrtc::H264Level::kLevel3_1));
		}

		if(IsDecoderSupported<FVideoDecoderConfigAV1>())
		{
			Codecs[EPixelStreamingCodec::AV1].push_back(webrtc::SdpVideoFormat(cricket::kAv1CodecName));
		}

		return Codecs;
	}

	/**
	 * Adds all the formats of a given codec to a destination list according to a list of supported formats
	 */
	void AddSupportedCodecFormats(EPixelStreamingCodec Codec,
		const TMap<EPixelStreamingCodec, std::vector<webrtc::SdpVideoFormat>>& SupportedFormatsForCodecs,
		std::vector<webrtc::SdpVideoFormat>& OutFormats)
	{
		if (SupportedFormatsForCodecs.Contains(Codec))
		{
			for (auto& Format : SupportedFormatsForCodecs[Codec])
			{
				if (std::find(std::begin(OutFormats), std::end(OutFormats), Format) == std::end(OutFormats))
				{
					OutFormats.push_back(Format);
				}
			}
		}
	}

	std::vector<webrtc::SdpVideoFormat> FVideoDecoderFactory::GetSupportedFormats() const
	{
		// static so we dont create the list every time this is called since the list will not change during runtime.
		static TMap<EPixelStreamingCodec, std::vector<webrtc::SdpVideoFormat>> CodecMap = CreateSupportedDecoderFormatMap();

		static EPixelStreamingCodec LastSelectedCodec = EPixelStreamingCodec::Invalid;
		const EPixelStreamingCodec SelectedCodec = Settings::GetSelectedCodec();
		const bool NegotiateCodecs = Settings::CVarPixelStreamingWebRTCNegotiateCodecs.GetValueOnAnyThread();

		// this is static so we don't have to generate it every time we get a new connection and we cant do that
		// on a member since this is a const method.
		static std::vector<webrtc::SdpVideoFormat> SupportedFormats;

		// if we're not negotiating codecs we only support the selected codec. If the selected codec is for
		// some reason unsupported, then we fall through to negotiating codecs.
		if (!NegotiateCodecs)
		{
			std::vector<webrtc::SdpVideoFormat> TempSupportedFormats;
			AddSupportedCodecFormats(SelectedCodec, CodecMap, TempSupportedFormats);

			if (TempSupportedFormats.empty())
			{
				UE_LOGF(LogPixelStreaming, Error, "Selected codec was not a supported codec, falling back to negotiating codecs...");
			}
			else
			{
				SupportedFormats.clear();
				SupportedFormats = TempSupportedFormats;
				return SupportedFormats;
			}
		}

		// negotiating codecs means we have a list of codecs we can support and webrtc can pick one.
		// the list will have the currently selected codec on the top so its the "preferred" codec.
		// only if this changes do we need to rebuild the list
		if (LastSelectedCodec != SelectedCodec)
		{
			SupportedFormats.clear();
			LastSelectedCodec = SelectedCodec;

			// order the codecs so the selected is first
			TArray<EPixelStreamingCodec> OrderedCodecList;
			OrderedCodecList.Add(SelectedCodec);
			for (auto& SupportedCodec : SupportedDecoderCodecList)
			{
				if (SupportedCodec != SelectedCodec)
				{
					OrderedCodecList.Add(SupportedCodec);
				}
			}

			// now just add each of the formats in order
			for (auto& Codec : OrderedCodecList)
			{
				AddSupportedCodecFormats(Codec, CodecMap, SupportedFormats);
			}
		}

		return SupportedFormats;
	}

	std::unique_ptr<webrtc::VideoDecoder> FVideoDecoderFactory::CreateVideoDecoder(const webrtc::SdpVideoFormat& format)
	{
		if (absl::EqualsIgnoreCase(format.name, cricket::kVp8CodecName))
		{
			return std::make_unique<FVideoDecoderSoftware>(EPixelStreamingCodec::VP8);
		}
		else if (absl::EqualsIgnoreCase(format.name, cricket::kVp9CodecName))
		{
			return std::make_unique<FVideoDecoderSoftware>(EPixelStreamingCodec::VP9);
		}
		else if (absl::EqualsIgnoreCase(format.name, cricket::kH264CodecName))
		{
			return std::make_unique<FVideoDecoderHardware>(EPixelStreamingCodec::H264);
		}
		else if (absl::EqualsIgnoreCase(format.name, cricket::kAv1CodecName))
		{
			return std::make_unique<FVideoDecoderHardware>(EPixelStreamingCodec::AV1);
		}
		return std::make_unique<FVideoDecoderStub>();
	}
} // namespace UE::PixelStreaming
