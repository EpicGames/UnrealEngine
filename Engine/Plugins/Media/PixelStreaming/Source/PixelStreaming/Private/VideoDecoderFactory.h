// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WebRTCIncludes.h"

namespace UE::PixelStreaming
{
	class FVideoDecoderFactory : public webrtc::VideoDecoderFactory
	{
	public:
		virtual std::vector<webrtc::SdpVideoFormat> GetSupportedFormats() const override;
		virtual std::unique_ptr<webrtc::VideoDecoder> CreateVideoDecoder(const webrtc::SdpVideoFormat& format) override;
	};
} // namespace UE::PixelStreaming
