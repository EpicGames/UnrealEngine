// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderHardware.h"

#include "FrameBufferRHI.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingTrace.h"
#include "Misc/AssertionMacros.h"

namespace UE::PixelStreaming
{
	// HACK (aidan.possemiers) the API surface SendPacket surface wants a SharedPtr for the encoded data but WebRTC already owns that and the types are incompatible
	struct FFakeDeleter
	{
		void operator()(uint8* Object) const
		{
		}
	};

	FVideoDecoderHardware::FVideoDecoderHardware(EPixelStreamingCodec Codec)
	{
		switch(Codec)
		{
			case EPixelStreamingCodec::H264:
			{
				FVideoDecoderConfigH264 DecoderConfig;
				Decoder = FVideoDecoder::CreateChecked<FVideoResourceRHI>(FAVDevice::GetHardwareDevice(), DecoderConfig);
				break;
			}
			case EPixelStreamingCodec::AV1:
			{
				FVideoDecoderConfigAV1 DecoderConfig;
				Decoder = FVideoDecoder::CreateChecked<FVideoResourceRHI>(FAVDevice::GetHardwareDevice(), DecoderConfig);
				break;
			}
			default:
				checkNoEntry();
		}
	}

	bool FVideoDecoderHardware::Configure(const Settings& settings)
	{
		return true;
	}

	int32 FVideoDecoderHardware::Decode(const webrtc::EncodedImage& input_image, bool missing_frames, int64_t render_time_ms)
	{
		const int64 TimestampDecodeStart = rtc::TimeMillis();
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Hardware Decoding Video", PixelStreamingChannel);

		FAVResult Result = Decoder->SendPacket(FVideoPacket(
			MakeShareable<uint8>(input_image.GetEncodedData()->data(), FFakeDeleter()),
			input_image.GetEncodedData()->size(),
			input_image.Timestamp(),
			FrameCount++,
			input_image.qp_,
			input_image._frameType == webrtc::VideoFrameType::kVideoFrameKey));

		if (Output != nullptr && Result.IsNotError())
		{
			FResolvableVideoResourceRHI DecoderResource;
			const FAVResult DecodeResult = Decoder->ReceiveFrame(DecoderResource);

			if (DecodeResult.IsSuccess())
			{
#if WEBRTC_5414
				rtc::scoped_refptr<webrtc::VideoFrameBuffer> FrameBuffer = rtc::make_ref_counted<UE::PixelStreaming::FFrameBufferRHI>(DecoderResource);
#else
				rtc::scoped_refptr<webrtc::VideoFrameBuffer> FrameBuffer = new rtc::RefCountedObject<UE::PixelStreaming::FFrameBufferRHI>(DecoderResource);
#endif
				check(FrameBuffer->width() != 0 && FrameBuffer->height() != 0); // TODO we should probably check that we are getting the frame back that we are expecting

				webrtc::VideoFrame Frame = webrtc::VideoFrame::Builder()
											   .set_video_frame_buffer(FrameBuffer)
											   .set_timestamp_rtp(input_image.Timestamp()) // NOTE This timestamp is load bearing as WebRTC stores the frames in a map with this as the index
											   .set_color_space(input_image.ColorSpace())
											   .build();

				Output->Decoded(Frame, rtc::TimeMillis() - TimestampDecodeStart, input_image.qp_);

				return WEBRTC_VIDEO_CODEC_OK;
			}
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "FVideoDecoderHardware::Decode FAILED");

			return WEBRTC_VIDEO_CODEC_OK_REQUEST_KEYFRAME;
		}

		UE_LOG(LogTemp, Error, TEXT("FVideoDecoderHardware::Decode ERROR"));

		return WEBRTC_VIDEO_CODEC_ERROR;
	}

	int32 FVideoDecoderHardware::RegisterDecodeCompleteCallback(webrtc::DecodedImageCallback* callback)
	{
		Output = callback;

		return 0;
	}

	int32 FVideoDecoderHardware::Release()
	{
		Decoder.Reset();

		return 0;
	}
} // namespace UE::PixelStreaming
