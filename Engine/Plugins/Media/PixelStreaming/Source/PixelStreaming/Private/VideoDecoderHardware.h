// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebRTCIncludes.h"
#include "PixelStreamingCodec.h"
#include "Video/Decoders/Configs/VideoDecoderConfigAV1.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Resources/VideoResourceRHI.h"

namespace UE::PixelStreaming
{
	class FVideoDecoderHardware : public webrtc::VideoDecoder
	{
	public:
		FVideoDecoderHardware(EPixelStreamingCodec Codec);
		virtual ~FVideoDecoderHardware() = default;

		virtual bool Configure(const Settings& settings) override;
		virtual int32 Decode(const webrtc::EncodedImage& input_image, bool missing_frames, int64_t render_time_ms) override;
		virtual int32 RegisterDecodeCompleteCallback(webrtc::DecodedImageCallback* callback) override;
		virtual int32 Release() override;
		virtual const char* ImplementationName() const { return "FVideoDecoderHardware"; }

	private:
		TSharedPtr<TVideoDecoder<FVideoResourceRHI>> Decoder;

		webrtc::DecodedImageCallback* Output = nullptr;

		uint32 FrameCount = 0;
	};
} // namespace UE::PixelStreaming
