// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderSoftware.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingTrace.h"

namespace UE::PixelStreaming
{
	FVideoDecoderSoftware::FVideoDecoderSoftware(EPixelStreamingCodec Codec)
	{
		switch(Codec)
		{
			case EPixelStreamingCodec::VP8:
				VideoDecoder = webrtc::VP8Decoder::Create();
				break;
			case EPixelStreamingCodec::VP9:
				VideoDecoder = webrtc::VP9Decoder::Create();
				break;
			case EPixelStreamingCodec::H264:
			case EPixelStreamingCodec::AV1:
			default:
				UE_LOGF(LogPixelStreaming, Fatal, "Unsupported video codec");
				break;
		}
	}

	bool FVideoDecoderSoftware::Configure(const Settings& settings)
	{
		return VideoDecoder->Configure(settings);
	}

	int32 FVideoDecoderSoftware::Decode(const webrtc::EncodedImage& input_image, bool missing_frames, int64_t render_time_ms)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Software Decoding Video", PixelStreamingChannel);
		return VideoDecoder->Decode(input_image, missing_frames, render_time_ms);
	}

	int32 FVideoDecoderSoftware::RegisterDecodeCompleteCallback(webrtc::DecodedImageCallback* callback)
	{
		return VideoDecoder->RegisterDecodeCompleteCallback(callback);
	}

	int32 FVideoDecoderSoftware::Release()
	{
		return VideoDecoder->Release();
	}
} // namespace UE::PixelStreaming
