// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebRTCIncludes.h"
#include "PixelStreamingCodec.h"

namespace UE::PixelStreaming
{
	class FVideoDecoderSoftware : public webrtc::VideoDecoder
	{
	public:
		FVideoDecoderSoftware(EPixelStreamingCodec Codec);
		virtual ~FVideoDecoderSoftware() = default;

		virtual bool Configure(const Settings& settings) override;
		virtual int32 Decode(const webrtc::EncodedImage& input_image, bool missing_frames, int64_t render_time_ms) override;
		virtual int32 RegisterDecodeCompleteCallback(webrtc::DecodedImageCallback* callback) override;
		virtual int32 Release() override;
		virtual const char* ImplementationName() const { return "VideoDecoderSoftware"; }

	private:
		std::unique_ptr<webrtc::VideoDecoder> VideoDecoder;
	};
} // namespace UE::PixelStreaming
