// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WebRTCIncludes.h"

namespace UE::PixelStreaming
{
	class FVideoDecoderStub : public webrtc::VideoDecoder
	{
	public:
		virtual bool Configure(const Settings& settings) override { return false; }
		virtual int32_t Decode(const webrtc::EncodedImage& input_image, bool missing_frames, int64_t render_time_ms) override { return 0; }
		virtual int32_t RegisterDecodeCompleteCallback(webrtc::DecodedImageCallback* callback) override { return 0; }
		virtual int32_t Release() override { return 0; }
	};
} // namespace UE::PixelStreaming
