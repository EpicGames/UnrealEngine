// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderFactoryLayered.h"
#include "Settings.h"
#include "WebRTCIncludes.h"
#include "VideoEncoderLayered.h"
#include "VideoEncoderFactorySingleLayer.h"

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "absl/strings/match.h"
#include "PostWebRTCApi.h"
// End WebRTC Includes

namespace UE::PixelStreaming
{
	FVideoEncoderFactoryLayered::FVideoEncoderFactoryLayered()
		: PrimaryEncoderFactory(MakeUnique<FVideoEncoderFactorySingleLayer>())
	{
		FScopeLock Lock(&EncoderFactoriesGuard);
		EncoderFactories.SetNum(Settings::SimulcastParameters.Layers.Num());
		for (int i = 0; i < Settings::SimulcastParameters.Layers.Num(); i++)
		{
			EncoderFactories[i] = MakeUnique<FVideoEncoderFactorySingleLayer>();
		}
	}

	FVideoEncoderFactoryLayered::~FVideoEncoderFactoryLayered()
	{
	}

	std::unique_ptr<webrtc::VideoEncoder> FVideoEncoderFactoryLayered::CreateVideoEncoder(const webrtc::SdpVideoFormat& format)
	{
		return std::make_unique<FVideoEncoderLayered>(*this, format);
	}

	FVideoEncoderFactorySingleLayer* FVideoEncoderFactoryLayered::GetEncoderFactory(int StreamIndex)
	{
		FScopeLock Lock(&EncoderFactoriesGuard);
		return EncoderFactories[StreamIndex].Get();
	}

	void FVideoEncoderFactoryLayered::ForceKeyFrame()
	{
		FScopeLock Lock(&EncoderFactoriesGuard);
		for (auto&& Encoder : EncoderFactories)
		{
			Encoder->ForceKeyFrame();
		}
	}

	std::vector<webrtc::SdpVideoFormat> FVideoEncoderFactoryLayered::GetSupportedFormats() const
	{
		return PrimaryEncoderFactory->GetSupportedFormats();
	}
#if WEBRTC_5414
	FVideoEncoderFactoryLayered::CodecSupport FVideoEncoderFactoryLayered::QueryCodecSupport(const webrtc::SdpVideoFormat& format, absl::optional<std::string> scalability_mode) const
	{
		return PrimaryEncoderFactory->QueryCodecSupport(format, scalability_mode);
	}
#else
	FVideoEncoderFactoryLayered::CodecInfo FVideoEncoderFactoryLayered::QueryVideoEncoder(const webrtc::SdpVideoFormat& format) const
	{
		return PrimaryEncoderFactory->QueryVideoEncoder(format);
	}
#endif
} // namespace UE::PixelStreaming
