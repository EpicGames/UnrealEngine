// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WebRTCIncludes.h"

namespace UE::PixelStreaming 
{
	class FVideoEncoderFactorySingleLayer;
	
	/**
	 * The main encoder factory for pixel streaming. This should be the factory used for all of Pixel Streaming.
	 * Provides Pixel Streaming peers with FVideoEncoderLayered encoders, which should be used regardless
	 * of whether simulcast is being used or not.
	 */
	class FVideoEncoderFactoryLayered : public webrtc::VideoEncoderFactory
	{
	public:
		FVideoEncoderFactoryLayered();
		virtual ~FVideoEncoderFactoryLayered();
		virtual std::vector<webrtc::SdpVideoFormat> GetSupportedFormats() const ;
#if WEBRTC_5414
		virtual CodecSupport QueryCodecSupport(const webrtc::SdpVideoFormat& format, absl::optional<std::string> scalability_mode) const override;
#else
		virtual CodecInfo QueryVideoEncoder(const webrtc::SdpVideoFormat& format) const override;
#endif
		virtual std::unique_ptr<webrtc::VideoEncoder> CreateVideoEncoder(const webrtc::SdpVideoFormat& format);

		FVideoEncoderFactorySingleLayer* GetEncoderFactory(int StreamIndex);

		void ForceKeyFrame();

	private:
		TArray<TUniquePtr<FVideoEncoderFactorySingleLayer>> EncoderFactories;
		FCriticalSection EncoderFactoriesGuard;

		TUniquePtr<FVideoEncoderFactorySingleLayer> PrimaryEncoderFactory;
	};
} // namespace UE::PixelStreaming
