// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderFactorySingleLayer.h"
#include "VideoEncoderSingleLayerHardware.h"
#include "VideoEncoderSingleLayerVPX.h"
#include "Settings.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingCoderUtils.h"
#include "Utils.h"
#include "Stats.h"
#include "PixelStreamingDelegates.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "NvmlEncoder.h"

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "absl/strings/match.h"
#include "PostWebRTCApi.h"
// End WebRTC Includes

namespace UE::PixelStreaming
{

	// the list of each individual codec we have encoder support for (order of this array is preference order after selected codec)
	const TArray<EPixelStreamingCodec>
		SupportedEncoderCodecList{ EPixelStreamingCodec::VP8, EPixelStreamingCodec::VP9, EPixelStreamingCodec::H264, EPixelStreamingCodec::AV1 };

	// mapping of codec to a list of video formats
	// done this way so we can order the list of formats based on selected codec in GetSupportedFormats
	TMap<EPixelStreamingCodec, std::vector<webrtc::SdpVideoFormat>> CreateSupportedEncoderFormatMap()
	{
		TMap<EPixelStreamingCodec, std::vector<webrtc::SdpVideoFormat>> Codecs;
		for (auto& Codec : SupportedEncoderCodecList)
		{
			Codecs.Add(Codec);
		}

		Codecs[EPixelStreamingCodec::VP8].push_back(webrtc::SdpVideoFormat(cricket::kVp8CodecName));
		Codecs[EPixelStreamingCodec::VP9].push_back(webrtc::SdpVideoFormat(cricket::kVp9CodecName));

		if(IsEncoderSupported<FVideoEncoderConfigH264>())
		{
			Codecs[EPixelStreamingCodec::H264].push_back(UE::PixelStreaming::CreateH264Format(webrtc::H264Profile::kProfileConstrainedBaseline, webrtc::H264Level::kLevel3_1));
			Codecs[EPixelStreamingCodec::H264].push_back(UE::PixelStreaming::CreateH264Format(webrtc::H264Profile::kProfileBaseline, webrtc::H264Level::kLevel3_1));
		}

		if(IsEncoderSupported<FVideoEncoderConfigAV1>())
		{
			Codecs[EPixelStreamingCodec::AV1].push_back(webrtc::SdpVideoFormat(cricket::kAv1CodecName));
		}

		return Codecs;
	}

	// we want this method to return all the formats we have encoders for but the selected codecs formats should be first in the list.
	// the reason for this is weird. when we receive video from another pixel streaming source, for some reason webrtc will query
	// the encoder factory on the receiving end and if it doesnt support the video we are receiving then transport_cc is not enabled
	// which leads to very low bitrate streams.
	std::vector<webrtc::SdpVideoFormat> FVideoEncoderFactorySingleLayer::GetSupportedFormats() const
	{

		// static so we dont create the list every time this is called since the list will not change during runtime.
		static TMap<EPixelStreamingCodec, std::vector<webrtc::SdpVideoFormat>> CodecMap = CreateSupportedEncoderFormatMap();

		// since this method is const we need to store this state statically. it means all instances will share this state
		// but that actually works in our favor since we're describing more about the plugin state than the actual
		// instance of this factory.
		static std::vector<webrtc::SdpVideoFormat> SupportedFormats;

		EPixelStreamingCodec SelectedCodec = UE::PixelStreaming::Settings::GetSelectedCodec();
#if (PLATFORM_WINDOWS && PLATFORM_CPU_X86_FAMILY) || PLATFORM_LINUX
		if ((SelectedCodec == EPixelStreamingCodec::H264 || SelectedCodec == EPixelStreamingCodec::AV1) && IsRHIDeviceNVIDIA())
		{
			// NOTE (william.belcher): This check will return false if all the encoding sessions are in use, even if the user intends
			// to stream share.
			int32 NumEncoderSessions = NvmlEncoder::GetEncoderSessionCount(0); // TODO we should probably actually figure out the GPU index rather than assume 0
			int32 MaxCVarAllowedSessions = Settings::CVarPixelStreamingEncoderMaxSessions.GetValueOnAnyThread();
			bool bCanCreateHardwareEncoder = true;

			if (MaxCVarAllowedSessions != -1 && NumEncoderSessions != -1)
			{
				// If our CVar is set and we receive a valid session count
				bCanCreateHardwareEncoder &= NumEncoderSessions < MaxCVarAllowedSessions;
			}
			else if (MaxCVarAllowedSessions == -1)
			{
				// If we receive a valid session count and our cvar isn't set
				bCanCreateHardwareEncoder &= NvmlEncoder::IsEncoderSessionAvailable(0); // TODO we should probably actually figure out the GPU index rather than assume 0
			}

			if (!bCanCreateHardwareEncoder)
			{
				// No more hardware encoder sessions available. Fallback to VP8
				// NOTE: CVars can only be set from game thread
				DoOnGameThread([]() {
					Settings::SetCodec(EPixelStreamingCodec::VP8);
					if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
					{
						Delegates->OnFallbackToSoftwareEncodering.Broadcast();
						Delegates->OnFallbackToSoftwareEncoderingNative.Broadcast();
					}
				});
				// Also update our local SelectedCodec to reflect what the state will be
				SelectedCodec = EPixelStreamingCodec::VP8;
				UE_LOGF(LogPixelStreaming, Warning, "No more HW encoders available. Falling back to software encoding");
				CodecMap.Remove(EPixelStreamingCodec::H264);
				CodecMap.Remove(EPixelStreamingCodec::AV1);
			}
		}
#endif // PLATFORM_WINDOWS || PLATFORM_LINUX

		// If we are not negotiating codecs simply return just the one codec that is selected in UE
		if (!Settings::CVarPixelStreamingWebRTCNegotiateCodecs.GetValueOnAnyThread())
		{
			SupportedFormats.clear();

			if (CodecMap.Contains(SelectedCodec))
			{
				for (auto& Format : CodecMap[SelectedCodec])
				{
					SupportedFormats.push_back(Format);
				}
			}
			else
			{
				UE_LOGF(LogPixelStreaming, Error, "Selected codec was not a supported codec.");
			}

			// We can still support flex-fec even if we're not negotiating codecs
			if (Settings::CVarPixelStreamingWebRTCEnableFlexFec.GetValueOnAnyThread())
			{
				SupportedFormats.push_back(webrtc::SdpVideoFormat(cricket::kFlexfecCodecName));
			}

			return SupportedFormats;
		}

		static EPixelStreamingCodec LastSelectedCodec = EPixelStreamingCodec::Invalid;
		if (LastSelectedCodec != SelectedCodec)
		{
			// build a new format list
			LastSelectedCodec = SelectedCodec;
			SupportedFormats.clear();

			// order the codecs so the selected is first
			TArray<EPixelStreamingCodec> OrderedCodecList;
			OrderedCodecList.Add(SelectedCodec);
			for (auto& SupportedCodec : SupportedEncoderCodecList)
			{
				if (SupportedCodec != SelectedCodec)
				{
					OrderedCodecList.Add(SupportedCodec);
				}
			}

			// now just add each of the formats in order
			for (auto& Codec : OrderedCodecList)
			{
				if (CodecMap.Contains(Codec))
				{
					for (auto& Format : CodecMap[Codec])
					{
						SupportedFormats.push_back(Format);
					}
				}
			}

			if (Settings::CVarPixelStreamingWebRTCEnableFlexFec.GetValueOnAnyThread())
			{
				SupportedFormats.push_back(webrtc::SdpVideoFormat(cricket::kFlexfecCodecName));
			}
		}

		return SupportedFormats;
	}

#if WEBRTC_5414
	FVideoEncoderFactorySingleLayer::CodecSupport FVideoEncoderFactorySingleLayer::QueryCodecSupport(const webrtc::SdpVideoFormat& Format, absl::optional<std::string> scalability_mode) const
	{
		webrtc::VideoEncoderFactory::CodecSupport CodecSupport;
		return CodecSupport;
	}
#else
	FVideoEncoderFactorySingleLayer::CodecInfo FVideoEncoderFactorySingleLayer::QueryVideoEncoder(const webrtc::SdpVideoFormat& format) const
	{
		webrtc::VideoEncoderFactory::CodecInfo CodecInfo;
		CodecInfo.has_internal_source = false;
		return CodecInfo;
	}
#endif

	std::unique_ptr<webrtc::VideoEncoder> FVideoEncoderFactorySingleLayer::CreateVideoEncoder(const webrtc::SdpVideoFormat& format)
	{
		if (absl::EqualsIgnoreCase(format.name, cricket::kVp8CodecName))
		{
			FStats::Get()->StoreApplicationStat(FStatData(FName(TEXT("Video Codec - VP8")), 1, 0));
			return std::make_unique<FVideoEncoderSingleLayerVPX>(8);
		}
		else if (absl::EqualsIgnoreCase(format.name, cricket::kVp9CodecName))
		{
			FStats::Get()->StoreApplicationStat(FStatData(FName(TEXT("Video Codec - VP9")), 1, 0));
			return std::make_unique<FVideoEncoderSingleLayerVPX>(9);
		}
		else if (absl::EqualsIgnoreCase(format.name, cricket::kAv1CodecName))
		{
			FStats::Get()->StoreApplicationStat(FStatData(FName(TEXT("Video Codec - AV1")), 1, 0));
			FScopeLock Lock(&ActiveEncodersGuard);
			auto VideoEncoder = std::make_unique<FVideoEncoderSingleLayerHardware>(*this, EPixelStreamingCodec::AV1);
			ActiveEncoders.Add(VideoEncoder.get());
			return VideoEncoder;
		}
		else
		{
			// Lock during encoder creation
			FStats::Get()->StoreApplicationStat(FStatData(FName(TEXT("Video Codec - H264")), 1, 0));
			FScopeLock Lock(&ActiveEncodersGuard);
			auto VideoEncoder = std::make_unique<FVideoEncoderSingleLayerHardware>(*this, EPixelStreamingCodec::H264);
			ActiveEncoders.Add(VideoEncoder.get());
			return VideoEncoder;
		}
	}

	void FVideoEncoderFactorySingleLayer::OnEncodedImage(const webrtc::EncodedImage& encoded_image, const webrtc::CodecSpecificInfo* codec_specific_info, uint32 StreamId)
	{
		// Lock as we send encoded image to each encoder.
		FScopeLock Lock(&ActiveEncodersGuard);

		if (codec_specific_info->codecType == webrtc::kVideoCodecH264 || codec_specific_info->codecType == webrtc::kVideoCodecAV1)
		{
			// Go through each encoder and send our encoded image to its callback
			for (FVideoEncoderSingleLayerHardware* Encoder : ActiveEncoders)
			{
				Encoder->SendEncodedImage(encoded_image, codec_specific_info, StreamId);
			}
		}
	}

	void FVideoEncoderFactorySingleLayer::ReleaseVideoEncoder(FVideoEncoderSingleLayerHardware* Encoder)
	{
		// Lock during deleting an encoder
		FScopeLock Lock(&ActiveEncodersGuard);
		ActiveEncoders.Remove(Encoder);

		FreeUnusedEncoders();
	}

	void FVideoEncoderFactorySingleLayer::FreeUnusedEncoders()
	{
		// first clear unused encoders.
		TArray<uint32> DeleteStreamIds;
		HardwareEncoders.Apply([&DeleteStreamIds](uint32 EncoderStreamId, TSharedPtr<FVideoEncoderHardware>& EncoderPtr) {
			if (EncoderPtr.IsUnique())
			{
				DeleteStreamIds.Add(EncoderStreamId);
			}
		});
		for (uint32 DeleteStreamId : DeleteStreamIds)
		{
			HardwareEncoders.Remove(DeleteStreamId);
		}
	}

	void FVideoEncoderFactorySingleLayer::ForceKeyFrame()
	{
		bForceNextKeyframe = true;
	}

	bool FVideoEncoderFactorySingleLayer::ShouldForceKeyframe() const
	{
		return bForceNextKeyframe;
	}

	void FVideoEncoderFactorySingleLayer::UnforceKeyFrame()
	{
		bForceNextKeyframe = false;
	}

} // namespace UE::PixelStreaming
