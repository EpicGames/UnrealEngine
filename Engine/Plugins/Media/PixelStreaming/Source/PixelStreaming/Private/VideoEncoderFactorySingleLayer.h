// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WebRTCIncludes.h"

#include "Video/VideoEncoder.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Resources/VideoResourceRHI.h"
#include "ThreadSafeMap.h"
#include "PixelStreamingPrivate.h"

namespace UE::PixelStreaming
{
	using FVideoEncoderHardware = TVideoEncoder<FVideoResourceRHI>;

	class FVideoEncoderSingleLayerHardware;

	/**
	 * An encoder factory for a single layer. Do not use this directly. Use FVideoEncoderFactoryLayered even
	 * when not using simulcast specifically.
	 */
	class PIXELSTREAMING_API FVideoEncoderFactorySingleLayer : public webrtc::VideoEncoderFactory
	{
	public:
		FVideoEncoderFactorySingleLayer() = default;
		virtual ~FVideoEncoderFactorySingleLayer() override = default;

		virtual std::vector<webrtc::SdpVideoFormat> GetSupportedFormats() const override;

#if WEBRTC_5414
		// Always returns our H264 hardware encoders codec_info for now/
		virtual CodecSupport QueryCodecSupport(const webrtc::SdpVideoFormat& Format, absl::optional<std::string> scalability_mode) const override;
#else
		virtual CodecInfo QueryVideoEncoder(const webrtc::SdpVideoFormat& Format) const override;
#endif

		// Always returns our H264 hardware encoder for now.
		virtual std::unique_ptr<webrtc::VideoEncoder> CreateVideoEncoder(const webrtc::SdpVideoFormat& Format) override;

		void ReleaseVideoEncoder(FVideoEncoderSingleLayerHardware* Encoder);

		void ForceKeyFrame();
		bool ShouldForceKeyframe() const;
		void UnforceKeyFrame();
		void OnEncodedImage(const webrtc::EncodedImage& Encoded_image, const webrtc::CodecSpecificInfo* CodecSpecificInfo, uint32 StreamId);

		template <typename ConfigType>
		TWeakPtr<TVideoEncoder<FVideoResourceRHI>> GetOrCreateHardwareEncoder(uint32 StreamId, const ConfigType& VideoConfig)
		{
			FScopeLock InitLock(&InitEncoderGuard);

			FreeUnusedEncoders();

			if (auto* ExistingEncoder = HardwareEncoders.Find(StreamId))
			{
				return *ExistingEncoder;
			}
			else
			{
				// Make the hardware encoder wrapper
				TSharedPtr<FVideoEncoderHardware> HardwareEncoder = FVideoEncoder::Create<FVideoResourceRHI>(FAVDevice::GetHardwareDevice(), VideoConfig);

				if (!HardwareEncoder.IsValid())
				{
					UE_LOGF(LogPixelStreaming, Error, "Could not create encoder. Check encoder config or perhaps you used up all your HW encoders.");
					// We could not make the encoder, so indicate the id was not set successfully.
					return nullptr;
				}

				HardwareEncoders.Add(StreamId, HardwareEncoder);
				return HardwareEncoder;
			}
		}

	private:
		void FreeUnusedEncoders();
		bool CheckEncoderSessionAvailable() const;

		TThreadSafeMap<uint32, TSharedPtr<FVideoEncoderHardware>> HardwareEncoders;

		uint8 bForceNextKeyframe : 1;

		// Encoders assigned to each peer. Each one of these will be assigned to one of the hardware encoders
		// raw pointers here because the factory interface wants a unique_ptr here so we cant own the object.
		TArray<FVideoEncoderSingleLayerHardware*> ActiveEncoders;
		FCriticalSection ActiveEncodersGuard;

		// Init encoder guard
		FCriticalSection InitEncoderGuard;
	};
} // namespace UE::PixelStreaming
