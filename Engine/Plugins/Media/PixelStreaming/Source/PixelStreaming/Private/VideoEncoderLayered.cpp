// Copyright Epic Games, Inc. All Rights Reserved.
#include "VideoEncoderLayered.h"
#include "FrameBufferMultiFormat.h"
#include "VideoEncoderFactorySingleLayer.h"
#include "Settings.h"
#include "PixelStreamingTrace.h"

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "modules/video_coding/codecs/vp8/include/vp8.h"
#include "modules/video_coding/codecs/vp9/include/vp9.h"
#include "PostWebRTCApi.h"
// End WebRTC Includes

namespace
{
	uint32_t SumStreamMaxBitrate(int Streams, const webrtc::VideoCodec& Codec)
	{
		uint32_t BitrateSum = 0;
		for (int i = 0; i < Streams; ++i)
		{
			BitrateSum += Codec.simulcastStream[i].maxBitrate;
		}
		return BitrateSum;
	}

	int GetNumberOfStreams(const webrtc::VideoCodec& Codec)
	{
		int Streams = Codec.numberOfSimulcastStreams < 1 ? 1 : Codec.numberOfSimulcastStreams;
		uint32_t SimulcastMaxBitrate = SumStreamMaxBitrate(Streams, Codec);
		if (SimulcastMaxBitrate == 0)
		{
			Streams = 1;
		}
		return Streams;
	}

	int GetNumActiveStreams(const webrtc::VideoCodec& Codec)
	{
		int NumConfiguredStreams = GetNumberOfStreams(Codec);
		int NumActiveStreams = 0;
		for (int i = 0; i < NumConfiguredStreams; ++i)
		{
			if (Codec.simulcastStream[i].active)
			{
				++NumActiveStreams;
			}
		}
		return NumActiveStreams;
	}

	void PopulateStreamCodec(const webrtc::VideoCodec& CodecSettings, int StreamIndex, uint32_t StartBitrateKbps, webrtc::VideoCodec* StreamCodec)
	{
		*StreamCodec = CodecSettings;

		// Stream specific settings.
		StreamCodec->numberOfSimulcastStreams = 0;
		StreamCodec->width = CodecSettings.simulcastStream[StreamIndex].width;
		StreamCodec->height = CodecSettings.simulcastStream[StreamIndex].height;
		StreamCodec->maxBitrate = CodecSettings.simulcastStream[StreamIndex].maxBitrate;
		StreamCodec->minBitrate = CodecSettings.simulcastStream[StreamIndex].minBitrate;
		StreamCodec->maxFramerate = CodecSettings.simulcastStream[StreamIndex].maxFramerate;
		StreamCodec->qpMax = CodecSettings.simulcastStream[StreamIndex].qpMax;
		StreamCodec->active = CodecSettings.simulcastStream[StreamIndex].active;
		if (CodecSettings.codecType == webrtc::kVideoCodecH264)
		{
			StreamCodec->H264()->numberOfTemporalLayers = CodecSettings.simulcastStream[StreamIndex].numberOfTemporalLayers;
		}
		StreamCodec->startBitrate = StartBitrateKbps;
	}

	// An EncodedImageCallback implementation that forwards on calls to a
	// UE::PixelStreaming::FVideoEncoderLayered, but with the stream index it's registered with as
	// the first parameter to Encoded.
	class AdapterEncodedImageCallback : public webrtc::EncodedImageCallback
	{
	public:
		AdapterEncodedImageCallback(UE::PixelStreaming::FVideoEncoderLayered* adapter,
			size_t stream_idx)
			: adapter_(adapter), stream_idx_(stream_idx)
		{
		}

		EncodedImageCallback::Result OnEncodedImage(
			const webrtc::EncodedImage& encoded_image,
			const webrtc::CodecSpecificInfo* codec_specific_info) override
		{
			return adapter_->OnEncodedImage(stream_idx_, encoded_image, codec_specific_info);
		}

		virtual void OnDroppedFrame(EncodedImageCallback::DropReason Reason) override
		{
			switch(Reason)
			{
				case EncodedImageCallback::DropReason::kDroppedByEncoder:
					UE_LOGF(LogPixelStreaming, Warning, "Frame was dropped by the encoder.")
					break;
				case EncodedImageCallback::DropReason::kDroppedByMediaOptimizations:
					UE_LOGF(LogPixelStreaming, Warning, "Frame was dropped due to media optimization (perhaps bitrate limits).")
					break;
				default:
					UE_LOGF(LogPixelStreaming, Warning, "Frame was dropped due to unknown drop reason, perhaps we need a new case here.).")
			}
		}

	private:
		UE::PixelStreaming::FVideoEncoderLayered* const adapter_;
		const size_t stream_idx_;
	};
} // namespace

namespace UE::PixelStreaming
{
	FVideoEncoderLayered::FVideoEncoderLayered(FVideoEncoderFactoryLayered& InSimulcastFactory, const webrtc::SdpVideoFormat& format)
		: Initialized(false)
		, SimulcastEncoderFactory(InSimulcastFactory)
		, VideoFormat(format)
		, EncodedCompleteCallback(nullptr)
	{
		memset(&CurrentCodec, 0, sizeof(webrtc::VideoCodec));
	}

	FVideoEncoderLayered::~FVideoEncoderLayered()
	{
		checkf(!IsInitialized(), TEXT("Encoder adapter being destroyed without being Released!"));
	}

	int FVideoEncoderLayered::Release()
	{
		{
			// Lock during deleting an encoder
			FScopeLock Lock(&StreamInfosGuard);
			while (!StreamInfos.empty())
			{
				std::unique_ptr<VideoEncoder> Encoder = std::move(StreamInfos.back().Encoder);
				// Even though it seems very unlikely, there are no guarantees that the
				// encoder will not call back after being Release()'d. Therefore, we first
				// disable the callbacks here.
				Encoder->RegisterEncodeCompleteCallback(nullptr);
				Encoder->Release();
				StreamInfos.pop_back(); // Deletes callback adapter.
			}
		}

		Initialized = false;

		return WEBRTC_VIDEO_CODEC_OK;
	}

	int FVideoEncoderLayered::InitEncode(const webrtc::VideoCodec* codec_settings, const webrtc::VideoEncoder::Settings& settings)
	{
		const int NumberOfStreams = GetNumberOfStreams(*codec_settings);
		const int NumActiveStreams = GetNumActiveStreams(*codec_settings);

		CurrentCodec = *codec_settings;

		// clang-format off
		const webrtc::SdpVideoFormat Format(CurrentCodec.codecType == webrtc::kVideoCodecVP8 ? "VP8"
										  : CurrentCodec.codecType == webrtc::kVideoCodecVP9 ? "VP9"
										  : CurrentCodec.codecType == webrtc::kVideoCodecAV1 ? "AV1"
																							 : "H264",
			                                VideoFormat.parameters);
		// clang-format on

		if (NumActiveStreams == 1)
		{
			// with one stream we just proxy the pixelstreaming encoder
			const int LastStreamIndex = UE::PixelStreaming::Settings::SimulcastParameters.Layers.Num() - 1; // Last stream is highest res.
			FVideoEncoderFactorySingleLayer* EncoderFactory = SimulcastEncoderFactory.GetEncoderFactory(LastStreamIndex);
			std::unique_ptr<VideoEncoder> Encoder = EncoderFactory->CreateVideoEncoder(Format);

			int ReturnCode = Encoder->InitEncode(&CurrentCodec, settings);
			if (ReturnCode < 0)
			{
				// Explicitly destroy the current encoder; because we haven't registered
				// a StreamInfo for it yet, Release won't do anything about it.
				Encoder.reset();
				Release();
				return ReturnCode;
			}

			std::unique_ptr<webrtc::EncodedImageCallback> Callback(new AdapterEncodedImageCallback(this, 0));
			Encoder->RegisterEncodeCompleteCallback(Callback.get());
			StreamInfos.emplace_back(
				std::move(Encoder), std::move(Callback),
				std::make_unique<webrtc::FramerateController>(CurrentCodec.maxFramerate),
				CurrentCodec.width, CurrentCodec.height, true, true);
		}
		else
		{
			for (int i = 0; i < NumberOfStreams; ++i)
			{
				webrtc::VideoCodec StreamCodec;
				uint32_t StartBitrateKbps = CurrentCodec.simulcastStream[i].targetBitrate;
				PopulateStreamCodec(CurrentCodec, i, StartBitrateKbps, &StreamCodec);

				FVideoEncoderFactorySingleLayer* EncoderFactory = SimulcastEncoderFactory.GetEncoderFactory(i);
				std::unique_ptr<VideoEncoder> Encoder = EncoderFactory->CreateVideoEncoder(Format);

				int ReturnCode = Encoder->InitEncode(&StreamCodec, settings);
				if (ReturnCode < 0)
				{
					// Explicitly destroy the current encoder; because we haven't registered
					// a StreamInfo for it yet, Release won't do anything about it.
					Encoder.reset();
					Release();
					return ReturnCode;
				}

				std::unique_ptr<webrtc::EncodedImageCallback> Callback(new AdapterEncodedImageCallback(this, i));
				Encoder->RegisterEncodeCompleteCallback(Callback.get());
				StreamInfos.emplace_back(
					std::move(Encoder), std::move(Callback),
					std::make_unique<webrtc::FramerateController>(StreamCodec.maxFramerate),
					StreamCodec.width, StreamCodec.height, true, true);
			}
		}

		Initialized = true;

		return WEBRTC_VIDEO_CODEC_OK;
	}

	int FVideoEncoderLayered::Encode(const webrtc::VideoFrame& input_image, const std::vector<webrtc::VideoFrameType>* frame_types)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Encoding Layer", PixelStreamingChannel);
		if (!IsInitialized())
		{
			return WEBRTC_VIDEO_CODEC_UNINITIALIZED;
		}

		if (EncodedCompleteCallback == nullptr)
		{
			return WEBRTC_VIDEO_CODEC_UNINITIALIZED;
		}

		// All active streams should generate a key frame if
		// a key frame is requested by any stream.
		bool bSendKeyFrame = false;
		if (frame_types)
		{
			for (size_t i = 0; i < frame_types->size(); ++i)
			{
				if (frame_types->at(i) == webrtc::VideoFrameType::kVideoFrameKey)
				{
					bSendKeyFrame = true;
					break;
				}
			}
		}

		for (size_t StreamIdx = 0; StreamIdx < StreamInfos.size(); ++StreamIdx)
		{
			if (StreamInfos[StreamIdx].KeyFrameRequest && StreamInfos[StreamIdx].bSendStream)
			{
				bSendKeyFrame = true;
				break;
			}
		}

		const FFrameBufferMultiFormatLayered* FrameBuffer = static_cast<FFrameBufferMultiFormatLayered*>(input_image.video_frame_buffer().get());

		for (size_t StreamIdx = 0; StreamIdx < StreamInfos.size(); ++StreamIdx)
		{
			// Don't encode frames in resolutions that we don't intend to send.
			if (!StreamInfos[StreamIdx].bSendStream)
			{
				continue;
			}

			// extract the specific layer frame buffer
			webrtc::VideoFrame NewFrame(input_image);
			rtc::scoped_refptr<FFrameBufferMultiFormat> LayerFrameBuffer = FrameBuffer->GetLayer({ StreamInfos[StreamIdx].Width, StreamInfos[StreamIdx].Height });
			NewFrame.set_video_frame_buffer(LayerFrameBuffer);

			// Convert timestamp from RTP 90kHz clock.
			webrtc::Timestamp FrameTimestamp = webrtc::Timestamp::Micros((1000 * NewFrame.timestamp()) / 90);
			// If adapter is passed through and only one sw encoder does simulcast,
			// frame types for all streams should be passed to the encoder unchanged.
			// Otherwise a single per-encoder frame type is passed.
			std::vector<webrtc::VideoFrameType> StreamFrameTypes(StreamInfos.size() == 1 ? GetNumberOfStreams(CurrentCodec) : 1);
			if (bSendKeyFrame)
			{
				std::fill(StreamFrameTypes.begin(), StreamFrameTypes.end(), webrtc::VideoFrameType::kVideoFrameKey);
				StreamInfos[StreamIdx].KeyFrameRequest = false;
			}
			else
			{
				if (StreamInfos[StreamIdx].FramerateController->ShouldDropFrame(FrameTimestamp.us() * 1000))
				{
					continue;
				}
				std::fill(StreamFrameTypes.begin(), StreamFrameTypes.end(), webrtc::VideoFrameType::kVideoFrameDelta);
			}
			const int RtcError = StreamInfos[StreamIdx].Encoder->Encode(NewFrame, &StreamFrameTypes);
			if (RtcError != WEBRTC_VIDEO_CODEC_OK)
			{
				return RtcError;
			}
		}

		return WEBRTC_VIDEO_CODEC_OK;
	}

	int FVideoEncoderLayered::RegisterEncodeCompleteCallback(webrtc::EncodedImageCallback* callback)
	{
		EncodedCompleteCallback = callback;
		return WEBRTC_VIDEO_CODEC_OK;
	}

	void FVideoEncoderLayered::SetRates(const RateControlParameters& parameters)
	{
		if (!IsInitialized())
		{
			RTC_LOG(LS_WARNING) << "SetRates while not initialized";
			return;
		}

		if (parameters.framerate_fps < 1.0)
		{
			RTC_LOG(LS_WARNING) << "Invalid framerate: " << parameters.framerate_fps;
			return;
		}

		CurrentCodec.maxFramerate = static_cast<uint32_t>(parameters.framerate_fps + 0.5);

		if (StreamInfos.size() == 1)
		{
			// Not doing simulcast.
			StreamInfos[0].Encoder->SetRates(parameters);
			return;
		}

		for (size_t StreamIdx = 0; StreamIdx < StreamInfos.size(); ++StreamIdx)
		{
			uint32_t StreamBitrateKbps = parameters.bitrate.GetSpatialLayerSum(StreamIdx) / 1000;

			// Need a key frame if we have not sent this stream before.
			if (StreamBitrateKbps > 0 && !StreamInfos[StreamIdx].bSendStream)
			{
				StreamInfos[StreamIdx].KeyFrameRequest = true;
			}
			StreamInfos[StreamIdx].bSendStream = StreamBitrateKbps > 0;

			// Slice the temporal layers out of the full allocation and pass it on to
			// the encoder handling the current simulcast stream.
			RateControlParameters StreamParameters = parameters;
			StreamParameters.bitrate = webrtc::VideoBitrateAllocation();
			for (int i = 0; i < webrtc::kMaxTemporalStreams; ++i)
			{
				if (parameters.bitrate.HasBitrate(StreamIdx, i))
				{
					StreamParameters.bitrate.SetBitrate(0, i, parameters.bitrate.GetBitrate(StreamIdx, i));
				}
			}

			// Assign link allocation proportionally to spatial layer allocation.
			if (!parameters.bandwidth_allocation.IsZero() && parameters.bitrate.get_sum_bps() > 0)
			{
				StreamParameters.bandwidth_allocation =
					webrtc::DataRate::BitsPerSec((parameters.bandwidth_allocation.bps() * StreamParameters.bitrate.get_sum_bps()) / parameters.bitrate.get_sum_bps());
				// Make sure we don't allocate bandwidth lower than target bitrate.
				if (StreamParameters.bandwidth_allocation.bps() < StreamParameters.bitrate.get_sum_bps())
				{
					StreamParameters.bandwidth_allocation =
						webrtc::DataRate::BitsPerSec(StreamParameters.bitrate.get_sum_bps());
				}
			}
			StreamParameters.framerate_fps = std::min<double>(parameters.framerate_fps, StreamInfos[StreamIdx].FramerateController->GetMaxFramerate());
			StreamInfos[StreamIdx].Encoder->SetRates(StreamParameters);
		}
	}

	void FVideoEncoderLayered::OnPacketLossRateUpdate(float packet_loss_rate)
	{
		for (StreamInfo& Info : StreamInfos)
		{
			Info.Encoder->OnPacketLossRateUpdate(packet_loss_rate);
		}
	}

	void FVideoEncoderLayered::OnRttUpdate(int64_t rtt_ms)
	{
		for (StreamInfo& Info : StreamInfos)
		{
			Info.Encoder->OnRttUpdate(rtt_ms);
		}
	}

	void FVideoEncoderLayered::OnLossNotification(const LossNotification& loss_notification)
	{
		for (StreamInfo& Info : StreamInfos)
		{
			Info.Encoder->OnLossNotification(loss_notification);
		}
	}

	webrtc::EncodedImageCallback::Result FVideoEncoderLayered::OnEncodedImage(
		size_t stream_idx,
		const webrtc::EncodedImage& encodedImage,
		const webrtc::CodecSpecificInfo* codecSpecificInfo)
	{
		webrtc::EncodedImage StreamImage(encodedImage);
		webrtc::CodecSpecificInfo StreamCodecSpecific = *codecSpecificInfo;

		StreamImage.SetSpatialIndex(stream_idx);

		return EncodedCompleteCallback->OnEncodedImage(StreamImage, &StreamCodecSpecific);
	}

	bool FVideoEncoderLayered::IsInitialized() const
	{
		return Initialized;
	}

	webrtc::VideoEncoder::EncoderInfo FVideoEncoderLayered::GetEncoderInfo() const
	{
		if (StreamInfos.size() == 1)
		{
			// Not using simulcast adapting functionality, just pass through.
			return StreamInfos[0].Encoder->GetEncoderInfo();
		}

		VideoEncoder::EncoderInfo EncoderInfo;
		EncoderInfo.implementation_name = "PixelStreamingSimulcastEncoderAdapter";
		EncoderInfo.requested_resolution_alignment = 1;
		EncoderInfo.supports_native_handle = true;
		EncoderInfo.scaling_settings.thresholds = absl::nullopt;
		if (StreamInfos.empty())
		{
			return EncoderInfo;
		}

		EncoderInfo.scaling_settings = VideoEncoder::ScalingSettings::kOff;
		int NumActiveStreams = GetNumActiveStreams(CurrentCodec);

		for (size_t i = 0; i < StreamInfos.size(); ++i)
		{
			VideoEncoder::EncoderInfo EncoderImplInfo = StreamInfos[i].Encoder->GetEncoderInfo();

			if (i == 0)
			{
				// Encoder name indicates names of all sub-encoders.
				EncoderInfo.implementation_name += " (";
				EncoderInfo.implementation_name += EncoderImplInfo.implementation_name;

				EncoderInfo.supports_native_handle = EncoderImplInfo.supports_native_handle;
				EncoderInfo.has_trusted_rate_controller = EncoderImplInfo.has_trusted_rate_controller;
				EncoderInfo.is_hardware_accelerated = EncoderImplInfo.is_hardware_accelerated;
#if !WEBRTC_5414
				EncoderInfo.has_internal_source = EncoderImplInfo.has_internal_source;
#endif
			}
			else
			{
				EncoderInfo.implementation_name += ", ";
				EncoderInfo.implementation_name += EncoderImplInfo.implementation_name;

				// Native handle supported if any encoder supports it.
				EncoderInfo.supports_native_handle |= EncoderImplInfo.supports_native_handle;

				// Trusted rate controller only if all encoders have it.
				EncoderInfo.has_trusted_rate_controller &= EncoderImplInfo.has_trusted_rate_controller;

				// Uses hardware support if any of the encoders uses it.
				// For example, if we are having issues with down-scaling due to
				// pipelining delay in HW encoders we need higher encoder usage
				// thresholds in CPU adaptation.
				EncoderInfo.is_hardware_accelerated |= EncoderImplInfo.is_hardware_accelerated;

				// Has internal source only if all encoders have it.
#if !WEBRTC_5414
				EncoderInfo.has_internal_source &= EncoderImplInfo.has_internal_source;
#endif
			}

			// Nasty hack to allow us to manually convert VPX frames to I420 later in the encode block
			if (CurrentCodec.codecType == webrtc::kVideoCodecVP8)
			{
				EncoderInfo.supports_native_handle = true;
			}

			EncoderInfo.fps_allocation[i] = EncoderImplInfo.fps_allocation[0];
			EncoderInfo.requested_resolution_alignment = cricket::LeastCommonMultiple(EncoderInfo.requested_resolution_alignment, EncoderImplInfo.requested_resolution_alignment);
			if (NumActiveStreams == 1 && CurrentCodec.simulcastStream[i].active)
			{
				EncoderInfo.scaling_settings = EncoderImplInfo.scaling_settings;
			}
		}
		EncoderInfo.implementation_name += ")";

		return EncoderInfo;
	}
} // namespace UE::PixelStreaming
