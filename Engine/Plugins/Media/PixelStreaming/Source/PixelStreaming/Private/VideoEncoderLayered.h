// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WebRTCIncludes.h"
#include "VideoEncoderFactoryLayered.h"

namespace webrtc
{
	class SimulcastRateAllocator;
	class VideoEncoderFactory;
} // namespace webrtc

namespace UE::PixelStreaming
{
	/**
	 * A highly modified version of webrtc::simulcast_encoder_adapter
	 * The main video encoder for Pixel Streaming peers whether simulcast is being used or not.
	 * Will pull a FVideoEncoderSingleLayerXXX from the given factory.
	 */
	class FVideoEncoderLayered : public webrtc::VideoEncoder
	{
	public:
		FVideoEncoderLayered(FVideoEncoderFactoryLayered& InSimulcastFactory, const webrtc::SdpVideoFormat& format);
		~FVideoEncoderLayered() override;

		// Implements VideoEncoder.
		int Release() override;
		int InitEncode(const webrtc::VideoCodec* codec_settings, const webrtc::VideoEncoder::Settings& settings) override;
		int Encode(const webrtc::VideoFrame& input_image, const std::vector<webrtc::VideoFrameType>* frame_types) override;
		int RegisterEncodeCompleteCallback(webrtc::EncodedImageCallback* callback) override;

		void SetRates(const RateControlParameters& parameters) override;
		void OnPacketLossRateUpdate(float packet_loss_rate) override;
		void OnRttUpdate(int64_t rtt_ms) override;
		void OnLossNotification(const LossNotification& loss_notification) override;

		// Eventual handler for the contained encoders' EncodedImageCallbacks, but
		// called from an internal helper that also knows the correct stream
		// index.
		webrtc::EncodedImageCallback::Result OnEncodedImage(
			size_t stream_idx,
			const webrtc::EncodedImage& encoded_image,
			const webrtc::CodecSpecificInfo* codec_specific_info);

		EncoderInfo GetEncoderInfo() const override;

	private:
		struct StreamInfo
		{
			StreamInfo(std::unique_ptr<webrtc::VideoEncoder> InEncoder,
				std::unique_ptr<webrtc::EncodedImageCallback> InCallback,
				std::unique_ptr<webrtc::FramerateController> InFramerateController,
				uint16_t InWidth,
				uint16_t InHeight,
				bool bInKeyFrame,
				bool bInSendStream)
				: Encoder(std::move(InEncoder))
				, Callback(std::move(InCallback))
				, FramerateController(std::move(InFramerateController))
				, Width(InWidth)
				, Height(InHeight)
				, KeyFrameRequest(bInKeyFrame)
				, bSendStream(bInSendStream)
			{
			}
			std::unique_ptr<webrtc::VideoEncoder> Encoder;
			std::unique_ptr<webrtc::EncodedImageCallback> Callback;
			std::unique_ptr<webrtc::FramerateController> FramerateController;
			uint16_t Width;
			uint16_t Height;
			bool KeyFrameRequest;
			bool bSendStream;
		};

		bool IsInitialized() const;

		TAtomic<bool> Initialized;
		FVideoEncoderFactoryLayered& SimulcastEncoderFactory;
		const webrtc::SdpVideoFormat VideoFormat;
		webrtc::VideoCodec CurrentCodec;
		FCriticalSection StreamInfosGuard;
		std::vector<StreamInfo> StreamInfos;
		webrtc::EncodedImageCallback* EncodedCompleteCallback;
	};
} // namespace UE::PixelStreaming
