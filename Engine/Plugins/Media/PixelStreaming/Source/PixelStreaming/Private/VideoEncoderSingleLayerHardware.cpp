// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderSingleLayerHardware.h"
#include "VideoEncoderFactorySingleLayer.h"
#include "FrameBufferMultiFormat.h"
#include "Settings.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Stats.h"
#include "PixelStreamingPeerConnection.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelStreamingTrace.h"
#include "FrameBufferRHI.h"

namespace UE::PixelStreaming
{
	FVideoEncoderSingleLayerHardware::FVideoEncoderSingleLayerHardware(FVideoEncoderFactorySingleLayer& InFactory, EPixelStreamingCodec Codec)
		: Factory(InFactory)
		, Codec(Codec)
	{
	}

	FVideoEncoderSingleLayerHardware::~FVideoEncoderSingleLayerHardware()
	{
		Factory.ReleaseVideoEncoder(this);
	}

	void SetInitialSettings(webrtc::VideoCodec const* InCodecSettings, FVideoEncoderConfig& VideoConfig)
	{
		VideoConfig.Preset = PixelStreaming::Settings::GetEncoderPreset();
		VideoConfig.LatencyMode = EAVLatencyMode::UltraLowLatency;
		// We set width and height to zero here because we initialize encoder from the first frame dimensions, not this config.
		VideoConfig.Width = 0;
		VideoConfig.Height = 0;
		VideoConfig.TargetFramerate = InCodecSettings->maxFramerate;
		VideoConfig.TargetBitrate = InCodecSettings->startBitrate;
		VideoConfig.MaxBitrate = InCodecSettings->maxBitrate;
		VideoConfig.MinQuality = (1.0f - (PixelStreaming::Settings::CVarPixelStreamingEncoderMaxQP.GetValueOnAnyThread() / 100.0f)) * 51.0f;
		VideoConfig.MaxQuality = (1.0f - (PixelStreaming::Settings::CVarPixelStreamingEncoderMinQP.GetValueOnAnyThread() / 100.0f)) * 51.0f;
		VideoConfig.RateControlMode = PixelStreaming::Settings::GetRateControlCVar();
		VideoConfig.bFillData = PixelStreaming::Settings::CVarPixelStreamingEnableFillerData.GetValueOnAnyThread();
		VideoConfig.KeyframeInterval = PixelStreaming::Settings::CVarPixelStreamingEncoderKeyframeInterval.GetValueOnAnyThread();
		VideoConfig.MultipassMode = PixelStreaming::Settings::GetMultipassCVar();
	}

	int FVideoEncoderSingleLayerHardware::InitEncode(webrtc::VideoCodec const* InCodecSettings, VideoEncoder::Settings const& Settings)
	{
		WebRtcProposedTargetBitrate = InCodecSettings->startBitrate;

		switch (Codec)
		{
		case EPixelStreamingCodec::H264:
		{
			TUniquePtr<FVideoEncoderConfigH264> VideoConfig = MakeUnique<FVideoEncoderConfigH264>();
			SetInitialSettings(InCodecSettings, *VideoConfig);
			VideoConfig->Profile = PixelStreaming::Settings::GetH264Profile();
			VideoConfig->RepeatSPSPPS = true;
			VideoConfig->IntraRefreshPeriodFrames = PixelStreaming::Settings::CVarPixelStreamingEncoderIntraRefreshPeriodFrames.GetValueOnAnyThread();
			VideoConfig->IntraRefreshCountFrames = PixelStreaming::Settings::CVarPixelStreamingEncoderIntraRefreshCountFrames.GetValueOnAnyThread();
			VideoConfig->KeyframeInterval = PixelStreaming::Settings::CVarPixelStreamingEncoderKeyframeInterval.GetValueOnAnyThread();
			// The WebRTC spec can only guarantee that the Baseline profile is supported. Therefore we use Baseline, but enable these extra
			// features to improve bitrate usage
			VideoConfig->AdaptiveTransformMode = EH264AdaptiveTransformMode::Enable;
			VideoConfig->EntropyCodingMode = EH264EntropyCodingMode::CABAC;
			InitialVideoConfig = MoveTemp(VideoConfig);
			return WEBRTC_VIDEO_CODEC_OK;
		}
		case EPixelStreamingCodec::AV1:
		{
			TUniquePtr<FVideoEncoderConfigAV1> VideoConfig = MakeUnique<FVideoEncoderConfigAV1>();
			SetInitialSettings(InCodecSettings, *VideoConfig);
			VideoConfig->RepeatSeqHdr = true;
			VideoConfig->IntraRefreshPeriodFrames = PixelStreaming::Settings::CVarPixelStreamingEncoderIntraRefreshPeriodFrames.GetValueOnAnyThread();
			VideoConfig->IntraRefreshCountFrames = PixelStreaming::Settings::CVarPixelStreamingEncoderIntraRefreshCountFrames.GetValueOnAnyThread();
			VideoConfig->KeyframeInterval = PixelStreaming::Settings::CVarPixelStreamingEncoderKeyframeInterval.GetValueOnAnyThread();
			InitialVideoConfig = MoveTemp(VideoConfig);
			return WEBRTC_VIDEO_CODEC_OK;
		}
		default:
			return WEBRTC_VIDEO_CODEC_ERROR;
		}
	}

	int32 FVideoEncoderSingleLayerHardware::RegisterEncodeCompleteCallback(webrtc::EncodedImageCallback* callback)
	{
		OnEncodedImageCallback = callback;
		return WEBRTC_VIDEO_CODEC_OK;
	}

	int32 FVideoEncoderSingleLayerHardware::Release()
	{
		OnEncodedImageCallback = nullptr;
		return WEBRTC_VIDEO_CODEC_OK;
	}

	void FVideoEncoderSingleLayerHardware::UpdateFrameMetadataPreEncode(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.UseCount++;
		FrameMetadata.LastEncodeStartTime = rtc::TimeMillis();
		if (FrameMetadata.UseCount == 1)
		{
			FrameMetadata.FirstEncodeStartTime = FrameMetadata.LastEncodeStartTime;
		}
	}

	void FVideoEncoderSingleLayerHardware::UpdateFrameMetadataPostEncode(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.LastEncodeEndTime = rtc::TimeMillis();

		FStats::Get()->AddFrameTimingStats(FrameMetadata);
	}

	void FVideoEncoderSingleLayerHardware::UpdateFrameMetadataPrePacketization(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.LastPacketizationStartTime = rtc::TimeMillis();
		if (FrameMetadata.UseCount == 1)
		{
			FrameMetadata.FirstPacketizationStartTime = FrameMetadata.LastPacketizationStartTime;
		}
	}

	void FVideoEncoderSingleLayerHardware::UpdateFrameMetadataPostPacketization(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.LastPacketizationEndTime = rtc::TimeMillis();

		FStats::Get()->AddFrameTimingStats(FrameMetadata);
	}

	webrtc::VideoFrame FVideoEncoderSingleLayerHardware::WrapAdaptedFrame(const webrtc::VideoFrame& ExistingFrame, const IPixelCaptureOutputFrame& AdaptedLayer)
	{
		webrtc::VideoFrame NewFrame(ExistingFrame);
		const FPixelCaptureOutputFrameRHI& RHILayer = StaticCast<const FPixelCaptureOutputFrameRHI&>(AdaptedLayer);
		// TODO-TE
#if WEBRTC_5414
		rtc::scoped_refptr<FFrameBufferRHI> RHIBuffer = rtc::make_ref_counted<FFrameBufferRHI>(MakeShared<FVideoResourceRHI>(HardwareEncoder.Pin()->GetDevice().ToSharedRef(), FVideoResourceRHI::FRawData{ RHILayer.GetFrameTexture(), nullptr, 0 }));
#else
		rtc::scoped_refptr<FFrameBufferRHI> RHIBuffer = new rtc::RefCountedObject<FFrameBufferRHI>(MakeShared<FVideoResourceRHI>(HardwareEncoder.Pin()->GetDevice().ToSharedRef(), FVideoResourceRHI::FRawData{ RHILayer.GetFrameTexture(), nullptr, 0 }));
#endif
		NewFrame.set_video_frame_buffer(RHIBuffer);
		return NewFrame;
	}

	void FVideoEncoderSingleLayerHardware::LateInitHardwareEncoder(uint32 StreamId)
	{
		checkf(InitialVideoConfig->Width > 0 && InitialVideoConfig->Height > 0, TEXT("Encoder config must have non-zero width and height."));

		switch (Codec)
		{
		case EPixelStreamingCodec::H264:
		{
			FVideoEncoderConfigH264& VideoConfig = *StaticCast<FVideoEncoderConfigH264*>(InitialVideoConfig.Get());
			HardwareEncoder = Factory.GetOrCreateHardwareEncoder(StreamId, VideoConfig);
			break;
		}
		case EPixelStreamingCodec::AV1:
		{
			FVideoEncoderConfigAV1& VideoConfig = *StaticCast<FVideoEncoderConfigAV1*>(InitialVideoConfig.Get());
			HardwareEncoder = Factory.GetOrCreateHardwareEncoder(StreamId, VideoConfig);
			break;
		}
		}
	}

	int32 FVideoEncoderSingleLayerHardware::Encode(webrtc::VideoFrame const& frame, std::vector<webrtc::VideoFrameType> const* frame_types)
	{
		const FFrameBufferMultiFormat* FrameBuffer = StaticCast<FFrameBufferMultiFormat*>(frame.video_frame_buffer().get());
		EncodingStreamId = FrameBuffer->GetSourceStreamId();

		IPixelCaptureOutputFrame* AdaptedLayer = FrameBuffer->RequestFormat(PixelCaptureBufferFormat::FORMAT_RHI);
		// Check whether the output frame is valid because null frames are passed to stream sharing encoders.
		if (AdaptedLayer == nullptr)
		{
			// probably the first request which starts the adapt pipeline for this format
			return WEBRTC_VIDEO_CODEC_OK;
		}

		const int32 FrameWidth = AdaptedLayer->GetWidth();
		const int32 FrameHeight = AdaptedLayer->GetHeight();

		// We late init here so we can pull the stream ID off the incoming frames and pull the correct encoder for the stream
		// earlier locations do not have this information.
		if (!HardwareEncoder.IsValid())
		{
			InitialVideoConfig->Width = FrameWidth;
			InitialVideoConfig->Height = FrameHeight;
			LateInitHardwareEncoder(EncodingStreamId);
		}

		// Update the encoding config using the incoming frame resolution
		UpdateConfig(FrameWidth, FrameHeight);

		if (TSharedPtr<FVideoEncoderHardware> const& PinnedHardwareEncoder = HardwareEncoder.Pin())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Hardware Encoding", PixelStreamingChannel);

			uint64 CurrentFrameId = AdaptedLayer->Metadata.Id;
			if (LastEncodedFrameId > 0)
			{
				if (CurrentFrameId < LastEncodedFrameId)
				{
					UE_LOGF(LogPixelStreaming, Verbose, "Out of order frame entered hardware encoder.")
				}
				else if (CurrentFrameId == LastEncodedFrameId)
				{
					UE_LOGF(LogPixelStreaming, Verbose, "Encoding the same frame.")
				}
				else if (CurrentFrameId - LastEncodedFrameId > 1)
				{
					UE_LOGF(LogPixelStreaming, Verbose, "Missed encoding %lld frames", CurrentFrameId - LastEncodedFrameId - 1);
				}
			}
			LastEncodedFrameId = CurrentFrameId;

			UpdateFrameMetadataPreEncode(*AdaptedLayer);
			const FPixelCaptureOutputFrameRHI& RHILayer = StaticCast<const FPixelCaptureOutputFrameRHI&>(*AdaptedLayer);
			rtc::scoped_refptr<FFrameBufferRHI> RHIBuffer;

			FPixelCaptureOutputFrameRHI::WrapperFormatData WrapperData;
			if (RHILayer.GetWrapperFormatData(WrapperData))
			{
				// If we have a wrapper format we need to also provide a descriptor of the raw format the data
				// is transported in
				FVideoDescriptor RawDescriptor = FVideoResourceRHI::GetDescriptorFrom(PinnedHardwareEncoder->GetDevice().ToSharedRef(), RHILayer.GetFrameTexture());

				// If we have a wrapper format for our resource, make sure the video descriptor specifies it
				RHIBuffer = new rtc::RefCountedObject<FFrameBufferRHI>(MakeShared<FVideoResourceRHI>(
					PinnedHardwareEncoder->GetDevice().ToSharedRef(),
					FVideoResourceRHI::FRawData{ RHILayer.GetFrameTexture(), nullptr, 0 },
					FVideoDescriptor(static_cast<EVideoFormat>(WrapperData.Format), WrapperData.Width, WrapperData.Height, RawDescriptor)));
			}
			else
			{
				RHIBuffer = new rtc::RefCountedObject<FFrameBufferRHI>(MakeShared<FVideoResourceRHI>(
					PinnedHardwareEncoder->GetDevice().ToSharedRef(),
					FVideoResourceRHI::FRawData{ RHILayer.GetFrameTexture(), nullptr, 0 }));
			}

			const bool bKeyframe = (frame_types && (*frame_types)[0] == webrtc::VideoFrameType::kVideoFrameKey) || Factory.ShouldForceKeyframe();

			PinnedHardwareEncoder->SendFrame(RHIBuffer->GetVideoResource(), frame.timestamp_us(), bKeyframe);

			Factory.UnforceKeyFrame();

			UpdateFrameMetadataPostEncode(*AdaptedLayer);

			FVideoPacket Packet;
			while (PinnedHardwareEncoder->ReceivePacket(Packet))
			{
				// Note: This works fine for 1:1 synchronous encoding, but if we ever need to relate frames to packets for async or gop this will need thought.
				webrtc::EncodedImage Image;

				Image.timing_.packetization_finish_ms = FTimespan::FromSeconds(FPlatformTime::Seconds()).GetTotalMilliseconds();
				Image.timing_.encode_start_ms = AdaptedLayer->Metadata.LastEncodeStartTime;
				Image.timing_.encode_finish_ms = AdaptedLayer->Metadata.LastEncodeEndTime;
				Image.timing_.flags = webrtc::VideoSendTiming::kTriggeredByTimer;
				Image.SetEncodedData(webrtc::EncodedImageBuffer::Create(Packet.DataPtr.Get(), Packet.DataSize));
				Image._encodedWidth = RHIBuffer->width();
				Image._encodedHeight = RHIBuffer->height();
				Image._frameType = Packet.bIsKeyframe ? webrtc::VideoFrameType::kVideoFrameKey : webrtc::VideoFrameType::kVideoFrameDelta;
				Image.content_type_ = webrtc::VideoContentType::UNSPECIFIED;
				Image.qp_ = Packet.QP;
				Image.SetSpatialIndex(0);
				Image.rotation_ = webrtc::VideoRotation::kVideoRotation_0;
				Image.SetTimestamp(frame.timestamp());
				Image.capture_time_ms_ = frame.timestamp_us() / 1000.0;

				webrtc::CodecSpecificInfo CodecInfo;
				switch (Codec)
				{
				case EPixelStreamingCodec::H264:
					CodecInfo.codecType = webrtc::VideoCodecType::kVideoCodecH264;
					CodecInfo.codecSpecific.H264.packetization_mode = webrtc::H264PacketizationMode::NonInterleaved;
					CodecInfo.codecSpecific.H264.temporal_idx = webrtc::kNoTemporalIdx;
					CodecInfo.codecSpecific.H264.idr_frame = Packet.bIsKeyframe;
					CodecInfo.codecSpecific.H264.base_layer_sync = false;

					break;
				case EPixelStreamingCodec::AV1:
					CodecInfo.codecType = webrtc::VideoCodecType::kVideoCodecAV1;
					break;
				}

#if PIXELSTREAMING_DUMP_ENCODING
				Packet.WriteToFile(TEXT("SingleLayerHardware"));
#endif

				UpdateFrameMetadataPrePacketization(*AdaptedLayer);

				Factory.OnEncodedImage(Image, &CodecInfo, EncodingStreamId);

				UpdateFrameMetadataPostPacketization(*AdaptedLayer);
			}

			return WEBRTC_VIDEO_CODEC_OK;
		}
		else
		{
			return WEBRTC_VIDEO_CODEC_ERROR;
		}
	}

	// Pass rate control parameters from WebRTC to our encoder
	// This is how WebRTC can control the bitrate/framerate of the encoder.
	void FVideoEncoderSingleLayerHardware::SetRates(RateControlParameters const& parameters)
	{
		PendingRateChange.Emplace(parameters);
	}

	webrtc::VideoEncoder::EncoderInfo FVideoEncoderSingleLayerHardware::GetEncoderInfo() const
	{
		VideoEncoder::EncoderInfo info;
		info.supports_native_handle = true;
		info.is_hardware_accelerated = true;
#if !WEBRTC_5414
		info.has_internal_source = false;
#endif
		info.supports_simulcast = false;
		info.implementation_name = TCHAR_TO_UTF8(*FString::Printf(TEXT("PIXEL_STREAMING_HW_ENCODER_%s"), GDynamicRHI->GetName()));

		const int LowQP = UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCLowQpThreshold.GetValueOnAnyThread();
		const int HighQP = UE::PixelStreaming::Settings::CVarPixelStreamingWebRTCHighQpThreshold.GetValueOnAnyThread();
		info.scaling_settings = VideoEncoder::ScalingSettings(LowQP, HighQP);

		// if true means HW encoder must be perfect and drop frames itself etc
		info.has_trusted_rate_controller = false;

		return info;
	}

	void FVideoEncoderSingleLayerHardware::UpdateConfig(uint32 width, uint32 height)
	{
		InitialVideoConfig->Width = width;
		InitialVideoConfig->Height = height;

		if (TSharedPtr<FVideoEncoderHardware> const& PinnedHardwareEncoder = HardwareEncoder.Pin())
		{
			FVideoEncoderConfig VideoConfigMinimal = PinnedHardwareEncoder->GetMinimalConfig();
			FVideoEncoderConfig* VideoConfig = &VideoConfigMinimal;

			switch (Codec)
			{
			case EPixelStreamingCodec::H264:
				if (PinnedHardwareEncoder->GetInstance()->Has<FVideoEncoderConfigH264>())
				{
					FVideoEncoderConfigH264& VideoConfigH264 = PinnedHardwareEncoder->GetInstance()->Edit<FVideoEncoderConfigH264>();
					VideoConfig = &VideoConfigH264;

					VideoConfigH264.Profile = UE::PixelStreaming::Settings::GetH264Profile();
				}

				break;
			case EPixelStreamingCodec::AV1:
				if (PinnedHardwareEncoder->GetInstance()->Has<FVideoEncoderConfigAV1>())
				{
					FVideoEncoderConfigAV1& VideoConfigAV1 = PinnedHardwareEncoder->GetInstance()->Edit<FVideoEncoderConfigAV1>();
					VideoConfig = &VideoConfigAV1;
				}

				break;
			}

			if (PendingRateChange.IsSet())
			{
				const RateControlParameters& RateChangeParams = PendingRateChange.GetValue();

				VideoConfig->TargetFramerate = RateChangeParams.framerate_fps;

				// We store what WebRTC wants as the bitrate, even if we are overriding it, so we can restore back to it when user stops using CVar.
				WebRtcProposedTargetBitrate = RateChangeParams.bitrate.get_sum_kbps() * 1000;

				// Clear the rate change request
				PendingRateChange.Reset();
			}

			// Change encoder settings through CVars
			const int32 MaxBitrateCVar = UE::PixelStreaming::Settings::CVarPixelStreamingEncoderMaxBitrate.GetValueOnAnyThread();
			const int32 TargetBitrateCVar = UE::PixelStreaming::Settings::CVarPixelStreamingEncoderTargetBitrate.GetValueOnAnyThread();
			const int32 MaxQPCVar = UE::PixelStreaming::Settings::CVarPixelStreamingEncoderMaxQP.GetValueOnAnyThread();
			const int32 MinQPCVar = UE::PixelStreaming::Settings::CVarPixelStreamingEncoderMinQP.GetValueOnAnyThread();
			const ERateControlMode RateControlCVar = UE::PixelStreaming::Settings::GetRateControlCVar();
			const EMultipassMode MultiPassCVar = UE::PixelStreaming::Settings::GetMultipassCVar();
			const bool bFillerDataCVar = UE::PixelStreaming::Settings::CVarPixelStreamingEnableFillerData.GetValueOnAnyThread();

			VideoConfig->MaxBitrate = MaxBitrateCVar > -1 ? MaxBitrateCVar : VideoConfig->MaxBitrate;
			VideoConfig->TargetBitrate = TargetBitrateCVar > -1 ? TargetBitrateCVar : WebRtcProposedTargetBitrate;
			VideoConfig->MinQuality = (1.0f - (MaxQPCVar / 51.0f)) * 100.0f;
			VideoConfig->MaxQuality = (1.0f - (MinQPCVar / 51.0f)) * 100.0f;
			VideoConfig->RateControlMode = RateControlCVar;
			VideoConfig->MultipassMode = MultiPassCVar;
			VideoConfig->bFillData = bFillerDataCVar;
			VideoConfig->Width = width;
			VideoConfig->Height = height;

			PinnedHardwareEncoder->SetMinimalConfig(*VideoConfig);
		}
	}

	void FVideoEncoderSingleLayerHardware::SendEncodedImage(webrtc::EncodedImage const& encoded_image, webrtc::CodecSpecificInfo const* codec_specific_info, uint32 StreamId)
	{
		if (StreamId == EncodingStreamId)
		{
			MaybeDumpFrame(encoded_image);
			if (OnEncodedImageCallback)
			{
				OnEncodedImageCallback->OnEncodedImage(encoded_image, codec_specific_info);
			}
		}
	}

	void FVideoEncoderSingleLayerHardware::MaybeDumpFrame(webrtc::EncodedImage const& encoded_image)
	{
		// Dump encoded frames to file for debugging if CVar is turned on.
		if (UE::PixelStreaming::Settings::CVarPixelStreamingDebugDumpFrame.GetValueOnAnyThread())
		{
			static IFileHandle* FileHandle = nullptr;
			if (!FileHandle)
			{
				IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
				FString TempFilePath = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), TEXT("encoded_frame"), TEXT(".raw"));
				FileHandle = PlatformFile.OpenWrite(*TempFilePath);
				check(FileHandle);
				// Note: To examine individual frames from this dump: ffmpeg -i "encoded_frame78134A5047638BB99AE1D88471E5E513.raw" "frames/out-%04d.jpg"
			}

			FileHandle->Write(encoded_image.data(), encoded_image.size());
			FileHandle->Flush();
		}
	}
} // namespace UE::PixelStreaming
