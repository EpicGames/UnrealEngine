// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WebRTCIncludes.h"
#include "IPixelCaptureOutputFrame.h"
#include "PixelStreamingCodec.h"
#include "VideoEncoderFactorySingleLayer.h"

namespace UE::PixelStreaming
{
	class FVideoEncoderFactorySingleLayer;

	/**
	 * A hardware encoder for a single layer of a frame.
	 */
	class FVideoEncoderSingleLayerHardware : public webrtc::VideoEncoder
	{
	public:
		FVideoEncoderSingleLayerHardware(FVideoEncoderFactorySingleLayer& InFactory, EPixelStreamingCodec Codec);
		virtual ~FVideoEncoderSingleLayerHardware() override;

		// WebRTC Interface
		virtual int InitEncode(webrtc::VideoCodec const* codec_settings, webrtc::VideoEncoder::Settings const& settings) override;
		virtual int32 RegisterEncodeCompleteCallback(webrtc::EncodedImageCallback* callback) override;
		virtual int32 Release() override;
		virtual int32 Encode(webrtc::VideoFrame const& frame, std::vector<webrtc::VideoFrameType> const* frame_types) override;
		virtual void SetRates(RateControlParameters const& parameters) override;
		virtual webrtc::VideoEncoder::EncoderInfo GetEncoderInfo() const override;

		// Note: These funcs can also be overriden but are not pure virtual
		// virtual void SetFecControllerOverride(FecControllerOverride* fec_controller_override) override;
		// virtual void OnPacketLossRateUpdate(float packet_loss_rate) override;
		// virtual void OnRttUpdate(int64_t rtt_ms) override;
		// virtual void OnLossNotification(const LossNotification& loss_notification) override;
		// End WebRTC Interface.

		void SendEncodedImage(webrtc::EncodedImage const& encoded_image, webrtc::CodecSpecificInfo const* codec_specific_info, uint32 StreamId);

	private:
		void LateInitHardwareEncoder(uint32 StreamId);
		void UpdateConfig(uint32 width, uint32 height);
		void MaybeDumpFrame(webrtc::EncodedImage const& encoded_image);

		void UpdateFrameMetadataPreEncode(IPixelCaptureOutputFrame& Frame);
		void UpdateFrameMetadataPostEncode(IPixelCaptureOutputFrame& Frame);
		void UpdateFrameMetadataPrePacketization(IPixelCaptureOutputFrame& Frame);
		void UpdateFrameMetadataPostPacketization(IPixelCaptureOutputFrame& Frame);
		webrtc::VideoFrame WrapAdaptedFrame(const webrtc::VideoFrame& ExistingFrame, const IPixelCaptureOutputFrame& AdaptedLayer);
		
		FVideoEncoderFactorySingleLayer& Factory;
		EPixelStreamingCodec Codec;

		TUniquePtr<FVideoEncoderConfig> InitialVideoConfig;
		TWeakPtr<FVideoEncoderHardware> HardwareEncoder;

		// We store this so we can restore back to it if the user decides to use then stop using the PixelStreaming.Encoder.TargetBitrate CVar.
		int32 WebRtcProposedTargetBitrate = 5000000;

		webrtc::EncodedImageCallback* OnEncodedImageCallback = nullptr;

		// WebRTC may request a bitrate/framerate change using SetRates(), we only respect this if this encoder is actually encoding
		// so we use this optional object to store a rate change and act upon it when this encoder does its next call to Encode().
		TOptional<RateControlParameters> PendingRateChange;

		// used to key into active hardware encoders and pull the correct encoder for the stream.
		uint32 EncodingStreamId;

		// Used to track how often we encode the same frame
		uint64 LastEncodedFrameId = 0;
	};
} // namespace UE::PixelStreaming
