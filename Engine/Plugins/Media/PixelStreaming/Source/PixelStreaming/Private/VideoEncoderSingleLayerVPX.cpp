// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderSingleLayerVPX.h"
#include "FrameBufferMultiFormat.h"
#include "Stats.h"
#include "FrameBufferI420.h"

#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelStreamingTrace.h"

namespace UE::PixelStreaming
{
	FVideoEncoderSingleLayerVPX::FVideoEncoderSingleLayerVPX(int VPXVersion)
	{
		if (VPXVersion == 8)
		{
			WebRTCVPXEncoder = webrtc::VP8Encoder::Create();
		}
		else if (VPXVersion == 9)
		{
			WebRTCVPXEncoder = webrtc::VP9Encoder::Create();
		}
	}

	int FVideoEncoderSingleLayerVPX::InitEncode(webrtc::VideoCodec const* InCodecSettings, VideoEncoder::Settings const& settings)
	{
		return WebRTCVPXEncoder->InitEncode(InCodecSettings, settings);
	}

	int32 FVideoEncoderSingleLayerVPX::RegisterEncodeCompleteCallback(webrtc::EncodedImageCallback* callback)
	{
		return WebRTCVPXEncoder->RegisterEncodeCompleteCallback(callback);
	}

	int32 FVideoEncoderSingleLayerVPX::Release()
	{
		return WebRTCVPXEncoder->Release();
	}

	void FVideoEncoderSingleLayerVPX::UpdateFrameMetadataPreEncode(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.UseCount++;
		FrameMetadata.LastEncodeStartTime = rtc::TimeMillis();
		if (FrameMetadata.UseCount == 1)
		{
			FrameMetadata.FirstEncodeStartTime = FrameMetadata.LastEncodeStartTime;
		}
	}

	void FVideoEncoderSingleLayerVPX::UpdateFrameMetadataPostEncode(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.LastEncodeEndTime = rtc::TimeMillis();

		FStats::Get()->AddFrameTimingStats(FrameMetadata);
	}

	webrtc::VideoFrame FVideoEncoderSingleLayerVPX::WrapAdaptedFrame(const webrtc::VideoFrame& ExistingFrame, const IPixelCaptureOutputFrame& AdaptedLayer)
	{
		webrtc::VideoFrame NewFrame(ExistingFrame);
		const FPixelCaptureOutputFrameI420& AdaptedLayerI420 = StaticCast<const FPixelCaptureOutputFrameI420&>(AdaptedLayer);
#if WEBRTC_5414
		rtc::scoped_refptr<FFrameBufferI420> I420Buffer = rtc::make_ref_counted<FFrameBufferI420>(AdaptedLayerI420.GetI420Buffer());
#else
		rtc::scoped_refptr<FFrameBufferI420> I420Buffer = new rtc::RefCountedObject<FFrameBufferI420>(AdaptedLayerI420.GetI420Buffer());
#endif
		NewFrame.set_video_frame_buffer(I420Buffer);
		return NewFrame;
	}

	int32 FVideoEncoderSingleLayerVPX::Encode(webrtc::VideoFrame const& frame, std::vector<webrtc::VideoFrameType> const* frame_types)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming VPX Encoding", PixelStreamingChannel);
		checkf(WebRTCVPXEncoder, TEXT("WebRTCVPXEncoder was null. Should never happen."));

		const FFrameBufferMultiFormat* FrameBuffer = StaticCast<FFrameBufferMultiFormat*>(frame.video_frame_buffer().get());
		IPixelCaptureOutputFrame* AdaptedLayer = FrameBuffer->RequestFormat(PixelCaptureBufferFormat::FORMAT_I420);

		if (AdaptedLayer == nullptr)
		{
			// probably the first request which starts the adapt pipeline for this format
			return WEBRTC_VIDEO_CODEC_OK;
		}

		int lidx = AdaptedLayer->Metadata.Layer;
		UpdateFrameMetadataPreEncode(*AdaptedLayer);
		const webrtc::VideoFrame NewFrame = WrapAdaptedFrame(frame, *AdaptedLayer);
		const int32 EncodeResult = WebRTCVPXEncoder->Encode(NewFrame, frame_types);
		UpdateFrameMetadataPostEncode(*AdaptedLayer);

		return EncodeResult;
	}

	// Pass rate control parameters from WebRTC to our encoder
	// This is how WebRTC can control the bitrate/framerate of the encoder.
	void FVideoEncoderSingleLayerVPX::SetRates(RateControlParameters const& parameters)
	{
		WebRTCVPXEncoder->SetRates(parameters);
	}

	webrtc::VideoEncoder::EncoderInfo FVideoEncoderSingleLayerVPX::GetEncoderInfo() const
	{
		VideoEncoder::EncoderInfo info = WebRTCVPXEncoder->GetEncoderInfo();
		info.supports_native_handle = true;
		return info;
	}
} // namespace UE::PixelStreaming
