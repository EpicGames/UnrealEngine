// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WebRTCIncludes.h"
#include "IPixelCaptureOutputFrame.h"

namespace UE::PixelStreaming
{
	/**
	 * A VPX encoder for a single layer of a frame.
	 */
	class FVideoEncoderSingleLayerVPX : public webrtc::VideoEncoder
	{
	public:
		FVideoEncoderSingleLayerVPX(int VPXVersion);
		virtual ~FVideoEncoderSingleLayerVPX() = default;

		// WebRTC Interface
		virtual int InitEncode(webrtc::VideoCodec const* codec_settings, webrtc::VideoEncoder::Settings const& settings) override;
		virtual int32 RegisterEncodeCompleteCallback(webrtc::EncodedImageCallback* callback) override;
		virtual int32 Release() override;
		virtual int32 Encode(webrtc::VideoFrame const& frame, std::vector<webrtc::VideoFrameType> const* frame_types) override;
		virtual void SetRates(RateControlParameters const& parameters) override;
		virtual webrtc::VideoEncoder::EncoderInfo GetEncoderInfo() const override;

	private:
		void UpdateFrameMetadataPreEncode(IPixelCaptureOutputFrame& Frame);
		void UpdateFrameMetadataPostEncode(IPixelCaptureOutputFrame& Frame);
		webrtc::VideoFrame WrapAdaptedFrame(const webrtc::VideoFrame& ExistingFrame, const IPixelCaptureOutputFrame& AdaptedLayer);

		std::unique_ptr<webrtc::VideoEncoder> WebRTCVPXEncoder;
	};
} // namespace UE::PixelStreaming
