// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoSource.h"
#include "Settings.h"
#include "FrameBufferMultiFormat.h"
#include "PixelStreamingTrace.h"

namespace UE::PixelStreaming
{
	FVideoSource::FVideoSource(bool bInRequireUniqueId, TSharedPtr<FPixelStreamingVideoInput> InVideoInput, const TFunction<bool()>& InShouldGenerateFramesCheck)
		: CurrentState(webrtc::MediaSourceInterface::SourceState::kInitializing)
		, VideoInput(InVideoInput)
		, ShouldGenerateFramesCheck(InShouldGenerateFramesCheck)
		, StreamId(0)
		, bRequireUniqueId(bInRequireUniqueId)
	{
		if (bRequireUniqueId)
		{
			StreamId = FPixelStreamingVideoInput::GetUniqueStreamId();
		}
	}

	void FVideoSource::MaybePushFrame()
	{
		if (VideoInput->IsReady())
		{
			CurrentState = webrtc::MediaSourceInterface::SourceState::kLive;

			if(ShouldGenerateFramesCheck())
			{
				PushFrame();
			}
			// We always send a frame for WebRTC purposes, but empty frames early exit out of the encoder.
			else
			{
				PushEmptyFrame();
			}
		}
	}

	void FVideoSource::SetStreamId(rtc::scoped_refptr<webrtc::VideoFrameBuffer> FrameBuffer) const
	{
		FFrameBufferMultiFormatBase* FrameBufferMultiFormat = StaticCast<FFrameBufferMultiFormatBase*>(FrameBuffer.get());
		FrameBufferMultiFormat->SetSourceStreamId(StreamId);
	}

	void FVideoSource::PushFrame()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Push Video Frame", PixelStreamingChannel);
		static int32 FrameId = 1;

		rtc::scoped_refptr<webrtc::VideoFrameBuffer> FrameBuffer = VideoInput->GetFrameBuffer();
		if (bRequireUniqueId)
		{
			SetStreamId(FrameBuffer);
		}
		check(FrameBuffer->width() != 0 && FrameBuffer->height() != 0);
		webrtc::VideoFrame Frame = webrtc::VideoFrame::Builder()
									   .set_video_frame_buffer(FrameBuffer)
									   .set_timestamp_us(rtc::TimeMicros())
									   .set_rotation(webrtc::VideoRotation::kVideoRotation_0)
									   .set_id(FrameId++)
									   .build();
		OnFrame(Frame);
	}

	void FVideoSource::PushEmptyFrame()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Push Empty Video Frame", PixelStreamingChannel);
		static int32 EmptyFrameId = 1;

		rtc::scoped_refptr<webrtc::VideoFrameBuffer> FrameBuffer = VideoInput->GetEmptyFrameBuffer();
		if (bRequireUniqueId)
		{
			SetStreamId(FrameBuffer);
		}
		webrtc::VideoFrame Frame = webrtc::VideoFrame::Builder()
									   .set_video_frame_buffer(FrameBuffer)
									   .set_timestamp_us(rtc::TimeMicros())
									   .set_rotation(webrtc::VideoRotation::kVideoRotation_0)
									   .set_id(EmptyFrameId++)
									   .build();
		OnFrame(Frame);
	}

} // namespace UE::PixelStreaming
