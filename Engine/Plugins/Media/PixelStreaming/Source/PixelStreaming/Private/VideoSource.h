// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdaptedVideoTrackSource.h"
#include "PixelStreamingVideoInput.h"

namespace UE::PixelStreaming
{
	/**
	 * A source of video frames for a WebRTC peer. Has a video input which will provide
	 * frame data to this source. The source will then pass that data to an adapter
	 * which will have one or many adapt processes that are provided by the input object
	 * and are responsible for converting the frame data to the format required for
	 * the selected video encoder.
	 * This video source should be contained within a FVideoSourceGroup which is responsible
	 * for telling each source to push a frame to WebRTC at the expected rate. This source
	 * will make sure that the adapter has valid output and if so will create a frame
	 * for WebRTC. Otherwise it will continue to wait until the next frame.
	 */
	class FVideoSource : public FAdaptedVideoTrackSource
	{
	public:
		FVideoSource(bool bInRequiresUniqueId, TSharedPtr<FPixelStreamingVideoInput> InVideoInput, const TFunction<bool()>& InShouldGenerateFramesCheck);
		virtual ~FVideoSource() = default;

		void MaybePushFrame();

		/* Begin UE::PixelStreaming::AdaptedVideoTrackSource overrides */
		virtual webrtc::MediaSourceInterface::SourceState state() const override { return CurrentState; }
		virtual bool remote() const override { return false; }
		virtual bool is_screencast() const override { return false; }
		virtual absl::optional<bool> needs_denoising() const override { return false; }
		/* End UE::PixelStreaming::AdaptedVideoTrackSource overrides */

	private:
		webrtc::MediaSourceInterface::SourceState CurrentState;
		TSharedPtr<FPixelStreamingVideoInput> VideoInput;
		TFunction<bool()> ShouldGenerateFramesCheck;
		uint32 StreamId;
		bool bRequireUniqueId;

		void SetStreamId(rtc::scoped_refptr<webrtc::VideoFrameBuffer> FrameBuffer) const;
		void PushFrame();
		void PushEmptyFrame();
	};
} // namespace UE::PixelStreaming
