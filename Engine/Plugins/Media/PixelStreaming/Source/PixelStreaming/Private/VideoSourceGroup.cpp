// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoSourceGroup.h"
#include "VideoSource.h"
#include "Settings.h"
#include "PixelStreamingPrivate.h"
#include "PixelStreamingTrace.h"
#include "Stats.h"

namespace UE::PixelStreaming
{
	TSharedPtr<FVideoSourceGroup> FVideoSourceGroup::Create()
	{
		return TSharedPtr<FVideoSourceGroup>(new FVideoSourceGroup());
	}

	FVideoSourceGroup::FVideoSourceGroup()
		: bCoupleFramerate(Settings::IsCoupledFramerate() || !Settings::CVarPixelStreamingCaptureUseFence.GetValueOnAnyThread()) /* Only couple framerate if not using fence */
		, FramesPerSecond(Settings::CVarPixelStreamingWebRTCFps.GetValueOnAnyThread())
	{
	}

	FVideoSourceGroup::~FVideoSourceGroup()
	{
		Stop();
	}

	void FVideoSourceGroup::SetVideoInput(TSharedPtr<FPixelStreamingVideoInput> InVideoInput)
	{
		if (VideoInput)
		{
			VideoInput->OnFrameCaptured.Remove(FrameDelegateHandle);
		}

		VideoInput = InVideoInput;

		if (VideoInput)
		{
			FrameDelegateHandle = VideoInput->OnFrameCaptured.AddSP(AsShared(), &FVideoSourceGroup::OnFrameCaptured);
		}
	}

	void FVideoSourceGroup::SetFPS(int32 InFramesPerSecond)
	{
		FramesPerSecond = InFramesPerSecond;
	}

	int32 FVideoSourceGroup::GetFPS()
	{
		return FramesPerSecond;
	}

	void FVideoSourceGroup::SetCoupleFramerate(bool Couple)
	{
		bCoupleFramerate = Couple;
	}

	rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> FVideoSourceGroup::CreateVideoSource(bool bInRequireUniqueId, const TFunction<bool()>& InShouldGenerateFramesCheck)
	{
#if WEBRTC_5414
		rtc::scoped_refptr<FVideoSource> NewVideoSource = rtc::scoped_refptr<FVideoSource>(new FVideoSource(bInRequireUniqueId, VideoInput, InShouldGenerateFramesCheck));
#else
		rtc::scoped_refptr<FVideoSource> NewVideoSource = new FVideoSource(InStreamId, VideoInput, InShouldGenerateFramesCheck);
#endif
		{
			FScopeLock Lock(&CriticalSection);
			VideoSources.Add(NewVideoSource);
		}
		CheckStartStopThread();
		return NewVideoSource;
	}

	void FVideoSourceGroup::RemoveVideoSource(const webrtc::VideoTrackSourceInterface* ToRemove)
	{
		{
			FScopeLock Lock(&CriticalSection);
			VideoSources.RemoveAll([ToRemove](const rtc::scoped_refptr<FVideoSource>& Target) {
				return Target.get() == ToRemove;
			});
		}
		CheckStartStopThread();
	}

	void FVideoSourceGroup::RemoveAllVideoSources()
	{
		{
			FScopeLock Lock(&CriticalSection);
			VideoSources.Empty();
		}
		CheckStartStopThread();
	}

	void FVideoSourceGroup::Start()
	{
		if (!bRunning)
		{
			if (VideoSources.Num() > 0)
			{
				StartThread();
			}
			bRunning = true;
		}
	}

	void FVideoSourceGroup::Stop()
	{
		if (bRunning)
		{
			StopThread();
			bRunning = false;
		}
	}

	void FVideoSourceGroup::Tick()
	{
	    TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming Video Source Group Tick", PixelStreamingChannel);
		FScopeLock Lock(&CriticalSection);
		// for each player session, push a frame
		for (auto& VideoSource : VideoSources)
		{
			if (VideoSource)
			{
				VideoSource->MaybePushFrame();
			}
		}
	}

	void FVideoSourceGroup::OnFrameCaptured()
	{
		if (bCoupleFramerate)
		{
			Tick();
		}
		else
		{
			// We are in decoupled render/streaming mode - we should wake our VideoSourceGroup::FFrameThread (if it is sleeping)
			if(FrameRunnable && FrameRunnable->FrameEvent.Get())
			{
				FrameRunnable->FrameEvent.Get()->Trigger();
			}
		}
	}

	void FVideoSourceGroup::StartThread()
	{
		if (!bCoupleFramerate && !bThreadRunning)
		{
			FrameRunnable = MakeUnique<FFrameThread>(AsWeak());
			FrameThread = FRunnableThread::Create(FrameRunnable.Get(), TEXT("FVideoSourceGroup Thread"), 0, TPri_TimeCritical);
			bThreadRunning = true;
		}
	}

	void FVideoSourceGroup::StopThread()
	{
		if (FrameThread != nullptr)
		{
			FrameThread->Kill(true);
		}
		FrameThread = nullptr;
		bThreadRunning = false;
	}

	void FVideoSourceGroup::CheckStartStopThread()
	{
		if (bRunning)
		{
			const int32 NumSources = VideoSources.Num();
			if (bThreadRunning && NumSources == 0)
			{
				StopThread();
			}
			else if (!bThreadRunning && NumSources > 0)
			{
				StartThread();
			}
		}
	}

	bool FVideoSourceGroup::FFrameThread::Init()
	{
		return true;
	}

	uint32 FVideoSourceGroup::FFrameThread::Run()
	{
		bIsRunning = true;

		while (bIsRunning)
		{
			if(TSharedPtr<FVideoSourceGroup> VideoSourceGroup = OuterVideoSourceGroup.Pin())
			{
				const double TimeSinceLastSubmitMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - LastSubmitCycles);

				// Decrease this value to make expected frame delivery more precise, however may result in more old frames being sent
				const double PrecisionFactor = 0.1;
				const double WaitFactor = UE::PixelStreaming::Settings::CVarPixelStreamingDecoupleWaitFactor.GetValueOnAnyThread();

				// In "auto" mode vary this value based on historical average
				const double TargetSubmitMs = 1000.0 / VideoSourceGroup->FramesPerSecond;
				const double TargetSubmitMsWithPadding = TargetSubmitMs * WaitFactor;
				const double CloseEnoughMs = TargetSubmitMs * PrecisionFactor;
				const bool bFrameOverdue = TimeSinceLastSubmitMs >= TargetSubmitMsWithPadding;

				// Check frame arrived in time
				if(!bFrameOverdue)
				{
					// Frame arrived in a timely fashion, but is it too soon to maintain our target rate? If so, sleep.
					double WaitTimeRemainingMs = TargetSubmitMsWithPadding - TimeSinceLastSubmitMs;
					if(WaitTimeRemainingMs > CloseEnoughMs)
					{
						bool bGotNewFrame = FrameEvent.Get()->Wait(WaitTimeRemainingMs);
						if(!bGotNewFrame)
						{
							UE_LOGF(LogPixelStreaming, VeryVerbose, "Old frame submitted");
						}
					}
				}

				// Push frame immediately
				PushFrame(VideoSourceGroup);

			}
		}
		return 0;
	}

	void FVideoSourceGroup::FFrameThread::Stop()
	{
		bIsRunning = false;
	}

	void FVideoSourceGroup::FFrameThread::Exit()
	{
		bIsRunning = false;
	}

	/*
	* Note this function is required as part of `FSingleThreadRunnable` and only gets called when engine is run in single-threaded mode, 
	* so the logic is much less complex as this is not a case we particularly optimize for, a simple tick on an interval will be acceptable.
	*/
	void FVideoSourceGroup::FFrameThread::Tick()
	{
		if(TSharedPtr<FVideoSourceGroup> VideoSourceGroup = OuterVideoSourceGroup.Pin())
		{
			const uint64 NowCycles = FPlatformTime::Cycles64();
			const double DeltaMs = FPlatformTime::ToMilliseconds64(NowCycles - LastSubmitCycles);
			const double TargetSubmitMs = 1000.0 / VideoSourceGroup->FramesPerSecond;
			if (DeltaMs >= TargetSubmitMs)
			{
				PushFrame(VideoSourceGroup);
			}
		}
	}

	void FVideoSourceGroup::FFrameThread::PushFrame(TSharedPtr<FVideoSourceGroup> VideoSourceGroup)
	{
		VideoSourceGroup->Tick();
		LastSubmitCycles = FPlatformTime::Cycles64();
	}
} // namespace UE::PixelStreaming
