// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "Templates/SharedPointer.h"
#include "VideoSource.h"
#include "PixelStreamingVideoInput.h"
#include "HAL/Event.h"

namespace UE::PixelStreaming
{
	class FVideoSourceGroup : public TSharedFromThis<FVideoSourceGroup>
	{
	public:
		static TSharedPtr<FVideoSourceGroup> Create();
		~FVideoSourceGroup();

		void SetVideoInput(TSharedPtr<FPixelStreamingVideoInput> InVideoInput);
		TSharedPtr<FPixelStreamingVideoInput> GetVideoInput() { return VideoInput; }
		void SetFPS(int32 InFramesPerSecond);
		int32 GetFPS();

		void SetCoupleFramerate(bool Couple);

		rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> CreateVideoSource(bool bInRequireUniqueId, const TFunction<bool()>& InShouldGenerateFramesCheck);
		void RemoveVideoSource(const webrtc::VideoTrackSourceInterface* ToRemove);
		void RemoveAllVideoSources();

		void Start();
		void Stop();
		void Tick();
		bool IsThreadRunning() const { return bRunning; }

	private:
		FVideoSourceGroup();

		void StartThread();
		void StopThread();
		void CheckStartStopThread();
		void OnFrameCaptured();

		class FFrameThread : public FRunnable, public FSingleThreadRunnable
		{
		public:
			FFrameThread(TWeakPtr<FVideoSourceGroup> InVideoSourceGroup)
				: OuterVideoSourceGroup(InVideoSourceGroup)
			{
			}
			virtual ~FFrameThread() = default;

			virtual bool Init() override;
			virtual uint32 Run() override;
			virtual void Stop() override;
			virtual void Exit() override;

			virtual FSingleThreadRunnable* GetSingleThreadInterface() override
			{
				bIsRunning = true;
				return this;
			}

			virtual void Tick() override;

			void PushFrame(TSharedPtr<FVideoSourceGroup> VideoSourceGroup);
			double CalculateSleepOffsetMs(double TargetSubmitMs, uint64 LastCaptureCycles, uint64 CyclesBetweenCaptures, bool& bResetOffset) const;

			bool bIsRunning = false;
			TWeakPtr<FVideoSourceGroup> OuterVideoSourceGroup = nullptr;
			uint64 LastSubmitCycles = 0;

			/* Use this event to signal when we should wake and also how long we should sleep for between transmitting a frame. */
			FEventRef FrameEvent;
		};

		bool bRunning = false;
		bool bThreadRunning = false;
		bool bCoupleFramerate = false;
		int32 FramesPerSecond = 30;
		TSharedPtr<FPixelStreamingVideoInput> VideoInput;
		TUniquePtr<FFrameThread> FrameRunnable;
		FRunnableThread* FrameThread = nullptr; // constant FPS tick thread
		TArray<rtc::scoped_refptr<FVideoSource>> VideoSources;

		FDelegateHandle FrameDelegateHandle;

		mutable FCriticalSection CriticalSection;
	};
} // namespace UE::PixelStreaming
