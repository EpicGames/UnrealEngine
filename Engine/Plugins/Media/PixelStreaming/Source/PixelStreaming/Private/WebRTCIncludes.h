// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelStreamingWebRTCIncludes.h"

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS

	#include "Windows/AllowWindowsPlatformTypes.h"

	// C4582: constructor is not implicitly called in "api/rtcerror.h", treated as an error by UnrealEngine
	// C6319: Use of the comma-operator in a tested expression causes the left argument to be ignored when it has no side-effects.
	// C6323: Use of arithmetic operator on Boolean type(s).
	#pragma warning(push)
	#pragma warning(disable : 6319 6323)

#endif // PLATFORM_WINDOWS

// Start WebRTC Includes
#include "PreWebRTCApi.h"

#include "api/rtp_receiver_interface.h"
#include "api/media_types.h"

#include "api/create_peerconnection_factory.h"
#include "api/task_queue/default_task_queue_factory.h"
#include "api/audio_codecs/audio_format.h"
#include "api/audio_codecs/audio_decoder_factory_template.h"
#include "api/audio_codecs/audio_encoder_factory_template.h"
#include "api/audio_codecs/opus/audio_decoder_opus.h"
#include "api/audio_codecs/opus/audio_encoder_opus.h"
#include "api/audio_codecs/builtin_audio_encoder_factory.h"
#include "api/video_codecs/video_decoder.h"
#include "api/video_codecs/sdp_video_format.h"
#include "api/video_codecs/video_encoder_software_fallback_wrapper.h"
#include "api/video/video_frame.h"
#include "api/video/video_rotation.h"
#include "api/video/video_sink_interface.h"

#include "rtc_base/thread.h"
#include "rtc_base/logging.h"
#include "rtc_base/ssl_adapter.h"
#include "rtc_base/arraysize.h"
#include "rtc_base/net_helpers.h"
#include "rtc_base/string_utils.h"
#include "rtc_base/internal/default_socket_server.h"
#include "rtc_base/physical_socket_server.h"
#include "rtc_base/experiments/rate_control_settings.h"

#include "pc/session_description.h"
#include "pc/video_track_source.h"

#include "media/engine/internal_decoder_factory.h"
#include "media/engine/internal_encoder_factory.h"
#include "media/base/adapted_video_track_source.h"
#include "media/base/media_channel.h"
#include "media/base/video_common.h"

#include "modules/video_capture/video_capture_factory.h"
#include "modules/audio_device/include/audio_device.h"
#include "modules/audio_device/include/audio_device_defines.h"
#include "modules/audio_device/audio_device_buffer.h"
#include "modules/audio_processing/include/audio_processing.h"
#include "modules/video_coding/codecs/h264/include/h264.h"
#include "modules/video_coding/utility/simulcast_rate_allocator.h"
#include "modules/video_coding/codecs/vp8/include/vp8.h"
#include "modules/video_coding/codecs/vp9/include/vp9.h"

#include "common_video/h264/h264_common.h"

#include "media/base/video_broadcaster.h"

#include "system_wrappers/include/field_trial.h"

#include "PostWebRTCApi.h"
// End WebRTC Includes

// because WebRTC uses STL
#include <string>
#include <memory>
#include <stack>

#if PLATFORM_WINDOWS
	#pragma warning(pop)

	#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_END
