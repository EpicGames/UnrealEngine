// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebRTCLogging.h"

#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "rtc_base/logging.h"
#include "PostWebRTCApi.h"
// End WebRTC Includes

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreamingWebRTC, Log, All);
DEFINE_LOG_CATEGORY(LogPixelStreamingWebRTC);

/**
	* Receives logging from WebRTC internals, and writes it to a log file
	* and VS's Output window
	*/
class FWebRtcLogsRedirector : public rtc::LogSink
{
public:
	explicit FWebRtcLogsRedirector(rtc::LoggingSeverity Verbosity)
	{
		UE_LOGF(LogPixelStreamingWebRTC, Verbose, "Starting WebRTC logging");

		rtc::LogMessage::AddLogToStream(this, Verbosity);

		// Disable WebRTC's internal calls to VS's OutputDebugString, because we are calling here,
		// so we can add timestamps. Do this after `rtc::LogMessage::AddLogToStream`
		rtc::LogMessage::LogToDebug(rtc::LS_NONE);
		rtc::LogMessage::SetLogToStderr(false);
	}

	~FWebRtcLogsRedirector() override
	{
		UE_LOGF(LogPixelStreamingWebRTC, Log, "Stopping WebRTC logging");

		rtc::LogMessage::RemoveLogToStream(this);
	}

private:
	void OnLogMessage(const std::string& message, rtc::LoggingSeverity severity,
		const char* tag) override
	{
#if !NO_LOGGING
		static const ELogVerbosity::Type RtcToUnrealLogCategoryMap[] = {
			ELogVerbosity::VeryVerbose,
			ELogVerbosity::Verbose,
			ELogVerbosity::Log,
			ELogVerbosity::Warning,
			ELogVerbosity::Error
		};

		if (LogPixelStreamingWebRTC.IsSuppressed(RtcToUnrealLogCategoryMap[severity]))
		{
			return;
		}

		size_t Size = message.size();
		if (Size != 0 && message.back() == '\n')
		{
			--Size;
		}

		// this involves two allocations and copies
		// one allocation can be amortised by turning `FString Msg` into a member variable
		// but since logging should be multithread-safe, it should be thread-local member or explicitly synchronised
		auto StrPtr = StringCast<TCHAR>(message.c_str(), Size);
		// to zero-terminate the string
		FString Msg = FString::ConstructFromPtrSize( StrPtr.Get(), StrPtr.Length() );

		switch (severity)
		{
			case rtc::LS_VERBOSE:
			{
				UE_LOGF(LogPixelStreamingWebRTC, Verbose, "%ls", *Msg);
				break;
			}
			case rtc::LS_INFO:
			{
				UE_LOGF(LogPixelStreamingWebRTC, Log, "%ls", *Msg);
				break;
			}
			case rtc::LS_WARNING:
			{
				UE_LOGF(LogPixelStreamingWebRTC, Warning, "%ls", *Msg);
				break;
			}
			case rtc::LS_ERROR:
			{
				UE_LOGF(LogPixelStreamingWebRTC, Error, "%ls", *Msg);
				break;
			}
		}
#endif
	}

	void OnLogMessage(const std::string& message) override
	{
		//unimplemented();
		UE_LOGF(LogPixelStreamingWebRTC, Verbose, "%ls", *FString(message.c_str()));
	}
};

void RedirectWebRtcLogsToUnreal(rtc::LoggingSeverity Verbosity)
{
#if !UE_BUILD_SHIPPING
	static FWebRtcLogsRedirector Redirector{ Verbosity };
#endif
}
