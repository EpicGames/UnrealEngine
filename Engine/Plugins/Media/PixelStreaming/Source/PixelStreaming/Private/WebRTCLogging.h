// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "rtc_base/logging.h"
#include "PostWebRTCApi.h"
// End WebRTC Includes

#if PLATFORM_WINDOWS
	#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

void RedirectWebRtcLogsToUnreal(rtc::LoggingSeverity Verbosity);
