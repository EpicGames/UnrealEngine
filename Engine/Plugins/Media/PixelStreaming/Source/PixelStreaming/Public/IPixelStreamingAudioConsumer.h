// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Interface for consuming audio coming in from the browser.
class PIXELSTREAMING_API IPixelStreamingAudioConsumer
{
public:
	IPixelStreamingAudioConsumer(){};
	virtual ~IPixelStreamingAudioConsumer(){};
	virtual void ConsumeRawPCM(const int16_t* AudioData, int InSampleRate, size_t NChannels, size_t NFrames) = 0;
	virtual void OnConsumerAdded() = 0;
	virtual void OnConsumerRemoved() = 0;
};
