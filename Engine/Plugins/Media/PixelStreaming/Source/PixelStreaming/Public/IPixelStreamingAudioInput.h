// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class PIXELSTREAMING_API IPixelStreamingAudioInput
{
public:
	IPixelStreamingAudioInput(){};
	virtual ~IPixelStreamingAudioInput(){};

    /* Pushes audio, which must match the underlying sample rate/num channels of the mixer. */
    virtual void PushAudio(const float* InBuffer, int32 NumSamples, int32 NumChannels, int32 SampleRate) = 0;
};
