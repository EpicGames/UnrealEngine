// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingWebRTCIncludes.h"

class IPixelStreamingAudioConsumer;

// Interface for a sink that collects audio coming in from the browser and passes into into UE's audio system.
class PIXELSTREAMING_API IPixelStreamingAudioSink : public webrtc::AudioTrackSinkInterface
{
public:
	IPixelStreamingAudioSink() {}
	virtual ~IPixelStreamingAudioSink() {}
	virtual void AddAudioConsumer(IPixelStreamingAudioConsumer* AudioConsumer) = 0;
	virtual void RemoveAudioConsumer(IPixelStreamingAudioConsumer* AudioConsumer) = 0;
	virtual bool HasAudioConsumers() = 0;
};