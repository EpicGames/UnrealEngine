// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/RefCounting.h"
#include "PixelStreamingPlayerId.h"
#include "IPixelStreamingAudioSink.h"
#include "PixelStreamingWebRTCIncludes.h"
#include "IPixelStreamingStreamer.h"
#include "PixelStreamingCodec.h"
#include "PixelStreamingInputProtocol.h"

class UPixelStreamingInput;

/**
 * The public interface of the Pixel Streaming module.
 */
class PIXELSTREAMING_API IPixelStreamingModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreamingModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreamingModule>("PixelStreaming");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("PixelStreaming");
	}

	/**
	 * Sets the encoder codec for the whole Pixel Streaming module. Should be called before any streaming starts.
	 * @param Codec A valid EPixelStreamingCodec value.
	 */
	virtual void SetCodec(EPixelStreamingCodec Codec) = 0;

	/**
	 * Gets the currently selected encoder codec for all of Pixel Streaming.
	 * @return A valid EPixelStreamingCodec value.
	 */
	virtual EPixelStreamingCodec GetCodec() const = 0;

	/*
	 * Event fired when internal streamer is initialized and the methods on this module are ready for use.
	 */
	DECLARE_EVENT_OneParam(IPixelStreamingModule, FReadyEvent, IPixelStreamingModule&);

	/**
	 * A getter for the OnReady event. Intent is for users to call IPixelStreamingModule::Get().OnReady().AddXXX.
	 * @return The bindable OnReady event.
	 */
	virtual FReadyEvent& OnReady() = 0;

	/**
	 * Is the PixelStreaming module actually ready to use? Is the streamer created.
	 * @return True if Pixel Streaming module methods are ready for use.
	 */
	virtual bool IsReady() = 0;

	/**
	 * Starts streaming on all streamers.
	 */
	virtual bool StartStreaming() = 0;

	/**
	 * Stops all streamers from streaming.
	 */
	virtual void StopStreaming() = 0;

	/**
	 * Creates a new streamer.
	 * @param StreamerId - The ID of the Streamer to be created.
	 */
	virtual TSharedPtr<IPixelStreamingStreamer> CreateStreamer(const FString& StreamerId) = 0;

	/**
	 * Returns a TArray containing the keys to the currently held streamers.
	 * @return TArray containing the keys to the currently held streamers.
	 */
	virtual TArray<FString> GetStreamerIds() = 0;

	/**
	 * Find a streamer by an ID.
	 * @return A pointer to the interface for a streamer. nullptr if the streamer isn't found
	 */
	virtual TSharedPtr<IPixelStreamingStreamer> FindStreamer(const FString& StreamerId) = 0;

	/**
	 * Remove a streamer by an ID
	 * @param StreamerId	-	The ID of the streamer to be removed.
	 * @return The removed streamer. nullptr if the streamer wasn't found.
	 */
	virtual TSharedPtr<IPixelStreamingStreamer> DeleteStreamer(const FString& StreamerId) = 0;

	/**
	 * Remove a streamer by its pointer
	 * @param ToBeDeleted The streamer to remove from the internal management.
	 */
	virtual void DeleteStreamer(TSharedPtr<IPixelStreamingStreamer> ToBeDeleted) = 0;

	/**
	 * Sets the target FPS for Externally Consumed video Tracks
	 * @param InFPS new FPS for the ExternalVideoSource to output at.
	 */
	virtual void SetExternalVideoSourceFPS(uint32 InFPS) = 0;

	/**
	 * Control coupling the external video source framerate to the video input framerate
	 * @param bShouldCoupleFPS Flag for coupling the video source and video input framerates
	 */
	virtual void SetExternalVideoSourceCoupleFramerate(bool bShouldCoupleFPS) = 0;

	/**
	 * Hook to set the input to the external video source. Will also call Start() on the external video source
	 */
	virtual void SetExternalVideoSourceInput(TSharedPtr<FPixelStreamingVideoInput> InVideoInput) = 0;

	/**
	 * Allows the creation of Video Tracks that are fed the backbuffer
	 */
	virtual rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> CreateExternalVideoSource() = 0;

	/**
	 * Used to clean up sources no longer needed
	 */
	virtual void ReleaseExternalVideoSource(const webrtc::VideoTrackSourceInterface* InVideoSource) = 0;

	/**
	 * Tell the input device about a new pixel streaming input component.
	 * @param InInputComponent - The new pixel streaming input component.
	 */
	virtual void AddInputComponent(UPixelStreamingInput* InInputComponent) = 0;

	/**
	 * Tell the input device that a pixel streaming input component is no longer relevant.
	 * @param InInputComponent - The pixel streaming input component which is no longer relevant.
	 */
	virtual void RemoveInputComponent(UPixelStreamingInput* InInputComponent) = 0;

	/**
	 * Get the input components currently attached to Pixel Streaming.
	 * @return An array of input components.
	 */
	virtual const TArray<UPixelStreamingInput*> GetInputComponents() = 0;

	/**
	 * Create a webrtc::VideoEncoderFactory pointer.
	 * @return A WebRTC video encoder factory with its encoders populated by Pixel Streaming.
	 */
	virtual TUniquePtr<webrtc::VideoEncoderFactory> CreateVideoEncoderFactory() = 0;

	/**
	 * Get the Default Streamer ID
	 * @return FString The default streamer ID
	 */
	virtual FString GetDefaultStreamerID() = 0;

	/**
	 * Get the Default Signaling URL ("ws://127.0.0.1:8888")
	 * @return FString The default signaling url ("ws://127.0.0.1:8888")
	 */
	virtual FString GetDefaultSignallingURL() = 0;

	/**
	 * @brief A method for iterating through all of the streamers on the module
	 *
	 * @param Func The lambda to execute with each streamer
	 */
	virtual void ForEachStreamer(const TFunction<void(TSharedPtr<IPixelStreamingStreamer>)>& Func) = 0;
};
