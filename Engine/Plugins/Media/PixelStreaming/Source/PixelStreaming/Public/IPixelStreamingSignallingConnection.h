// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingWebRTCIncludes.h"
#include "Dom/JsonObject.h"
#include "Engine/EngineTypes.h"
#include "PixelStreamingPlayerId.h"
#include "IPixelStreamingSignallingConnectionObserver.h"

/**
 * A specialized signalling server connection object for Pixel Streaming signalling servers.
 */
class PIXELSTREAMING_API IPixelStreamingSignallingConnection
{
public:

	virtual ~IPixelStreamingSignallingConnection() {}
	
	/**
	 * Connect to a specified signalling server at ths given URL
	 * @param URL The url of the destination signalling server.
	 */
	virtual void TryConnect(FString URL) = 0;

	/**
	 * Disconnects from the signalling server. Safe to call even when not connected.
	 */
	UE_DEPRECATED(5.4, "Disconnect has been deprecated and will be removed in future versions. Please use: Disconnect(FString Reason)")
	virtual void Disconnect() = 0;

	/**
	 * Disconnects from the signalling server with a reason message. Safe to call even when not connected.
	 */
	virtual void Disconnect(FString Reason) = 0;

	/**
	 * Checks if the connection is established.
	 * @returns True when the signalling server is connected.
	 */
	virtual bool IsConnected() const = 0;

	/**
	 * [Streamer only] Sends an offer to the specified player.
	 * @param PlayerId The Id of the destination player.
	 * @param SDP The offer session description for the player.
	 */
	virtual void SendOffer(FPixelStreamingPlayerId PlayerId, const webrtc::SessionDescriptionInterface& SDP) = 0;

	/**
	 * [Streamer only] Sends an answer to the specified player.
	 * @param PlayerId The Id of the destination player.
	 * @param SDP The answer session description for the player.
	 */
	virtual void SendAnswer(FPixelStreamingPlayerId PlayerId, const webrtc::SessionDescriptionInterface& SDP) = 0;

	/**
	 * [Streamer only] Sends ice candidate information to the specified player.
	 * @param PlayerId The Id of the destination player.
	 * @param IceCandidate The ice candidate information to send.
	 */
	virtual void SendIceCandidate(FPixelStreamingPlayerId PlayerId, const webrtc::IceCandidateInterface& IceCandidate) = 0;

	/**
	 * [Streamer only] Send a disconnect message to the specified player.
	 * @param PlayerId The Id of the destination player.
	 * @param Reason An optional reason for the disconnection.
	 */
	virtual void SendDisconnectPlayer(FPixelStreamingPlayerId PlayerId, const FString& Reason) = 0;

	/**
	 * Request a list of active streamers from the signalling server. This is a request to the signalling server so can be used by anyone.
	 */
	virtual void RequestStreamerList() = 0;
	
	/**
	 * [Stream receiver only] Send a subscription request to the signalling server.
	 * @param StreamerId The ID of the streamer to subscribe to.
	 */
	virtual void SendSubscribe(const FString& StreamerId) = 0;

	/**
	 * [Stream receiver only] Send an unsubscribe request to the signalling server.
	 */
	virtual void SendUnsubscribe() = 0;

	/**
	 * [Stream receiver only] Sends an offer to the streamer.
	 * @param SDP The local description offer.
	 */
	virtual void SendOffer(const webrtc::SessionDescriptionInterface& SDP) = 0;

	/**
	 * [Stream receiver only] Sends an answer back to the streamer after receiving an offer.
	 * @param SDP The local description answer.
	 */
	virtual void SendAnswer(const webrtc::SessionDescriptionInterface& SDP) = 0;

	/**
	 * [Stream receiver only] Sends ICE candidate information to the streamer.
	 * @param IceCandidate The ICE candidate information.
	 */
	virtual void SendIceCandidate(const webrtc::IceCandidateInterface& IceCandidate) = 0;

	/**
	 * Enables or disables the keep alive pings on this connection. Receiving connections
	 * should disable the keep alive since the reference signalling server will reject
	 * pings from players and close the connection.
	 * @param bKeepAlive True to enable keep alive pings. False to disable.
	 */
	virtual void SetKeepAlive(bool bKeepAlive) = 0;

	/**
	 * Toggles automatic reconnecting when websocket is closed or unreachable.
	 * @param bAutoReconnect True to enable auto reconnect. False to disable.
	 */
	virtual void SetAutoReconnect(bool bAutoReconnect) = 0;

	/**
	 * Send a message to the signalling server.
	 * @param Msg A string message to send - typically a JSON payload.
	 */
	virtual void SendMessage(const FString& Msg) = 0;
};
