// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingPlayerId.h"
#include "PixelStreamingWebRTCIncludes.h"
#include "PixelStreamingPlayerConfig.h"

// callback interface for `FPixelStreamingSignallingConnection`
class PIXELSTREAMING_API IPixelStreamingSignallingConnectionObserver
{
public:
	virtual void OnSignallingConnected() = 0;
	virtual void OnSignallingDisconnected(int32 StatusCode, const FString& Reason, bool bWasClean) = 0;
	virtual void OnSignallingError(const FString& ErrorMsg) = 0;
	virtual void OnSignallingConfig(const webrtc::PeerConnectionInterface::RTCConfiguration& Config) = 0;

	// Streamer-only
	virtual void OnSignallingSessionDescription(FPixelStreamingPlayerId PlayerId, webrtc::SdpType Type, const FString& Sdp) = 0;
	virtual void OnSignallingRemoteIceCandidate(FPixelStreamingPlayerId PlayerId, const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) = 0;
	virtual void OnSignallingPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer) = 0;
	virtual void OnSignallingPlayerDisconnected(FPixelStreamingPlayerId PlayerId) = 0;
	virtual void OnSignallingSFUPeerDataChannels(FPixelStreamingPlayerId SFUId, FPixelStreamingPlayerId PlayerId, int32 SendStreamId, int32 RecvStreamId) = 0;
	virtual void OnPlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate) = 0;

	// Player-only
	virtual void OnSignallingStreamerList(const TArray<FString>& StreamerList) = 0;
	virtual void OnSignallingSessionDescription(webrtc::SdpType Type, const FString& Sdp) = 0;
	virtual void OnSignallingRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) = 0;
	virtual void OnSignallingPlayerCount(uint32 Count) = 0;
	virtual void OnSignallingPeerDataChannels(int32 SendStreamId, int32 RecvStreamId) = 0;
};
