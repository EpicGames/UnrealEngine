// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingPlayerId.h"
#include "PixelStreamingVideoInput.h"
#include "CoreMinimal.h"
#include "Widgets/SViewport.h"
#include "Widgets/SWindow.h"
#include "Templates/SharedPointer.h"
#include "IPixelStreamingAudioSink.h"
#include "IPixelStreamingAudioInput.h"
#include "IPixelStreamingInputHandler.h"
#include "IPixelStreamingSignallingConnection.h"
#include "PixelStreamingInputEnums.h"

class UTexture2D;
class FPixelStreamingSignallingConnection;

class PIXELSTREAMING_API IPixelStreamingStreamer
{
public:
	virtual ~IPixelStreamingStreamer() = default;

	/**
	 * @brief Set the Stream FPS
	 * @param InFramesPerSecond The number of frames per second the streamer will stream at
	 */
	virtual void SetStreamFPS(int32 InFramesPerSecond) = 0;

	/**
	 * @brief Get the Stream FPS
	 * @return - The number of frames per second the streamer will stream at
	 */
	virtual int32 GetStreamFPS() = 0;

	/**
	 * Setting this to true will cause the streamer to ignore the FPS value and instead push out frames
	 * as they are submitted from the video input. If the encode process takes longer than the time
	 * between frames, frames will be dropped.
	 * @param bCouple true to couple the streamer framerate to that of the video input.
	 */
	virtual void SetCoupleFramerate(bool bCouple) = 0;

	/**
	 * @brief Set the Video Input object
	 * @param Input The FPixelStreamingVideoInput that this streamer will stream
	 */
	virtual void SetVideoInput(TSharedPtr<FPixelStreamingVideoInput> Input) = 0;

	/**
	 * @brief Get the Video Input object
	 * @return The FPixelStreamingVideoInput that this streamer will stream
	 */
	virtual TWeakPtr<FPixelStreamingVideoInput> GetVideoInput() = 0;

	/**
	 * @brief Set the Target Viewport for this streamer. This is used to ensure input between the browser and application scales correctly
	 * @param InTargetViewport The target viewport
	 */
	virtual void SetTargetViewport(TWeakPtr<SViewport> InTargetViewport) = 0;

	/**
	 * @brief Get the Target Viewport for this streamer
	 * @return The target viewport
	 */
	virtual TWeakPtr<SViewport> GetTargetViewport() = 0;

	/**
	 * @brief Set the target window for this streamer. This is used to ensure mouse events are directed to the correct window if multiple
	 * windows are tiled on top of one another as seen in the editor
	 * @param InTargetWindow The target window
	 */
	virtual void SetTargetWindow(TWeakPtr<SWindow> InTargetWindow) = 0;

	/**
	 * @brief Get the target window for this streamer
	 * @return The target window
	 */
	virtual TWeakPtr<SWindow> GetTargetWindow() = 0;

	/**
	 * @brief Set the target screen rectangle for this streamer. This is used to when the streamer doesn't have a singular target window / viewport
	 * and as such we just use the manual scale
	 * @param InTargetScreenRect The target screen rectangle
	 */
	virtual void SetTargetScreenRect(TWeakPtr<FIntRect> InTargetScreenRect) = 0;

	/**
	 * @brief Get the target screen rectangle for this streamer
	 * @return The target screen rectangle
	 */
	virtual TWeakPtr<FIntRect> GetTargetScreenRect() = 0;

	/**
	 * @brief Set the Signalling Server URL
	 * @param InSignallingServerURL
	 */
	virtual void SetSignallingServerURL(const FString& InSignallingServerURL) = 0;

	/**
	 * @brief Get the Signalling Server URL
	 * @return The Signalling Server URL
	 */
	virtual FString GetSignallingServerURL() = 0;

	/**
	 * @brief Get this streamer's ID
	 *
	 */
	virtual FString GetId() = 0;

	/**
	 * @return True if the streamer is connected to the signalling mechanism (e.g. websocket for signalling server).
	 */
	virtual bool IsSignallingConnected() = 0;

	/**
	 * @brief Start streaming this streamer
	 */
	virtual void StartStreaming() = 0;

	/**
	 * @brief Stop this streamer from streaming
	 */
	virtual void StopStreaming() = 0;

	/**
	 * @brief Get the current state of this streamer
	 * @return true
	 * @return false
	 */
	virtual bool IsStreaming() const = 0;

	/**
	 * @brief Event fired just before the streamer begins connecting to signalling.
	 */
	DECLARE_EVENT_OneParam(IPixelStreamingStreamer, FPreConnectionEvent, IPixelStreamingStreamer*);

	/**
	 * @brief A getter for the OnPreConnection event. Intent is for users to call IPixelStreamingModule::Get().FindStreamer(ID)->OnPreConnection().AddXXX.
	 * @return - The bindable OnPreConnection event.
	 */
	virtual FPreConnectionEvent& OnPreConnection() = 0;

	/**
	 * @brief Event fired when the streamer has connected to a signalling server and is ready for peers.
	 */
	DECLARE_EVENT_OneParam(IPixelStreamingStreamer, FStreamingStartedEvent, IPixelStreamingStreamer*);

	/**
	 * @brief A getter for the OnStreamingStarted event. Intent is for users to call IPixelStreamingModule::Get().FindStreamer(ID)->OnStreamingStarted().AddXXX.
	 * @return - The bindable OnStreamingStarted event.
	 */
	virtual FStreamingStartedEvent& OnStreamingStarted() = 0;

	/**
	 * @brief Event fired when the streamer has disconnected from a signalling server and has stopped streaming.
	 */
	DECLARE_EVENT_OneParam(IPixelStreamingStreamer, FStreamingStoppedEvent, IPixelStreamingStreamer*);

	/**
	 * Broadcast when data comes in from the data channel.
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnInputReceived, FPixelStreamingPlayerId, uint8, TArray<uint8>);
	FOnInputReceived OnInputReceived;

	/**
	 * @brief A getter for the OnStreamingStopped event. Intent is for users to call IPixelStreamingModule::Get().FindStreamer(ID)->OnStreamingStopped().AddXXX.
	 * @return - The bindable OnStreamingStopped event.
	 */
	virtual FStreamingStoppedEvent& OnStreamingStopped() = 0;

	/**
	 * @brief Force a key frame to be sent.
	 */
	virtual void ForceKeyFrame() = 0;

	/**
	 * @brief Freeze Pixel Streaming.
	 * @param Texture 		- The freeze frame to display. If null then the back buffer is captured.
	 */
	virtual void FreezeStream(UTexture2D* Texture) = 0;

	/**
	 * @brief Unfreeze Pixel Streaming.
	 */
	virtual void UnfreezeStream() = 0;

	/**
	 * @brief Send all players connected to this streamer a message
	 * @param Type The message type to be sent to the player
	 * @param Descriptor The contents of the message
	 */
	virtual void SendPlayerMessage(uint8 Type, const FString& Descriptor) = 0;

	/**
	 * @brief Send a single player connected to this streamer a message
	 * @param PlayerId The ID of the player to which to send the message
	 * @param Type The message type to be sent to the player
	 * @param Descriptor The contents of the message
	 */
	virtual void SendPlayerMessage(FPixelStreamingPlayerId PlayerId, uint8 Type, const FString& Descriptor) = 0;

	/**
	 * @brief Send a file to the browser where we are sending video.
	 * @param ByteData	 	- The raw byte data of the file.
	 * @param MimeType	 	- The files Mime type. Used for reconstruction on the front end.
	 * @param FileExtension - The files extension. Used for reconstruction on the front end.
	 */
	virtual void SendFileData(const TArray64<uint8>& ByteData, FString& MimeType, FString& FileExtension) = 0;

	/**
	 * @brief Kick a player by player id.
	 * @param PlayerId		- The ID of the player to kick
	 */
	virtual void KickPlayer(FPixelStreamingPlayerId PlayerId) = 0;

	/**
	 * @brief Set the Layer Preference for a specific player. Only useful if using the SFU
	 * @param PlayerId The player id
	 * @param SpatialLayerId The id of the preferred spatial layer (resolution)
	 * @param TemporalLayerId The id of the preferred temporal layer (frame rate)
	 */
	virtual void SetPlayerLayerPreference(FPixelStreamingPlayerId PlayerId, int SpatialLayerId, int TemporalLayerId) = 0;

	/**
	 * @brief Get the connected players
	 *
	 * @return TArray<FPixelStreamingPlayerId> The connected players
	 */
	virtual TArray<FPixelStreamingPlayerId> GetConnectedPlayers() = 0;

	/**
	 * @brief Set the streamer's input handler
	 * @param InInputHandler
	 */
	virtual void SetInputHandler(TSharedPtr<IPixelStreamingInputHandler> InInputHandler) = 0;

	/**
	 * @brief Get the streamer's input handler
	 * @return The streamer's input handler
	 */
	virtual TWeakPtr<IPixelStreamingInputHandler> GetInputHandler() = 0;

	/**
	 * @brief Get the connection to the signalling server.
	 * @return The connection to the signalling server.
	 */
	virtual TWeakPtr<IPixelStreamingSignallingConnection> GetSignallingConnection() = 0;

	/**
	 * @brief Set the connection to the signalling server
	 *
	 * @param InSignallingConnection The connection to the signalling server
	 */
	virtual void SetSignallingConnection(TSharedPtr<IPixelStreamingSignallingConnection> InSignallingConnection) = 0;

	/**
	 * @brief Get the Signalling Connection Observer object
	 *
	 * @return TWeakPtr<IPixelStreamingSignallingConnectionObserver>
	 */
	virtual TWeakPtr<IPixelStreamingSignallingConnectionObserver> GetSignallingConnectionObserver() = 0;

	/**
	 * @brief Set the type for this streamers input handler. This controls whether input is routed to widgets or windows
	 */
	virtual void SetInputHandlerType(EPixelStreamingInputType InputType) = 0;

	/**
	 * @brief Get the audio sink associated with a specific peer/player.
	 */
	virtual IPixelStreamingAudioSink* GetPeerAudioSink(FPixelStreamingPlayerId PlayerId) = 0;

	/**
	 * @brief Get an audio sink that has no peers/players listening to it.
	 */
	virtual IPixelStreamingAudioSink* GetUnlistenedAudioSink() = 0;

	/**
	 * @brief Create an audio input that can be used to push audio into the Pixel Streaming mix.
	 * Note: Requires launching with -PixelStreamingExperimentalAudioInput.
	 * @return The audio input to push to.
	 */
	virtual TSharedPtr<IPixelStreamingAudioInput> CreateAudioInput() = 0;

	/**
	 * @brief Remove the audio input from the Pixel Streaming mix.
	 */
	virtual void RemoveAudioInput(TSharedPtr<IPixelStreamingAudioInput> AudioInput) = 0;

	/**
	 * @brief Allows sending arbitrary configuration options during initial connection
	 *
	 * @param Value Setting a value to an empty string clears it from the mapping and prevents it being sent
	 */
	virtual void SetConfigOption(const FName& OptionName, const FString& Value) = 0;
	virtual bool GetConfigOption(const FName& OptionName, FString& OutValue) = 0;

	/**
	 */
	virtual void PlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate) = 0;

	virtual void RefreshStreamBitrate() = 0;
};
