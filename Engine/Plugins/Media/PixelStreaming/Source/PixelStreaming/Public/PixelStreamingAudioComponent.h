// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelStreamingPlayerId.h"
#include "Components/SynthComponent.h"
#include "IPixelStreamingAudioConsumer.h"
#include "IPixelStreamingAudioSink.h"
#include "Sound/SoundGenerator.h"
#include "PixelStreamingAudioComponent.generated.h"

/*
 * An `ISoundGenerator` implementation to pump some audio from WebRTC into this synth component
 */
class PIXELSTREAMING_API FWebRTCSoundGenerator : public ISoundGenerator
{
public:
	FWebRTCSoundGenerator();

	// Called when a new buffer is required.
	virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

	// Returns the number of samples to render per callback
	virtual int32 GetDesiredNumSamplesToRenderPerCallback() const;

	// Optional. Called on audio generator thread right when the generator begins generating.
	virtual void OnBeginGenerate() { bGeneratingAudio = true; };

	// Optional. Called on audio generator thread right when the generator ends generating.
	virtual void OnEndGenerate() { bGeneratingAudio = false; };

	// Optional. Can be overridden to end the sound when generating is finished.
	virtual bool IsFinished() const { return false; };

	void AddAudio(const int16_t* AudioData, int InSampleRate, size_t NChannels, size_t NFrames);

	int32 GetSampleRate() { return Params.SampleRate; }
	virtual int32 GetNumChannels() const override { return Params.NumChannels; }
	void EmptyBuffers();
	void SetParameters(const FSoundGeneratorInitParams& InitParams);

private:
	FSoundGeneratorInitParams Params;
	TArray<int16_t> Buffer;
	FCriticalSection CriticalSection;

public:
	FThreadSafeBool bGeneratingAudio = false;
	FThreadSafeBool bShouldGenerateAudio = false;
};

/**
 * Allows in-engine playback of incoming WebRTC audio from a particular Pixel Streaming player/peer using their mic in the browser.
 * Note: Each audio component associates itself with a particular Pixel Streaming player/peer (using the the Pixel Streaming player id).
 */
UCLASS(Blueprintable, ClassGroup = (PixelStreaming), meta = (BlueprintSpawnableComponent))
class PIXELSTREAMING_API UPixelStreamingAudioComponent : public USynthComponent, public IPixelStreamingAudioConsumer
{
	GENERATED_BODY()

protected:
	UPixelStreamingAudioComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin USynthComponent interface
	virtual void OnBeginGenerate() override;
	virtual void OnEndGenerate() override;
	virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;
	//~ End USynthComponent interface

	//~ Begin UObject interface
	virtual void BeginDestroy() override;
	//~ End UObject interface

	//~ Begin UActorComponent interface
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent interface

public:
	/**
	 *   The Pixel Streaming streamer of the player that we wish to listen to.
	 *   If this is left blank this component will use the default Pixel Streaming streamer
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Audio Component")
	FString StreamerToHear;

	/**
	 *   The Pixel Streaming player/peer whose audio we wish to listen to.
	 *   If this is left blank this component will listen to the first non-listened to peer that connects after this component is ready.
	 *   Note: that when the listened to peer disconnects this component is reset to blank and will once again listen to the next non-listened to peer that connects.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Audio Component")
	FString PlayerToHear;

	/**
	 *  If not already listening to a player/peer will try to attach for listening to the "PlayerToHear" each tick.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Audio Component")
	bool bAutoFindPeer;

private:
	IPixelStreamingAudioSink* AudioSink;
	TSharedPtr<FWebRTCSoundGenerator, ESPMode::ThreadSafe> SoundGenerator;

public:
	// Listen to a specific player on the default streamer. If the player is not found this component will be silent.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	bool ListenTo(FString PlayerToListenTo);

	// Listen to a specific player. If the player is not found this component will be silent.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	bool StreamerListenTo(FString StreamerId, FString PlayerToListenTo);

	// True if listening to a connected WebRTC peer through Pixel Streaming.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	bool IsListeningToPlayer();

	bool WillListenToAnyPlayer();

	// Stops listening to any connected player/peer and resets internal state so component is ready to listen again.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	void Reset();

	//~ Begin IPixelStreamingAudioConsumer interface
	void ConsumeRawPCM(const int16_t* AudioData, int InSampleRate, size_t NChannels, size_t NFrames);
	void OnConsumerAdded();
	void OnConsumerRemoved();
	//~ End IPixelStreamingAudioConsumer interface
};