// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "PixelStreamingWebRTCIncludes.h"

/**
 * Helpers for creating webrtc buffers.
 */
namespace UE::PixelStreaming
{
	template <typename T>
	inline size_t ValueSize(T&& Value)
	{
		return sizeof(Value);
	}

	template <typename T>
	inline const void* ValueLoc(T&& Value)
	{
		return &Value;
	}

	inline size_t ValueSize(FString&& Value)
	{
		return Value.Len() * sizeof(TCHAR);
	}

	inline const void* ValueLoc(FString&& Value)
	{
		return *Value;
	}

	struct BufferBuilder
	{
		rtc::CopyOnWriteBuffer Buffer;
		size_t Pos;

		BufferBuilder(size_t size)
			: Buffer(size), Pos(0) {}

		template <typename T>
		void Insert(T&& Value)
		{
			const size_t VSize = ValueSize(Forward<T>(Value));
			const void* VLoc = ValueLoc(Forward<T>(Value));
			FMemory::Memcpy(Buffer.MutableData() + Pos, VLoc, VSize);
			Pos += VSize;
		}
	};
} // namespace UE::PixelStreaming
