// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

enum class EPixelStreamingCodec
{
	Invalid,
	H264,
	H265 UE_DEPRECATED(5.4, "EPixelStreamingCodec::H265 has been deprecated and is no longer supported. If your GPU supports it, consider using using AV1 instead"),
	AV1,
	VP8,
	VP9,
};
