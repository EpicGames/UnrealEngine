// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingBufferBuilder.h"
#include "PixelStreamingWebRTCIncludes.h"

class FPixelStreamingPeerConnection;

/**
 * A specialized representation of a WebRTC data channel for Pixel Streaming.
 */
class PIXELSTREAMING_API FPixelStreamingDataChannel : public webrtc::DataChannelObserver, public TSharedFromThis<FPixelStreamingDataChannel>
{
public:
	static TSharedPtr<FPixelStreamingDataChannel> Create(rtc::scoped_refptr<webrtc::DataChannelInterface> InChannel);
	static TSharedPtr<FPixelStreamingDataChannel> Create(FPixelStreamingPeerConnection& Connection, int32 SendStreamId, int32 RecvStreamId);
	virtual ~FPixelStreamingDataChannel();

	/**
	 * Sends a series of arguments to the data channel with the given type.
	 * @param Type Should be the ID from a registered PixelStreamingProtocol message
	 * @returns True of the message was successfully sent.
	 */
	template <typename... Args>
	bool SendMessage(uint8 Type, Args... VarArgs)
	{
		if (SendChannel->state() != webrtc::DataChannelInterface::DataState::kOpen)
		{
			return false;
		}

		UE::PixelStreaming::BufferBuilder Builder(sizeof(Type) + (0 + ... + UE::PixelStreaming::ValueSize(Forward<Args>(VarArgs))));
		Builder.Insert(Type);
		(Builder.Insert(Forward<Args>(VarArgs)), ...);
		return SendChannel->Send(webrtc::DataBuffer(Builder.Buffer, true));
	}

	/**
	 * Sends a large buffer of data to the data channel.
	 * @param Type See SendMessage. The type of the message.
	 * @param DataBytes The raw byte buffer to send.
	 * @returns True when all data was successfully sent.
	 */
	bool SendArbitraryData(uint8 Type, const TArray64<uint8>& DataBytes) const;

	/**
	 * Broadcast when the data channel state changes to open
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOpen, FPixelStreamingDataChannel&);
	FOnOpen OnOpen;

	/**
	 * Broadcast when the data channel state changes to closed
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnClosed, FPixelStreamingDataChannel&);
	FOnClosed OnClosed;

	/**
	 * Broadcast when data comes in from the data channel.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMessage, uint8, const webrtc::DataBuffer&);
	FOnMessage OnMessageReceived;

protected:
	// webrtc::DataChannelObserver implementation.
	virtual void OnStateChange() override;
	virtual void OnMessage(const webrtc::DataBuffer& Buffer) override;

private:
	rtc::scoped_refptr<webrtc::DataChannelInterface> SendChannel;
	rtc::scoped_refptr<webrtc::DataChannelInterface> RecvChannel;

	FPixelStreamingDataChannel(rtc::scoped_refptr<webrtc::DataChannelInterface> InChannel);
	FPixelStreamingDataChannel(FPixelStreamingPeerConnection& Connection, int32 SendStreamId, int32 RecvStreamId);
};
