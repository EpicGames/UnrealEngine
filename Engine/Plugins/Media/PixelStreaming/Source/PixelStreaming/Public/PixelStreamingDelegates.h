// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "PixelStreamingPlayerId.h"
#include "PixelStreamingDataChannel.h"
#include "Containers/UnrealString.h"
#include "PixelStreamingDelegates.generated.h"

namespace webrtc
{
	class DataChannelInterface;
}

UCLASS()
class PIXELSTREAMING_API UPixelStreamingDelegates : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * A connection to the signalling server was made.
	 */
	// BP Delegate
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FConnectedToSignallingServer);
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FConnectedToSignallingServer OnConnectedToSignallingServer;
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE(FConnectedToSignallingServerNative);
	FConnectedToSignallingServerNative OnConnectedToSignallingServerNative;

	/**
	 * A connection to the signalling server was lost.
	 */
	// BP Delegate
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FDisconnectedFromSignallingServer);
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FDisconnectedFromSignallingServer OnDisconnectedFromSignallingServer;
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE(FDisconnectedFromSignallingServerNative);
	FDisconnectedFromSignallingServerNative OnDisconnectedFromSignallingServerNative;

	/**
	 * A new connection has been made to the session.
	 */
	// BP Delegate
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FNewConnection, FString, StreamerId, FString, PlayerId, bool, QualityController);
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FNewConnection OnNewConnection;
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FNewConnectionNative, FString, FPixelStreamingPlayerId, bool);
	FNewConnectionNative OnNewConnectionNative;

	/**
	 * A connection to a player was lost.
	 */
	// BP Delegate
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FClosedConnection, FString, StreamerId, FString, PlayerId, bool, WasQualityController);
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FClosedConnection OnClosedConnection;
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FClosedConnectionNative, FString, FPixelStreamingPlayerId, bool);
	FClosedConnectionNative OnClosedConnectionNative;

	/**
	 * All connections have closed and nobody is viewing or interacting with
	 * the app. This is an opportunity to reset the app.
	 */
	// BP Delegate
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAllConnectionsClosed, FString, StreamerId);
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FAllConnectionsClosed OnAllConnectionsClosed;
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE_OneParam(FAllConnectionsClosedNative, FString);
	FAllConnectionsClosedNative OnAllConnectionsClosedNative;

	/**
	 * The quality controlling peer has changed
	 */
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE_TwoParams(FQualityControllerChanged, FString, FPixelStreamingPlayerId);
	FQualityControllerChanged OnQualityControllerChangedNative;

	/**
	 * A new data channel has been opened
	 */
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FDataChannelOpen, FString, FPixelStreamingPlayerId, FPixelStreamingDataChannel*);
	FDataChannelOpen OnDataChannelOpenNative;

	/**
	 * An existing data channel has been closed
	 */
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE_TwoParams(FDataChannelClosed, FString, FPixelStreamingPlayerId);
	FDataChannelClosed OnDataChannelClosedNative;

	/**
	 * A pixel streaming stat has changed
	 */
	// BP Delegate
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FStatsChanged, FString, PlayerId, FName, StatName, float, StatValue);
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FStatsChanged OnStatChanged;
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FStatChangedNative, FPixelStreamingPlayerId, FName, float);
	FStatChangedNative OnStatChangedNative;

	/**
	 * The GPU ran out of available HW encoders and fell back to software encoders
	 */
	// BP Delegate
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FFallbackToSoftwareEncodering);
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FFallbackToSoftwareEncodering OnFallbackToSoftwareEncodering;
	// C++ Delegate
	DECLARE_MULTICAST_DELEGATE(FFallbackToSoftwareEncoderingNative);
	FFallbackToSoftwareEncoderingNative OnFallbackToSoftwareEncoderingNative;

	/**
	 * Create the singleton.
	 */
	static UPixelStreamingDelegates* CreateInstance();

	static UPixelStreamingDelegates* GetPixelStreamingDelegates()
	{
		return CreateInstance();
	}

	virtual ~UPixelStreamingDelegates()
	{
		Singleton = nullptr;
		bIsExiting = true;
	}

private:
	// The singleton object.
	static UPixelStreamingDelegates* Singleton;
	static bool bIsExiting;
};
