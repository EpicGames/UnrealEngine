// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/ActorComponent.h"
#include "IPixelStreamingModule.h"
#include "PixelStreamingInputComponent.generated.h"

/**
 * This component may be attached to an actor to allow UI interactions to be
 * handled as the delegate will be notified about the interaction and will be
 * supplied with a generic descriptor string containing, for example, JSON data.
 * Responses back to the source of the UI interactions may also be sent.
 */
UCLASS(Blueprintable, ClassGroup = (PixelStreaming), meta = (BlueprintSpawnableComponent))
class PIXELSTREAMING_API UPixelStreamingInput : public UActorComponent
{
	GENERATED_BODY()

public:
	UPixelStreamingInput(const FObjectInitializer& ObjectInitializer);
	void BeginPlay() override;
	void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	// The delegate which will be notified about a UI interaction.
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnInput, const FString&, Descriptor);
	UPROPERTY(BlueprintAssignable, Category = "PixelStreaming Input")
	FOnInput OnInputEvent;

	/**
	 * Send a response back to the source of the UI interactions.
	 * @param Descriptor - A generic descriptor string.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming Input")
	void SendPixelStreamingResponse(const FString& Descriptor);

	/**
	 * Helper function to extract a string field from a JSON descriptor of a
	 * UI interaction given its field name.
	 * The field name may be hierarchical, delimited by a period. For example,
	 * to access the Width value of a Resolution command above you should use
	 * "Resolution.Width" to get the width value.
	 * @param Descriptor - The UI interaction JSON descriptor.
	 * @param FieldName - The name of the field to look for in the JSON.
	 * @param StringValue - The string value associated with the field name.
	 * @param Success - True if the field exists in the JSON data.
	 */
	UFUNCTION(BlueprintPure, Category = "PixelStreaming Input")
	void GetJsonStringValue(FString Descriptor, FString FieldName, FString& StringValue, bool& Success);

	/**
	 * Helper function to add a string field to a JSON descriptor. This produces
	 * a new descriptor which may then be chained to add further string fields.
	 * @param Descriptor - The initial JSON descriptor which may be blank initially.
	 * @param FieldName - The name of the field to add to the JSON.
	 * @param StringValue - The string value associated with the field name.
	 * @param NewDescriptor - The JSON descriptor with the string field added.
	 * @param Success - True if the string field could be added successfully.
	 */
	UFUNCTION(BlueprintPure, Category = "PixelStreaming Input")
	void AddJsonStringValue(const FString& Descriptor, FString FieldName, FString StringValue, FString& NewDescriptor, bool& Success);

private:
	// For convenience we keep a reference to the Pixel Streaming plugin.
	IPixelStreamingModule* PixelStreamingModule;
};
