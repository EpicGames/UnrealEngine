// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingAudioSink.h"
#include "IPixelStreamingAudioInput.h"
#include "PixelStreamingWebRTCIncludes.h"
#include "PixelStreamingCodec.h"

class FPixelStreamingDataChannel;

namespace UE::PixelStreaming
{
	class FAudioInputMixer;
}

// TODO(Luke.Bermingham): PixelStreamingPeerConnection should have an IPixelStreamingPeerConnection and FPixelStreamingPeerConnection can go private.

// An interface that allows us to collect regular stats from anything that implements it (we use this for WebRTC stats polling internally).
class IPixelStreamingStatsSource
{
public:
	virtual ~IPixelStreamingStatsSource() = default;
	/* Implement your function to poll whatever stats you are returning/storing. */
	virtual void PollStats() const = 0;
};

/**
 * A specialized representation of a WebRTC peer connection for Pixel Streaming.
 */
class PIXELSTREAMING_API FPixelStreamingPeerConnection : public webrtc::PeerConnectionObserver
{
public:
	virtual ~FPixelStreamingPeerConnection();

	using FRTCConfig = webrtc::PeerConnectionInterface::RTCConfiguration;

	/**
	 * Creates a new peer connection using the given RTC configuration.
	 * @param Config The RTC configuration for the connection. Usually provided from the signalling server.
	 * @param IsSFU Set to true to create a peer connection to an SFU. Controls whether it recieves simulcast and some audio settings are affected.
	 * @returns A new peer connection ready to be used.
	 */
	static TUniquePtr<FPixelStreamingPeerConnection> Create(const FRTCConfig& RTCConfig, bool IsSFU = false);

	/**
	 * Shuts down the WebRTC thread and destroys the peer connection factory. No peer connections should be in use after this call.
	 */
	static void Shutdown();

	using VoidCallback = TFunction<void()>;
	using SDPCallback = TFunction<void(const webrtc::SessionDescriptionInterface*)>;
	using ErrorCallback = TFunction<void(const FString& Error)>;

	/**
	 * Flags for CreateOffer/CreateAnswer that indicate what kind of media we indend to receive. Dictates what the resulting
	 * SDP will contain. To actually receive audio or video you will need to supply sinks via SetAudioSink/SetVideoSink
	 */
	enum EReceiveMediaOption : int
	{
		// Receive no media
		Nothing = 0x00,

		// Can receive audio
		Audio = 0x01,

		// Can receive video
		Video = 0x02,

		// Can receive both audio and video
		All = Audio | Video,
	};

	/**
	 * Asynchronously creates an offer session description for this local peer connection.
	 * @param ReceiveOption Indicates what kind of media this connection intends to receive.
	 * @param SuccessCallback A callback to be called when the offer is ready.
	 * @param ErrorCallback A callback to be called on the event of an error.
	 */
	void CreateOffer(EReceiveMediaOption ReceiveOption, const SDPCallback& SuccessCallback, const ErrorCallback& ErrorCallback);

	/**
	 * Asynchronously creates an answer session description for this local peer connection.
	 * @param ReceiveOption Indicates what kind of media this connection intends to receive.
	 * @param SuccessCallback A callback to be called when the offer is ready.
	 * @param ErrorCallback A callback to be called on the event of an error.
	 */
	void CreateAnswer(EReceiveMediaOption ReceiveOption, const SDPCallback& SuccessCallback, const ErrorCallback& ErrorCallback);

	/**
	 * Asynchronously sets the remote description from a session description string offered from a signalling server.
	 * @param SDP The string representation of the remote session description.
	 * @param SuccessCallback A callback to be called when the remote description was successfully set.
	 * @param ErrorCallback A callback to be called on the event of an error.
	 */
	void ReceiveOffer(const FString& SDP, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback);

	/**
	 * Asynchronously sets the remote description from a session description string answered from a signalling server.
	 * @param SDP The string representation of the remote session description.
	 * @param SuccessCallback A callback to be called when the remote description was successfully set.
	 * @param ErrorCallback A callback to be called on the event of an error.
	 */
	void ReceiveAnswer(const FString& SDP, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback);

	/**
	 * Asynchronously sets the local description using the given session description object.
	 * @param SDP The WebRTC session description object.
	 * @param SuccessCallback A callback to be called when the local description was successfully set.
	 * @param ErrorCallback A callback to be called on the event of an error.
	 */
	void SetLocalDescription(webrtc::SessionDescriptionInterface* SDP, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback);

	/**
	 * Asynchronously sets the remote description using the given session description object.
	 * @param SDP The WebRTC session description object.
	 * @param SuccessCallback A callback to be called when the remote description was successfully set.
	 * @param ErrorCallback A callback to be called on the event of an error.
	 */
	void SetRemoteDescription(webrtc::SessionDescriptionInterface* SDP, const VoidCallback& SuccessCallback, const ErrorCallback& ErrorCallback);

	/**
	 * Gets the current local description.
	 * @returns The session description object representing the local description. Can be nullptr.
	 */
	const webrtc::SessionDescriptionInterface* GetLocalDescription() const;

	/**
	 * Gets the current remote description.
	 * @returns The session description object representing the remote description. Can be nullptr.
	 */
	const webrtc::SessionDescriptionInterface* GetRemoteDescription() const;

	/**
	 * Sets a video source that can supply this connection with frame data.
	 * @params InVideoSource The source of frame data.
	 */
	void SetVideoSource(rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> InVideoSource);

	/**
	 * Sets an audio source that can supply this connection with audio data.
	 * @params InAudioSource The source of audio data.
	 */
	void SetAudioSource(rtc::scoped_refptr<webrtc::AudioSourceInterface> InAudioSource);

	/**
	 * Gets the currently set video source on this connection.
	 * @returns The currently set video source. Can be nullptr.
	 */
	rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> GetVideoSource() const { return VideoSource; }

	/**
	 * Gets the currently set audio source on this connection.
	 * @returns The currently set audio source. Can be nullptr.
	 */
	rtc::scoped_refptr<webrtc::AudioSourceInterface> GetAudioSource() const { return AudioSource; }

	/**
	 * Sets a video sink on this connection that will receive frame data from this connection.
	 * @param InSink The sink that will receive the frame data.
	 */
	void SetVideoSink(rtc::VideoSinkInterface<webrtc::VideoFrame>* InSink);

	/**
	 * Sets an audio sink on this connection that will receive audio data from this connection.
	 * @param InSink The sink that will receive the audio data.
	 */
	void SetAudioSink(TSharedPtr<IPixelStreamingAudioSink> InSink);

	/**
	 * Gets the currently set video sink on this connection.
	 * @return The currently set video sink. Can be nullptr.
	 */
	rtc::VideoSinkInterface<webrtc::VideoFrame>* GetVideoSink() { return VideoSink; }

	/**
	 * Gets the currently set audio sink on this connection.
	 * @return The currently set audio sink. Can be nullptr.
	 */
	TSharedPtr<IPixelStreamingAudioSink> GetAudioSink() { return AudioSink; }

	/**
	 * @brief A method for iterating through all of the tranceivers on the peer connection. You could use this to check for the existence of an audio/video transceiver
	 *
	 * @param Func The lambda to execute with each transceiver
	 */
	virtual void ForEachTransceiver(const TFunction<void(rtc::scoped_refptr<webrtc::RtpTransceiverInterface>)>& Func);

	/**
	 * Adds ICE candidate data to the peer connection. Usually supplied from the signalling server.
	 * @param SDPMid
	 * @param SDPMLineIndex
	 * @param SDP
	 */
	void AddRemoteIceCandidate(const FString& SDPMid, int SDPMLineIndex, const FString& SDP);

	/**
	 * Creates a data connection associated with this peer connection.
	 * @param Id The channel id to be used. 0 - 1023
	 * @param Negotiated Whether the connection is negoriated or not. Id is ignored if Negotiated is false
	 * #returns The data channel object created.
	 */
	TSharedPtr<FPixelStreamingDataChannel> CreateDataChannel(int Id, bool Negotiated);

	/**
	 * Sets the callback to be used when the stats collector comes knocking. Decides how stats should be collected.
	 * @param InCallback The callback object.
	 */
	void SetWebRTCStatsCallback(rtc::scoped_refptr<webrtc::RTCStatsCollectorCallback> InCallback);

	void RefreshStreamBitrate();

	/**
	 * @return The negotiated video codec as parsed from the SDP offer/answer.
	 * Note: This changes as new offer/answers come in potentially.
	 */
	EPixelStreamingCodec GetNegotiatedVideoCodec() const;

	/**
	 * Called when the ICE candidate is emitted.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnIceCandidate, const webrtc::IceCandidateInterface*);
	FOnIceCandidate OnEmitIceCandidate;

	/**
	 * Called when a data channel is opened from the remote side of this connection.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDataChannel, TSharedPtr<FPixelStreamingDataChannel>);
	FOnDataChannel OnNewDataChannel;

	/**
	 * Called when the ICE state changes.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnIceStateChanged, webrtc::PeerConnectionInterface::IceConnectionState);
	FOnIceStateChanged OnIceStateChanged;

	/**
	 * Gets the global audio source.
	 * @returns The global audio source.
	 */
	static rtc::scoped_refptr<webrtc::AudioSourceInterface> GetApplicationAudioSource();

	/**
	 * Causes the video stream to generate a keyframe
	 */
	static void ForceVideoKeyframe();

	/**
	 * Post a task onto the WebRTC signalling thread.
	 */
	template <typename FunctorT>
	static void PostSignalingTask(FunctorT&& InFunc)
	{
		// Someone may accidentally call this static function without calling FPixelStreamingPeerConnection::Create first
		if (SignallingThread != nullptr)
		{
#if WEBRTC_5414
			SignallingThread->PostTask(Forward<FunctorT>(InFunc));
#else
			SignallingThread->PostTask(RTC_FROM_HERE, Forward<FunctorT>(InFunc));
#endif
		}
	}

	static TSharedPtr<IPixelStreamingAudioInput> CreateAudioInput();
	static void RemoveAudioInput(TSharedPtr<IPixelStreamingAudioInput> AudioInput);

protected:
	//
	// webrtc::PeerConnectionObserver implementation.
	//
	virtual void OnSignalingChange(webrtc::PeerConnectionInterface::SignalingState NewState) override;
	virtual void OnAddStream(rtc::scoped_refptr<webrtc::MediaStreamInterface> Stream) override;
	virtual void OnRemoveStream(rtc::scoped_refptr<webrtc::MediaStreamInterface> Stream) override;
	virtual void OnDataChannel(rtc::scoped_refptr<webrtc::DataChannelInterface> Channel) override;
	virtual void OnRenegotiationNeeded() override;
	virtual void OnIceConnectionChange(webrtc::PeerConnectionInterface::IceConnectionState NewState) override;
	virtual void OnIceGatheringChange(webrtc::PeerConnectionInterface::IceGatheringState NewState) override;
	virtual void OnIceCandidate(const webrtc::IceCandidateInterface* Candidate) override;
	virtual void OnIceCandidatesRemoved(const std::vector<cricket::Candidate>& candidates) override;
	virtual void OnIceConnectionReceivingChange(bool Receiving) override;
	virtual void OnTrack(rtc::scoped_refptr<webrtc::RtpTransceiverInterface> Transceiver) override;
	virtual void OnRemoveTrack(rtc::scoped_refptr<webrtc::RtpReceiverInterface> receiver) override;

private:
	FPixelStreamingPeerConnection();

	enum class ESDPType
	{
		Offer,
		Answer,
	};

	void CreateSDP(ESDPType SDPType, EReceiveMediaOption ReceiveOption, const SDPCallback& SuccessCallback, const ErrorCallback& ErrorCallback);

	rtc::scoped_refptr<webrtc::PeerConnectionInterface> PeerConnection;

	rtc::scoped_refptr<webrtc::VideoTrackSourceInterface> VideoSource;
	rtc::scoped_refptr<webrtc::AudioSourceInterface> AudioSource;

	rtc::VideoSinkInterface<webrtc::VideoFrame>* VideoSink = nullptr;
	TSharedPtr<IPixelStreamingAudioSink> AudioSink;

	/* The video codec that is negotiated and extracted from the SDP. */
	EPixelStreamingCodec NegotiatedVideoCodec = EPixelStreamingCodec::Invalid;

	/* Source of WebRTC stats that we poll regularly. */
	TSharedPtr<IPixelStreamingStatsSource> StatsSource;

	bool IsSFU = false;
	bool bIsDestroying = false;

	// TODO these static methods can probably be moved off into a singleton or something
	static void CreatePeerConnectionFactory();
	static TUniquePtr<rtc::Thread> SignallingThread;
	static rtc::scoped_refptr<webrtc::PeerConnectionFactoryInterface> PeerConnectionFactory;
	static rtc::scoped_refptr<webrtc::AudioSourceInterface> ApplicationAudioSource;
	static TSharedPtr<UE::PixelStreaming::FAudioInputMixer> AudioMixer;

	// we want to be able to create data channels from events emitted from the peer connection
	friend FPixelStreamingDataChannel;
};
