// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

struct PIXELSTREAMING_API FPixelStreamingPlayerConfig
{
	bool SupportsDataChannel = false;
	bool IsSFU = false;
};
