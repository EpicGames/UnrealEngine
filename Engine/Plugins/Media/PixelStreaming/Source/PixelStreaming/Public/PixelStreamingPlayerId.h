// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

using FPixelStreamingPlayerId = FString;

PIXELSTREAMING_API inline FPixelStreamingPlayerId ToPlayerId(FString PlayerIdString)
{
	return FPixelStreamingPlayerId(PlayerIdString);
}

PIXELSTREAMING_API inline FPixelStreamingPlayerId ToPlayerId(int32 PlayerIdInteger)
{
	return FString::FromInt(PlayerIdInteger);
}

PIXELSTREAMING_API inline int32 PlayerIdToInt(FPixelStreamingPlayerId PlayerId)
{
	return FCString::Atoi(*PlayerId);
}

PIXELSTREAMING_API extern const FPixelStreamingPlayerId INVALID_PLAYER_ID;
PIXELSTREAMING_API extern const FPixelStreamingPlayerId SFU_PLAYER_ID;