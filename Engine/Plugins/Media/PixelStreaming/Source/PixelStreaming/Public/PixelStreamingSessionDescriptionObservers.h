// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingWebRTCIncludes.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/Function.h"
#include "Containers/UnrealString.h"

#include <string>
#include <memory>

//////////////////////////////////////////////////////////////////////////
// FPixelStreamingSetSessionDescriptionObserver
// WebRTC requires an implementation of `webrtc::SetSessionDescriptionObserver` interface as a callback
// for setting session description, either on receiving remote `offer` (`PeerConnection::SetRemoteDescription`)
// of on sending `answer` (`PeerConnection::SetLocalDescription`)
class PIXELSTREAMING_API FPixelStreamingSetSessionDescriptionObserver : public webrtc::SetSessionDescriptionObserver
{
public:
	using FSuccessCallback = TFunction<void()>;
	using FFailureCallback = TFunction<void(const FString&)>;

	static FPixelStreamingSetSessionDescriptionObserver* Create(const FSuccessCallback& successCallback, const FFailureCallback& failureCallback)
	{
		return new rtc::RefCountedObject<FPixelStreamingSetSessionDescriptionObserver>(successCallback, failureCallback);
	}

	FPixelStreamingSetSessionDescriptionObserver(const FSuccessCallback& successCallback, const FFailureCallback& failureCallback)
		: SuccessCallback(successCallback)
		, FailureCallback(failureCallback)
	{
	}

	// we don't need to do anything on success
	void OnSuccess() override
	{
		SuccessCallback();
	}

	// errors usually mean incompatibility between our session configuration (often H.264, its profile and level) and
	// player, malformed SDP or if player doesn't support UnifiedPlan (whatever was used by proxy)
	void OnFailure(webrtc::RTCError Error) override
	{
		FailureCallback(FString(UTF8_TO_TCHAR(Error.message())));
	}

private:
	FSuccessCallback SuccessCallback;
	FFailureCallback FailureCallback;
};

class PIXELSTREAMING_API FPixelStreamingCreateSessionDescriptionObserver : public webrtc::CreateSessionDescriptionObserver
{
public:
	using FSuccessCallback = TFunction<void(webrtc::SessionDescriptionInterface*)>;
	using FFailureCallback = TFunction<void(const FString&)>;

	static FPixelStreamingCreateSessionDescriptionObserver* Create(const FSuccessCallback& successCallback, const FFailureCallback& failureCallback)
	{
		return new rtc::RefCountedObject<FPixelStreamingCreateSessionDescriptionObserver>(successCallback, failureCallback);
	}

	FPixelStreamingCreateSessionDescriptionObserver(const FSuccessCallback& successCallback, const FFailureCallback& failureCallback)
		: SuccessCallback(successCallback)
		, FailureCallback(failureCallback)
	{
	}

	void OnSuccess(webrtc::SessionDescriptionInterface* SDP) override
	{
		SuccessCallback(SDP);
	}

	void OnFailure(webrtc::RTCError Error) override
	{
		FailureCallback(ANSI_TO_TCHAR(Error.message()));
	}

private:
	FSuccessCallback SuccessCallback;
	FFailureCallback FailureCallback;
};
