// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

/**
 * A struct representing the simulcast paramaters used by PixelStreaming. The parameters
 * contain an array of layers, with each layer having a Scale, MinBitrate and MaxBitrate
 * 
 */
struct PIXELSTREAMING_API FPixelStreamingSimulcastParameters
{
	struct PIXELSTREAMING_API FPixelStreamingSimulcastLayer
	{
		float Scaling;
		int MinBitrate;
		int MaxBitrate;
	};

	TArray<FPixelStreamingSimulcastLayer> Layers;
};

/**
 * A collection of static methods used to expose settings and their values to
 * other modules dependedant on PixelStreaming
 */
class PIXELSTREAMING_API FPixelStreamingSettings
{
public:
	static FPixelStreamingSimulcastParameters GetSimulcastParameters();
	static bool GetCaptureUseFence();
	static bool GetVPXUseCompute();
};