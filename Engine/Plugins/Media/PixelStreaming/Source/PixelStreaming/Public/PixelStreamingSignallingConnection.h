// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingWebRTCIncludes.h"
#include "Dom/JsonObject.h"
#include "Engine/EngineTypes.h"
#include "PixelStreamingPlayerId.h"
#include "IPixelStreamingSignallingConnectionObserver.h"
#include "IPixelStreamingSignallingConnection.h"

class IWebSocket;

/**
 * A specialized signalling server connection object for web socket based Pixel Streaming signalling servers.
 */
class PIXELSTREAMING_API FPixelStreamingSignallingConnection : public IPixelStreamingSignallingConnection
{
public:
	FPixelStreamingSignallingConnection(TSharedPtr<IPixelStreamingSignallingConnectionObserver> InObserver, FString InStreamerId = "", TSharedPtr<IWebSocket> InWebSocket = nullptr);
	virtual ~FPixelStreamingSignallingConnection();

	/* IPixelStreamingSignallingConnection Interface */
	virtual void TryConnect(FString URL) override;
	UE_DEPRECATED(5.4, "Disconnect has been deprecated and will be removed in future versions. Please use: Disconnect(FString Reason)")
	virtual void Disconnect() override;
	virtual void Disconnect(FString Reason) override;
	virtual bool IsConnected() const override;
	virtual void SendOffer(FPixelStreamingPlayerId PlayerId, const webrtc::SessionDescriptionInterface& SDP) override;
	virtual void SendAnswer(FPixelStreamingPlayerId PlayerId, const webrtc::SessionDescriptionInterface& SDP) override;
	virtual void SendIceCandidate(FPixelStreamingPlayerId PlayerId, const webrtc::IceCandidateInterface& IceCandidate) override;
	virtual void SendDisconnectPlayer(FPixelStreamingPlayerId PlayerId, const FString& Reason) override;
	virtual void RequestStreamerList() override;
	virtual void SendSubscribe(const FString& StreamerId) override;
	virtual void SendUnsubscribe() override;
	virtual void SendOffer(const webrtc::SessionDescriptionInterface& SDP) override;
	virtual void SendAnswer(const webrtc::SessionDescriptionInterface& SDP) override;
	virtual void SendIceCandidate(const webrtc::IceCandidateInterface& IceCandidate) override;
	virtual void SetKeepAlive(bool bKeepAlive) override;
	virtual void SetAutoReconnect(bool bAutoReconnect) override;
	virtual void SendMessage(const FString& Msg) override;

private:
	void Connect(FString Url, bool bIsReconnect);
	void OnConnected();
	void OnConnectionError(const FString& Error);
	void OnClosed(int32 StatusCode, const FString& Reason, bool bWasClean);
	void OnMessage(const FString& Msg);
	void OnBinaryMessage(const uint8* Data, int32 Length, bool bIsLastFragment);

	using FJsonObjectPtr = TSharedPtr<FJsonObject>;
	void RegisterHandler(const FString& messageType, const TFunction<void(FJsonObjectPtr)>& handler);

	void OnIdRequested();
	void OnConfig(const FJsonObjectPtr& Json);
	void OnPing(const FJsonObjectPtr& Json);
	void OnPong(const FJsonObjectPtr& Json);
	void OnSessionDescription(const FJsonObjectPtr& Json);
	void OnIceCandidate(const FJsonObjectPtr& Json);
	void OnPlayerCount(const FJsonObjectPtr& Json);
	void OnPlayerConnected(const FJsonObjectPtr& Json);
	void OnPlayerDisconnected(const FJsonObjectPtr& Json);
	void OnSFUPeerDataChannels(const FJsonObjectPtr& Json);
	void OnPeerDataChannels(const FJsonObjectPtr& Json);
	void OnStreamerList(const FJsonObjectPtr& Json);
	void SetPlayerIdJson(FJsonObjectPtr& JsonObject, FPixelStreamingPlayerId PlayerId);
	bool GetPlayerIdJson(const FJsonObjectPtr& Json, FPixelStreamingPlayerId& OutPlayerId, const FString& FieldId = TEXT("playerId"));

	void StartKeepAliveTimer();
	void StopKeepAliveTimer();

	void StartReconnectTimer();
	void StopReconnectTimer();

	template <typename... T>
	void PlayerError(FPixelStreamingPlayerId PlayerId, UE::Core::TCheckedFormatString<FString::FmtCharType, T...> Msg, T... args)
	{
		const FString FormattedMsg = FString::Printf(Msg, args...);
		PlayerError(PlayerId, FormattedMsg);
	}
	void PlayerError(FPixelStreamingPlayerId PlayerId, const FString& Msg);

private:
	TSharedPtr<IPixelStreamingSignallingConnectionObserver> Observer;
	FString StreamerId;
	FString Url;

	TSharedPtr<IWebSocket> WebSocket;

	FDelegateHandle OnConnectedHandle;
	FDelegateHandle OnConnectionErrorHandle;
	FDelegateHandle OnClosedHandle;
	FDelegateHandle OnMessageHandle;
	FDelegateHandle OnBinaryMessageHandle;

	bool bAutoReconnectEnabled = true;
	bool bKeepAliveEnabled = true;
	bool bIsConnected = false;
	/** Handle for efficient management of KeepAlive timer */
	FTimerHandle TimerHandle_KeepAlive;
	FTimerHandle TimerHandle_Reconnect;
	const float KEEP_ALIVE_INTERVAL = 60.0f;

	TMap<FString, TFunction<void(FJsonObjectPtr)>> MessageHandlers;

	// we can used WeakPtrs pointing to this SharedPtr to detect when this class is destroyed
	// that allows the following Safe... delayed executions to safely check if the connection
	// no longer exists.
	TSharedPtr<int> AliveSemaphore;

	/**
	 * These templates are defined in the cpp since theyre private and we dont want to include the private
	 * include file "Utils.h" here in this public header.
	 */
	template<typename T> void SafeGameThreadExecute(T&& Func);
	template<typename T> void SafeGameThreadExecuteAndWait(uint32 Timeout, T&& Func);
};
