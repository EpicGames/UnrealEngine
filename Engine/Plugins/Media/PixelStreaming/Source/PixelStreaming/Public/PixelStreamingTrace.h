// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"
#include "Trace/Trace.h"

UE_TRACE_CHANNEL_EXTERN(PixelStreamingChannel, PIXELSTREAMING_API);
