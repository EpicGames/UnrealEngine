// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::PixelStreaming
{
	/**
	 * Extends an existing JSON string with a new name: value property.
	 * @param Descriptor The JSON string.
	 * @param FieldName The field name to add the json string.
	 * @param StringValue The value to set in the new field.
	 * @param OutNewDescriptor The new JSON string.
	 * @param OutSuccess True if we were able to extend the JSON string.
	 */
	PIXELSTREAMING_API void ExtendJsonWithField(const FString& Descriptor, FString FieldName, FString StringValue, FString& OutNewDescriptor, bool& OutSuccess);

	/**
	 * Extract a field's value from a JSON string.
	 * @param Descriptor The JSON string.
	 * @param FieldName The field name to extract a value from.
	 * @param OutStringValue The value from the extracted field.
	 * @param OutSuccess True if we were able to extract the value.
	 */
	PIXELSTREAMING_API void ExtractJsonFromDescriptor(FString Descriptor, FString FieldName, FString& OutStringValue, bool& OutSuccess);
}