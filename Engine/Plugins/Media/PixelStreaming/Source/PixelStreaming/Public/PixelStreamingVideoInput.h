// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureCapturerSource.h"
#include "PixelCaptureCapturerMultiFormat.h"
#include "Delegates/IDelegateInstance.h"
#include "PixelStreamingWebRTCIncludes.h"
#include "PixelCaptureBufferFormat.h"

// Forward declaration
namespace UE::PixelStreaming
{
	class FVideoSource;
}

/**
 * The input of the Pixel Streaming system. Frames enter the system when OnFrame is called.
 * The input buffers might need to be converted/adapted to suit the selected video encoder.
 * CreateAdaptProcess should create a PixelStreamingFrameAdapterProcess that does this work.
 */
class PIXELSTREAMING_API FPixelStreamingVideoInput : public IPixelCaptureCapturerSource
{
public:
	FPixelStreamingVideoInput();
	virtual ~FPixelStreamingVideoInput() = default;

	void AddOutputFormat(int32 Format);

	/**
	 * Feed the input with a new captured frame.
	 * @param InputFrame The raw input frame.
	 */
	virtual void OnFrame(const IPixelCaptureInputFrame& InputFrame);

	/**
	 * A human readable identifier used when displaying what the streamer is streaming in the toolbar
	 */
	virtual FString ToString();

	bool IsReady() const { return Ready; }

	/**
	 * Used internally by the encode process.
	 */
	rtc::scoped_refptr<webrtc::VideoFrameBuffer> GetFrameBuffer();

	/**
	 * Used internally by the encode process.
	 * @return A frame buffer that has a null capturer.
	 */
	rtc::scoped_refptr<webrtc::VideoFrameBuffer> GetEmptyFrameBuffer();

	/**
	 *
	 */
	TSharedPtr<IPixelCaptureOutputFrame> RequestFormat(int32 Format, TOptional<FIntPoint> Resolution = TOptional<FIntPoint>());

	/**
	 * This is broadcast each time a frame exits the adapt process. Used to synchronize framerates with input rates.
	 * This should be called once per frame taking into consideration all the target formats and layers within the frame.
	 */
	DECLARE_MULTICAST_DELEGATE(FOnFrameCaptured);
	FOnFrameCaptured OnFrameCaptured;

private:
	int32 LastFrameWidth = -1;
	int32 LastFrameHeight = -1;
	bool  Ready = false;

	TSharedPtr<FPixelCaptureCapturerMultiFormat> FrameCapturer;
	FDelegateHandle								 CaptureCompleteHandle;
	TSet<int32>									 PreInitFormats;

	void CreateFrameCapturer();
	void OnCaptureComplete();

	static uint32 GetUniqueStreamId();

	// video inputs need a unique stream id so we can properly divide hardware encoders between inputs
	uint32		  StreamId = 0;
	static uint32 NextStreamId;

	// FVideoSource needs to set unique StreamId
	friend class UE::PixelStreaming::FVideoSource;
};
