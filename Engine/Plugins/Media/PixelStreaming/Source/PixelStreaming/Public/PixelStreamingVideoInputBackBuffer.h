// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInputRHI.h"
#include "Widgets/SWindow.h"
#include "RHI.h"
#include "Slate/SlateViewportProvider.h"
#include "Delegates/IDelegateInstance.h"

/*
 * Use this if you want to send the UE backbuffer as video input.
 */
class PIXELSTREAMING_API FPixelStreamingVideoInputBackBuffer : public FPixelStreamingVideoInputRHI
{
public:
	static TSharedPtr<FPixelStreamingVideoInputBackBuffer> Create();
	virtual ~FPixelStreamingVideoInputBackBuffer();

	virtual FString ToString() override;
private:
	FPixelStreamingVideoInputBackBuffer() = default;

	void OnBackBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider);

	FDelegateHandle DelegateHandle;
};
