// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInput.h"

/*
 * A Generic video input for I420 frames
 */
class PIXELSTREAMING_API FPixelStreamingVideoInputI420 : public FPixelStreamingVideoInput
{
public:
	FPixelStreamingVideoInputI420() = default;
	virtual ~FPixelStreamingVideoInputI420() = default;

protected:
	virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, FIntPoint Resolution) override;
};
