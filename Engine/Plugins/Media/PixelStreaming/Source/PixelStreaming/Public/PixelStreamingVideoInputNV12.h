// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInput.h"

/*
 * A Generic video input for NV12 frames
 */
class PIXELSTREAMING_API FPixelStreamingVideoInputNV12 : public FPixelStreamingVideoInput
{
public:
	FPixelStreamingVideoInputNV12() = default;
	virtual ~FPixelStreamingVideoInputNV12() = default;

protected:
	virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, FIntPoint Resolution) override;
};
