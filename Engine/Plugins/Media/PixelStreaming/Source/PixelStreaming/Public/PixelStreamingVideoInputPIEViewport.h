// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInputRHI.h"

class FViewport;

/*
 * An extension of the back buffer input that can handle PIE sessions. Primarily to be used in blueprints
 */
class PIXELSTREAMING_API FPixelStreamingVideoInputPIEViewport : public FPixelStreamingVideoInputRHI
{
public:
	static TSharedPtr<FPixelStreamingVideoInputPIEViewport> Create();
	virtual ~FPixelStreamingVideoInputPIEViewport();

	virtual FString ToString() override;

private:
	FPixelStreamingVideoInputPIEViewport() = default;

	void OnViewportRendered(FViewport* InViewport);

	FDelegateHandle DelegateHandle;
};
