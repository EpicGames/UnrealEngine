// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInput.h"

/*
 * A Generic video input for RHI frames
 */
class PIXELSTREAMING_API FPixelStreamingVideoInputRHI : public FPixelStreamingVideoInput
{
public:
	FPixelStreamingVideoInputRHI() = default;
	virtual ~FPixelStreamingVideoInputRHI() = default;

protected:
	virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, FIntPoint Resolution) override;
};
