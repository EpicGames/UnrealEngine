// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInputRHI.h"
#include "Widgets/SWindow.h"
#include "RHI.h"
#include "Delegates/IDelegateInstance.h"

class UTextureRenderTarget2D;

/*
 * Use this if you want to send the contents of a render target.
 */
class PIXELSTREAMING_API FPixelStreamingVideoInputRenderTarget : public FPixelStreamingVideoInputRHI
{
public:
	static TSharedPtr<FPixelStreamingVideoInputRenderTarget> Create(UTextureRenderTarget2D* Target);
	virtual ~FPixelStreamingVideoInputRenderTarget();

	virtual FString ToString() override;
private:
	FPixelStreamingVideoInputRenderTarget(UTextureRenderTarget2D* InTarget);
	void OnEndFrameRenderThread();

	UTextureRenderTarget2D* Target = nullptr;
	FDelegateHandle DelegateHandle;
};
