// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingWebRTCIncludes.h"
#include "Engine/Texture2DDynamic.h"
#include "RenderTargetPool.h"

class PIXELSTREAMING_API FPixelStreamingVideoSink : public rtc::VideoSinkInterface<webrtc::VideoFrame>
{
public:
    FPixelStreamingVideoSink() = default;
    ~FPixelStreamingVideoSink() = default;

    // Begin VideoSinkInterface interface
    // This method handles the conversion from a WebRTC frame to a UE texture
    virtual void OnFrame(const webrtc::VideoFrame& Frame) override;
    // End VideoSinkInterface interface

    // Derivative classes implement this method to handle how the decoded textures should actually be displayed
    // eg PixelStreamingBlueprint/PixelStreamingPeerVideoSink
    virtual void OnFrame(FTextureRHIRef Frame) { unimplemented(); }

private:
    FCriticalSection RenderSyncContext;
    FPooledRenderTargetDesc RenderTargetDescriptor;
    TRefCountPtr<IPooledRenderTarget> RenderTarget;
    TArray<uint8_t> Buffer;
};