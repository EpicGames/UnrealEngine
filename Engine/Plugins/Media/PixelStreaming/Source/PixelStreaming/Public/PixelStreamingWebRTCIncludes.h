// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"

	// C4582: constructor is not implicitly called in "api/rtcerror.h", treated as an error by UnrealEngine
	// C6319: Use of the comma-operator in a tested expression causes the left argument to be ignored when it has no side-effects.
	// C6323: Use of arithmetic operator on Boolean type(s).
	#pragma warning(push)
	#pragma warning(disable : 6319 6323)
#endif // PLATFORM_WINDOWS

// Start WebRTC Includes
#include "PreWebRTCApi.h"
#include "api/rtc_error.h"

#include "api/data_channel_interface.h"
#include "api/media_stream_interface.h"
#include "api/peer_connection_interface.h"
#include "api/video/video_frame_buffer.h"
#include "api/video/i420_buffer.h"
#include "api/video_codecs/video_encoder_factory.h"
#include "modules/audio_device/include/audio_device_defines.h"
#include "rtc_base/copy_on_write_buffer.h"

#include "PostWebRTCApi.h"
// End WebRTC Includes

#if PLATFORM_WINDOWS
	#pragma warning(pop)

	#include "Windows/HideWindowsPlatformTypes.h"
#else
	#ifdef PF_MAX
		#undef PF_MAX
	#endif
#endif // PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_END
