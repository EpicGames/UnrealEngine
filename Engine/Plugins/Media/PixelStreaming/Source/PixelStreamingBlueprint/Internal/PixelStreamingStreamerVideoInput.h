// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2DDynamic.h"
#include "PixelStreamingVideoInput.h"
#include "PixelStreamingStreamerVideoInput.generated.h"

UCLASS(NotBlueprintType, NotBlueprintable, Category = "PixelStreaming", META = (DisplayName = "Video Input"))
class PIXELSTREAMINGBLUEPRINT_API UPixelStreamingStreamerVideoInput : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<FPixelStreamingVideoInput> GetVideoInput() { return VideoInput; }

protected:
	TSharedPtr<FPixelStreamingVideoInput> VideoInput;
};
