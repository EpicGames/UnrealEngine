// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingStreamerVideoInput.h"
#include "PixelStreamingStreamerVideoInputBackBuffer.generated.h"

UCLASS(NotBlueprintType, NotBlueprintable, Category = "PixelStreaming", META = (DisplayName = "Back Buffer Streamer Video Input"))
class PIXELSTREAMINGBLUEPRINT_API UPixelStreamingStreamerVideoInputBackBuffer : public UPixelStreamingStreamerVideoInput
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<FPixelStreamingVideoInput> GetVideoInput() override;
};
