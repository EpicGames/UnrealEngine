// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingStreamerVideoInput.h"
#include "PixelStreamingStreamerVideoInputMediaCapture.generated.h"

// Equivalent to UPixelStreamingStreamerVideoInputBackBuffer but uses the MediaIO Framework to capture the frame rather than Pixel Capture
UCLASS(NotBlueprintType, NotBlueprintable, Category = "PixelStreaming", META = (DisplayName = "Media Capture Streamer Video Input"))
class PIXELSTREAMINGBLUEPRINT_API UPixelStreamingStreamerVideoInputMediaCapture : public UPixelStreamingStreamerVideoInput
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<FPixelStreamingVideoInput> GetVideoInput() override;
};
