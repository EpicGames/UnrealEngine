// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingStreamerVideoInput.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PixelStreamingStreamerVideoInputRenderTarget.generated.h"

UCLASS(NotBlueprintType, NotBlueprintable, Category = "PixelStreaming", META = (DisplayName = "Render Target Streamer Video Input"))
class PIXELSTREAMINGBLUEPRINT_API UPixelStreamingStreamerVideoInputRenderTarget : public UPixelStreamingStreamerVideoInput
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Video, AssetRegistrySearchable)
	TObjectPtr<UTextureRenderTarget2D> Target;

	virtual TSharedPtr<FPixelStreamingVideoInput> GetVideoInput() override;
};
