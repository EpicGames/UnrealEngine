// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class PixelStreamingBlueprint : ModuleRules
	{
		public PixelStreamingBlueprint(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			//PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
			//PrivatePCHHeaderFile = "Private/PCH.h";

			var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);

			PublicIncludePaths.AddRange(
				new string[] {
				});

			PrivateIncludePaths.AddRange(
				new string[] {
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"PixelStreaming"
				});
		}
	}
}
