// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingBlueprintPrivate.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogPixelStreamingBlueprint);

class FPixelStreamingBlueprintModule : public IModuleInterface
{
public:
private:
	/** IModuleInterface implementation */
	void StartupModule() override {}
	void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FPixelStreamingBlueprintModule, PixelStreamingBlueprint)
