// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerComponent.h"
#include "IPixelStreamingModule.h"
#include "PixelStreamingVideoInputBackBuffer.h"
#include "Engine/GameEngine.h"
#include "Slate/SceneViewport.h"

UPixelStreamingStreamerComponent::UPixelStreamingStreamerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([this]() {
		OnPostEngineInit();
	});
}

UPixelStreamingStreamerComponent::~UPixelStreamingStreamerComponent()
{
	if (Streamer)
	{
		IPixelStreamingModule::Get().DeleteStreamer(Streamer);
	}
}

FString UPixelStreamingStreamerComponent::GetId()
{
	if (Streamer)
	{
		return Streamer->GetId();
	}
	return "";
}

bool UPixelStreamingStreamerComponent::IsSignallingConnected()
{
	return Streamer && Streamer->IsSignallingConnected();
}

void UPixelStreamingStreamerComponent::StartStreaming()
{
	if (!Streamer)
	{
		CreateStreamer();
	}

	if (VideoInput && VideoInput->GetVideoInput())
	{
		Streamer->SetVideoInput(VideoInput->GetVideoInput());
	}
	else
	{
		Streamer->SetVideoInput(FPixelStreamingVideoInputBackBuffer::Create());
	}

	Streamer->SetCoupleFramerate(CoupleFramerate);
	Streamer->SetStreamFPS(StreamFPS);
	if (UsePixelStreamingURL)
	{
		FString ServerURL;
		FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingURL="), ServerURL);
		Streamer->SetSignallingServerURL(ServerURL);
	}
	else
	{
		Streamer->SetSignallingServerURL(SignallingServerURL);
	}
	Streamer->StartStreaming();
}

void UPixelStreamingStreamerComponent::StopStreaming()
{
	if (Streamer)
	{
		Streamer->StopStreaming();
	}
}

bool UPixelStreamingStreamerComponent::IsStreaming()
{
	return Streamer && Streamer->IsStreaming();
}

void UPixelStreamingStreamerComponent::ForceKeyFrame()
{
	if (Streamer)
	{
		Streamer->ForceKeyFrame();
	}
}

void UPixelStreamingStreamerComponent::FreezeStream(UTexture2D* Texture)
{
	if (Streamer)
	{
		Streamer->FreezeStream(Texture);
	}
}

void UPixelStreamingStreamerComponent::UnfreezeStream()
{
	if (Streamer)
	{
		Streamer->UnfreezeStream();
	}
}

void UPixelStreamingStreamerComponent::SendPlayerMessage(uint8 Type, const FString& Descriptor)
{
	if (Streamer)
	{
		Streamer->SendPlayerMessage(Type, Descriptor);
	}
}

void UPixelStreamingStreamerComponent::OnPostEngineInit()
{
	if (Streamer)
	{
		SetupStreamerInput();
	}

	bEngineStarted = true;
}

void UPixelStreamingStreamerComponent::CreateStreamer()
{
	Streamer = IPixelStreamingModule::Get().CreateStreamer(StreamerId);
	Streamer->OnStreamingStarted().AddUObject(this, &UPixelStreamingStreamerComponent::StreamingStarted);
	Streamer->OnStreamingStopped().AddUObject(this, &UPixelStreamingStreamerComponent::StreamingStopped);
	Streamer->OnInputReceived.AddUObject(this, &UPixelStreamingStreamerComponent::StreamingInput);

	if (bEngineStarted)
	{
		SetupStreamerInput();
	}
}

void UPixelStreamingStreamerComponent::StreamingStarted(IPixelStreamingStreamer*)
{
	OnStreamingStarted.Broadcast();
}

void UPixelStreamingStreamerComponent::StreamingStopped(IPixelStreamingStreamer*)
{
	OnStreamingStopped.Broadcast();
}

void UPixelStreamingStreamerComponent::StreamingInput(FPixelStreamingPlayerId PlayerId, uint8 Type, TArray<uint8> Data)
{
	OnInputReceived.Broadcast(PlayerId, Type, Data);
}

void UPixelStreamingStreamerComponent::SetupStreamerInput()
{
	if (!GIsEditor)
	{
		// default to the scene viewport if we have a game engine
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			TSharedPtr<FSceneViewport> TargetViewport = GameEngine->SceneViewport;
			if (TargetViewport.IsValid())
			{
				Streamer->SetTargetViewport(TargetViewport->GetViewportWidget());
				Streamer->SetTargetWindow(TargetViewport->FindWindow());
			}
		}
	}
}
