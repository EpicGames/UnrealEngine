// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingStreamer.h"
#include "PixelStreamingPlayerId.h"
#include "Components/ActorComponent.h"
#include "PixelStreamingStreamerVideoInput.h"
#include "PixelStreamingStreamerComponent.generated.h"

UCLASS(BlueprintType, Blueprintable, Category = "PixelStreaming", META = (DisplayName = "Streamer Component", BlueprintSpawnableComponent))
class PIXELSTREAMINGBLUEPRINT_API UPixelStreamingStreamerComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	virtual ~UPixelStreamingStreamerComponent();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	FString GetId();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	bool IsSignallingConnected();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void StartStreaming();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void StopStreaming();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	bool IsStreaming();

	DECLARE_EVENT(UPixelStreamingStreamerComponent, FStreamingStartedEvent);
	FStreamingStartedEvent OnStreamingStarted;

	DECLARE_EVENT(UPixelStreamingStreamerComponent, FStreamingStoppedEvent);
	FStreamingStoppedEvent OnStreamingStopped;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnInputReceived, FPixelStreamingPlayerId, uint8, TArray<uint8>);
	FOnInputReceived OnInputReceived;

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void ForceKeyFrame();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void FreezeStream(UTexture2D* Texture);

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void UnfreezeStream();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void SendPlayerMessage(uint8 Type, const FString& Descriptor);

	UPROPERTY(EditAnywhere, Category = "PixelStreaming")
	FString StreamerId = "Streamer Component";

	UPROPERTY(EditAnywhere, Category = "PixelStreaming")
	FString SignallingServerURL = "ws://127.0.0.1:8888";

	UPROPERTY(EditAnywhere, Category = "PixelStreaming")
	bool UsePixelStreamingURL = false;

	UPROPERTY(EditAnywhere, Category = "PixelStreaming")
	int32 StreamFPS = 60;

	UPROPERTY(EditAnywhere, Category = "PixelStreaming")
	bool CoupleFramerate = false;

	UPROPERTY(EditAnywhere, Category = "PixelStreaming")
	TObjectPtr<UPixelStreamingStreamerVideoInput> VideoInput = nullptr;

private:
	TSharedPtr<IPixelStreamingStreamer> Streamer;
	bool bEngineStarted = false;

	void OnPostEngineInit();
	void CreateStreamer();
	void SetupStreamerInput();

	void StreamingStarted(IPixelStreamingStreamer*);
	void StreamingStopped(IPixelStreamingStreamer*);
	void StreamingInput(FPixelStreamingPlayerId PlayerId, uint8 Type, TArray<uint8> Data);
};
