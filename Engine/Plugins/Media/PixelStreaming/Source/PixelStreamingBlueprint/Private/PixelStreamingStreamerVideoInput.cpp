// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerVideoInput.h"

UPixelStreamingStreamerVideoInput::UPixelStreamingStreamerVideoInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
