// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerVideoInputBackBuffer.h"
#include "PixelStreamingVideoInputPIEViewport.h"
#include "PixelStreamingVideoInputBackBuffer.h"

UPixelStreamingStreamerVideoInputBackBuffer::UPixelStreamingStreamerVideoInputBackBuffer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<FPixelStreamingVideoInput> UPixelStreamingStreamerVideoInputBackBuffer::GetVideoInput()
{
	if (!VideoInput)
	{
		// detect if we're in PIE mode or not
		bool IsGame = false;
		FParse::Bool(FCommandLine::Get(), TEXT("game"), IsGame);
		if (GIsEditor && !IsGame)
		{
			VideoInput = FPixelStreamingVideoInputPIEViewport::Create();
		}
		else
		{
			VideoInput = FPixelStreamingVideoInputBackBuffer::Create();
		}
	}
	return VideoInput;
}

