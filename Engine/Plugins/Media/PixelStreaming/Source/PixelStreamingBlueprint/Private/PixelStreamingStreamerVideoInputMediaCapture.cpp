// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerVideoInputMediaCapture.h"
#include "PixelStreamingVideoInputPIEViewport.h"
#include "PixelStreamingVideoInputMediaCapture.h"

UPixelStreamingStreamerVideoInputMediaCapture::UPixelStreamingStreamerVideoInputMediaCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<FPixelStreamingVideoInput> UPixelStreamingStreamerVideoInputMediaCapture::GetVideoInput()
{
	if (!VideoInput)
	{
		// detect if we're in PIE mode or not
		bool IsGame = false;
		FParse::Bool(FCommandLine::Get(), TEXT("game"), IsGame);
		if (GIsEditor && !IsGame)
		{
			VideoInput = FPixelStreamingVideoInputPIEViewport::Create();
		}
		else
		{
			VideoInput = FPixelStreamingVideoInputMediaCapture::CreateActiveViewportCapture();
		}
	}
	return VideoInput;
}

