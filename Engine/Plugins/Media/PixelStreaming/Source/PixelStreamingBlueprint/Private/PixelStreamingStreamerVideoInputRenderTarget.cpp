// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerVideoInputRenderTarget.h"
#include "PixelStreamingVideoInputRenderTarget.h"

UPixelStreamingStreamerVideoInputRenderTarget::UPixelStreamingStreamerVideoInputRenderTarget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<FPixelStreamingVideoInput> UPixelStreamingStreamerVideoInputRenderTarget::GetVideoInput()
{
	if (!VideoInput)
	{
		VideoInput = FPixelStreamingVideoInputRenderTarget::Create(Target);
	}
	return VideoInput;
}
