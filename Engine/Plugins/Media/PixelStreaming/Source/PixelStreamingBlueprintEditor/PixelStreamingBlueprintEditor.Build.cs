// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class PixelStreamingBlueprintEditor : ModuleRules
	{
		public PixelStreamingBlueprintEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);

			PublicIncludePaths.AddRange(
				new string[] {
				});

			PrivateIncludePaths.AddRange(
				new string[] {
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"SlateCore",
					"PixelStreaming",
					"PixelStreamingBlueprint",
					"PixelStreamingEditor",
					"UnrealEd"
				});
		}
	}
}
