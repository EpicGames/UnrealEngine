// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_StreamerVideoInput.h"

#include "PixelStreamingStreamerVideoInput.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_StreamerVideoInput"

FText UAssetDefinition_StreamerVideoInput::GetAssetDisplayName() const
{
	return LOCTEXT("StreamerVideoInput_AssetName", "Streamer Video Input Actions");
}

TSoftClassPtr<UObject> UAssetDefinition_StreamerVideoInput::GetAssetClass() const
{
	return UPixelStreamingStreamerVideoInput::StaticClass();
}

FLinearColor UAssetDefinition_StreamerVideoInput::GetAssetColor() const
{
	return FColor(192,64,64);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_StreamerVideoInput::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Misc, LOCTEXT("StreamerVideoInput_CategorySection", "Pixel Streaming"), ECategoryMenuType::Section) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
