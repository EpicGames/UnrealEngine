// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_StreamerVideoInput.generated.h"

UCLASS()
class UAssetDefinition_StreamerVideoInput : public UAssetDefinitionDefault
{
	GENERATED_BODY()
	
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual void BuildFilters(TArray<FAssetFilterData>& OutFilters) const override { }
	// UAssetDefinition End
};
