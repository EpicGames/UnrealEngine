// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "PixelStreamingStyle.h"

#define IMAGE_BRUSH_SVG(Style, RelativePath, ...) FSlateVectorImageBrush(Style.RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

class FPixelStreamingBlueprintEditorModule : public IModuleInterface
{
public:
private:
	/** IModuleInterface implementation */
	void StartupModule() override
	{
		FSlateStyleSet& StyleInstance = UE::EditorPixelStreaming::FPixelStreamingStyle::Get();

		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		StyleInstance.Set("ClassThumbnail.PixelStreamingStreamerVideoInputBackBuffer", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming_64", Icon64x64));
		StyleInstance.Set("ClassIcon.PixelStreamingStreamerVideoInputBackBuffer", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming_20", Icon20x20));
		StyleInstance.Set("ClassThumbnail.PixelStreamingStreamerVideoInputRenderTarget",new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming_64", Icon64x64));
		StyleInstance.Set("ClassIcon.PixelStreamingStreamerVideoInputRenderTarget", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming_20", Icon20x20));
		StyleInstance.Set("ClassThumbnail.PixelStreamingStreamerVideoInputMediaCapture",new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming_64", Icon64x64));
		StyleInstance.Set("ClassIcon.PixelStreamingStreamerVideoInputMediaCapture", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming_20", Icon20x20));
	}

	void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FPixelStreamingBlueprintEditorModule, PixelStreamingBlueprintEditor)
