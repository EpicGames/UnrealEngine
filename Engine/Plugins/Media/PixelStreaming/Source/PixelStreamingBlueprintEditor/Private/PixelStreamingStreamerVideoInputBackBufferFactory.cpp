// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerVideoInputBackBufferFactory.h"

#include <AssetTypeCategories.h>
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "PixelStreamingStreamerVideoInputBackBuffer.h"

#define LOCTEXT_NAMESPACE "PixelStreaming"

UPixelStreamingStreamerVideoInputBackBufferFactory::UPixelStreamingStreamerVideoInputBackBufferFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPixelStreamingStreamerVideoInputBackBuffer::StaticClass();
}

FText UPixelStreamingStreamerVideoInputBackBufferFactory::GetDisplayName() const
{
	return LOCTEXT("VideoInputBackBufferDisplayName", "Back Buffer Streamer Video Input");
}

uint32 UPixelStreamingStreamerVideoInputBackBufferFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

UObject* UPixelStreamingStreamerVideoInputBackBufferFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreamingStreamerVideoInputBackBuffer* Resource = NewObject<UPixelStreamingStreamerVideoInputBackBuffer>(InParent, InName, Flags | RF_Transactional))
	{
		return Resource;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
