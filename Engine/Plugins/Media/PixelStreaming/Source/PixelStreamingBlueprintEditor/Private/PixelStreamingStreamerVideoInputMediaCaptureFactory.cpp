// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerVideoInputMediaCaptureFactory.h"

#include <AssetTypeCategories.h>
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "PixelStreamingStreamerVideoInputMediaCapture.h"

#define LOCTEXT_NAMESPACE "PixelStreaming"

UPixelStreamingStreamerVideoInputMediaCaptureFactory::UPixelStreamingStreamerVideoInputMediaCaptureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPixelStreamingStreamerVideoInputMediaCapture::StaticClass();
}

FText UPixelStreamingStreamerVideoInputMediaCaptureFactory::GetDisplayName() const
{
	return LOCTEXT("VideoInputMediaCaptureDisplayName", "Media Capture Streamer Video Input");
}

uint32 UPixelStreamingStreamerVideoInputMediaCaptureFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

UObject* UPixelStreamingStreamerVideoInputMediaCaptureFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreamingStreamerVideoInputMediaCapture* Resource = NewObject<UPixelStreamingStreamerVideoInputMediaCapture>(InParent, InName, Flags | RF_Transactional))
	{
		return Resource;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
