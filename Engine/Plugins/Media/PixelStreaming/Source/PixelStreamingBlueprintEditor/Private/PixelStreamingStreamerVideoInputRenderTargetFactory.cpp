// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStreamerVideoInputRenderTargetFactory.h"

#include <AssetTypeCategories.h>
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "PixelStreamingStreamerVideoInputRenderTarget.h"

#define LOCTEXT_NAMESPACE "PixelStreaming"

UPixelStreamingStreamerVideoInputRenderTargetFactory::UPixelStreamingStreamerVideoInputRenderTargetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;

	SupportedClass = UPixelStreamingStreamerVideoInputRenderTarget::StaticClass();
}

FText UPixelStreamingStreamerVideoInputRenderTargetFactory::GetDisplayName() const
{
	return LOCTEXT("VideoInputRenderTargetDisplayName", "Render Target Streamer Video Input");
}

uint32 UPixelStreamingStreamerVideoInputRenderTargetFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

UObject* UPixelStreamingStreamerVideoInputRenderTargetFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreamingStreamerVideoInputRenderTarget* Resource = NewObject<UPixelStreamingStreamerVideoInputRenderTarget>(InParent, InName, Flags | RF_Transactional))
	{
		return Resource;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
