// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class PixelStreamingEditor : ModuleRules
	{
		public PixelStreamingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);

			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Projects",
				"RenderCore",
				"Renderer",
				"RHI",
				"PixelStreaming",
				"Slate",
				"SlateCore",
				"EngineSettings",
				"InputCore",
				"Json",
				"PixelCapture",
				"PixelStreamingServers",
				"HTTP",
				"Sockets",
				"ApplicationCore",
				"PixelStreamingInput",
				"AVCodecsCore"
			});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(new string[]
				{
					"UnrealEd",
					"ToolMenus",
					"EditorStyle",
					"DesktopPlatform",
					"LevelEditor",
					"MainFrame"
				});
			}

			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
			}
		}
	}
}
