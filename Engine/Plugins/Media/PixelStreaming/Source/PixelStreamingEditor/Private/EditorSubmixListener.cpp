// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSubmixListener.h"

#include "AudioDevice.h"
#include "PixelStreamingPeerConnection.h"

namespace UE::EditorPixelStreaming
{
	TSharedRef<FEditorSubmixListener, ESPMode::ThreadSafe> FEditorSubmixListener::FEditorSubmixListener::Create(const FAudioDeviceHandle& AudioDevice)
	{
		TSharedRef<FEditorSubmixListener, ESPMode::ThreadSafe> NewListener = MakeShared<FEditorSubmixListener, ESPMode::ThreadSafe>(AudioDevice);
		if (AudioDevice.IsValid())
		{
			AudioDevice.GetAudioDevice()->RegisterSubmixBufferListener(NewListener, AudioDevice->GetMainSubmixObject());
		}

		return NewListener;
	}

	FEditorSubmixListener::FEditorSubmixListener(const FAudioDeviceHandle& AudioDevice)
		: AudioDeviceId(AudioDevice.GetDeviceID())
		, AudioInput(FPixelStreamingPeerConnection::CreateAudioInput())
	{
	}

	FEditorSubmixListener::~FEditorSubmixListener()
	{
	}

	const FString& FEditorSubmixListener::GetListenerName() const
	{
		static const FString ListenerName(TEXT("PixelStreamingEditorListener"));
		return ListenerName;
	}

	void FEditorSubmixListener::Shutdown()
	{
		if (FAudioDevice* AudioDevice = FAudioDeviceManager::Get()->GetAudioDeviceRaw(AudioDeviceId))
		{
			AudioDevice->UnregisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
		}
	}

	void FEditorSubmixListener::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData,
		int32 NumSamples, int32 NumChannels,
		const int32 SampleRate, double AudioClock)
	{
		AudioInput->PushAudio(AudioData, NumSamples, NumChannels, SampleRate);
	}

} // namespace UE::EditorPixelStreaming
