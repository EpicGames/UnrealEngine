// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceHandle.h"
#include "ISubmixBufferListener.h"
#include "IPixelStreamingAudioInput.h"

namespace UE::EditorPixelStreaming
{
	class FEditorSubmixListener : public ISubmixBufferListener
	{
	public:
		static TSharedRef<FEditorSubmixListener, ESPMode::ThreadSafe> Create(const FAudioDeviceHandle& DeviceHandle);
		virtual ~FEditorSubmixListener();

		// ISubmixBufferListener interface
		void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData,
			int32 NumSamples, int32 NumChannels,
			const int32 SampleRate, double AudioClock) override;

		void Shutdown();

		virtual const FString& GetListenerName() const override;

	protected:
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;

		FEditorSubmixListener(const FAudioDeviceHandle& AudioDevice);

	private:
		Audio::FDeviceId AudioDeviceId;
		TSharedPtr<IPixelStreamingAudioInput> AudioInput;
	};
} // namespace UE::EditorPixelStreaming