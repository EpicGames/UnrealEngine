// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingCommands.h"

namespace UE::EditorPixelStreaming
{
#define LOCTEXT_NAMESPACE "PixelStreamingToolBar"
	void FPixelStreamingCommands::RegisterCommands()
	{
		UI_COMMAND(ExternalSignalling, "Use Remote Signalling Server", "Check this option if you wish to use a remote Signalling Server", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(StartSignalling, "Launch Signalling Server", "Launch a Signalling Server that will listen for connections on the ports specified above", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StopSignalling, "Stop Signalling Server", "Stop Signalling Server", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StreamLevelEditor, "Stream Level Editor", "Stream the Level Editor viewport", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StreamEditor, "Stream Full Editor", "Stream the Full Editor", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(VP8, "VP8", "VP8", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(VP9, "VP9", "VP9", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(H264, "H264", "H264", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(AV1, "AV1", "AV1 (Requires an Nvidia \"Ada Lovelace\" family GPU)", EUserInterfaceActionType::RadioButton, FInputChord());
	}
#undef LOCTEXT_NAMESPACE
} // namespace UE::EditorPixelStreaming