// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

/**
 * These are the commands used in the toolbar visible in the editor
 *
 */
namespace UE::EditorPixelStreaming
{
	class FPixelStreamingCommands : public TCommands<FPixelStreamingCommands>
	{
	public:
		FPixelStreamingCommands()
			: TCommands<FPixelStreamingCommands>(TEXT("PixelStreaming"), NSLOCTEXT("Contexts", "PixelStreaming", "PixelStreaming Plugin"), NAME_None, FName(TEXT("PixelStreamingStyle")))
		{
		}

		virtual void RegisterCommands() override;

		TSharedPtr<FUICommandInfo> ExternalSignalling;
		TSharedPtr<FUICommandInfo> VP8;
		TSharedPtr<FUICommandInfo> VP9;
		TSharedPtr<FUICommandInfo> H264;
		TSharedPtr<FUICommandInfo> AV1;
		TSharedPtr<FUICommandInfo> StartSignalling;
		TSharedPtr<FUICommandInfo> StopSignalling;
		TSharedPtr<FUICommandInfo> StreamLevelEditor;
		TSharedPtr<FUICommandInfo> StreamEditor;
	};
} // namespace UE::EditorPixelStreaming