// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingEditorModule.h"
#include "Editor/EditorPerformanceSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "PixelStreamingToolbar.h"
#include "IPixelStreamingModule.h"
#include "PixelStreamingStyle.h"
#include "IPixelStreamingStreamer.h"
#include "PixelStreamingVideoInputBackBufferComposited.h"
#include "PixelStreamingVideoInputViewport.h"
#include "Settings.h"
#include "UnrealEngine.h"
#include "Interfaces/IMainFrameModule.h"
#include "PixelStreamingInputProtocol.h"
#include "PixelStreamingUtils.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "LevelEditorViewport.h"
#include "Slate/SceneViewport.h"
#include "SLevelViewport.h"
#include "PixelStreamingInputEnums.h"
#include "IPixelStreamingInputModule.h"
#include "IPixelStreamingInputHandler.h"
#include "PixelStreamingDelegates.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "AudioDeviceManager.h"

#define LOCTEXT_NAMESPACE "PixelStreamingEditorModule"

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreamingEditor, Log, All);
DEFINE_LOG_CATEGORY(LogPixelStreamingEditor);

/**
 * IModuleInterface implementation
 */
void FPixelStreamingEditorModule::StartupModule()
{
	using namespace UE::EditorPixelStreaming;
	// Initialize the editor toolbar
	FPixelStreamingStyle::Initialize();
	FPixelStreamingStyle::ReloadTextures();
	Toolbar = MakeShared<FPixelStreamingToolbar>();

	Settings::InitialiseSettings();
	bUseExternalSignallingServer = Settings::CVarEditorPixelStreamingUseRemoteSignallingServer.GetValueOnAnyThread();

	// Enable experimental audio so we can mix the different editor audio outputs
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.ExperimentalAudioInput")); 
	CVar->Set(true);

	FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddLambda([&](Audio::FDeviceId AudioDeviceId)
	{
		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		FAudioDeviceHandle Device = DeviceManager->GetAudioDevice(AudioDeviceId);
		AudioInputs.Add(AudioDeviceId, FEditorSubmixListener::Create(Device));
	});
	FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddLambda([&](Audio::FDeviceId AudioDeviceId)
	{
		TSharedPtr<UE::EditorPixelStreaming::FEditorSubmixListener, ESPMode::ThreadSafe> Input = AudioInputs.FindRef(AudioDeviceId);
		if (Input.IsValid())
		{
			Input->Shutdown();
			AudioInputs.Remove(AudioDeviceId);
		}
	});

	IPixelStreamingModule& Module = IPixelStreamingModule::Get();
	Module.OnReady().AddRaw(this, &FPixelStreamingEditorModule::InitEditorStreaming);
}

void FPixelStreamingEditorModule::ShutdownModule()
{
	StopStreaming();
}

void FPixelStreamingEditorModule::InitEditorStreaming(IPixelStreamingModule& Module)
{
	if (!FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingID="), EditorStreamerID))
	{
		EditorStreamerID = "Editor";
	}

	EditorStreamer = Module.CreateStreamer(EditorStreamerID);

	// Bind to start/stop streaming so we disable/restore relevant editor settings
	EditorStreamer->OnStreamingStarted().AddLambda([this](IPixelStreamingStreamer* Streamer)
	{ 
		DisableCPUThrottlingSetting(); 
	});
	EditorStreamer->OnStreamingStopped().AddLambda([this](IPixelStreamingStreamer* Streamer)
	{ 
		if(!IsEngineExitRequested())
		{
			RestoreCPUThrottlingSetting(); 
		}
	});

	/**
	 * Called before the engine exits. Separate from OnPreExit as OnEnginePreExit occurs before shutting down any core modules.
	*/
	FCoreDelegates::OnEnginePreExit.AddLambda([this]() {
		RestoreCPUThrottlingSetting();
	});

	// Give the editor streamer the default url if the user hasn't specified one when launching the editor
	if (EditorStreamer->GetSignallingServerURL().IsEmpty())
	{
		// No URL was passed on the command line, initialize defaults
		StreamerPort = 8888;
		SignallingDomain = TEXT("ws://127.0.0.1");

		EditorStreamer->SetSignallingServerURL(FString::Printf(TEXT("%s:%d"), *SignallingDomain, StreamerPort));
	}
	else
	{
		FString SpecifiedSignallingURL = EditorStreamer->GetSignallingServerURL();
		TOptional<uint16> ExtractedStreamerPort = FGenericPlatformHttp::GetUrlPort(SpecifiedSignallingURL);
		StreamerPort = (int32)ExtractedStreamerPort.Get(8888);

		FString ExtractedSignallingDomain = FGenericPlatformHttp::GetUrlDomain(SpecifiedSignallingURL);
		if (FGenericPlatformHttp::IsSecureProtocol(SpecifiedSignallingURL).Get(false))
		{
			SignallingDomain = FString::Printf(TEXT("wss://%s"), *ExtractedSignallingDomain);
		}
		else
		{
			SignallingDomain = FString::Printf(TEXT("ws://%s"), *ExtractedSignallingDomain);
		}
	}

	EditorStreamer->SetConfigOption(FName(*FString(TEXT("DefaultToHover"))), TEXT("true"));

	if (GIsEditor)
	{
		IMainFrameModule::Get().OnMainFrameCreationFinished().AddLambda([&](TSharedPtr<SWindow> RootWindow, bool bIsRunningStartupDialog) {
			MaybeResizeEditor(RootWindow);
			
			if (UE::EditorPixelStreaming::Settings::CVarEditorPixelStreamingStartOnLaunch.GetValueOnAnyThread())
			{
				// Default our source to the full editor
				UE::EditorPixelStreaming::EStreamTypes Source = UE::EditorPixelStreaming::EStreamTypes::Editor;

				FString SourceStr = UE::EditorPixelStreaming::Settings::CVarEditorPixelStreamingSource.GetValueOnAnyThread();
				if(SourceStr == TEXT("Editor") || SourceStr == TEXT("editor"))
				{
					Source = UE::EditorPixelStreaming::EStreamTypes::Editor;
				}
				else if(SourceStr == TEXT("LevelEditor") || SourceStr == TEXT("leveleditor"))
				{
					Source = UE::EditorPixelStreaming::EStreamTypes::LevelEditorViewport;
				}
				
				StartStreaming(Source);
			}
		});
	}

	if (UPixelStreamingDelegates* Delegates = UPixelStreamingDelegates::GetPixelStreamingDelegates())
	{
		Delegates->OnFallbackToSoftwareEncoderingNative.AddLambda([]() {
			// Creates a new notification info, we pass in our text as the parameter.
			FNotificationInfo Info(LOCTEXT("PixelStreamingEditorModule_Notification", "Pixel Streaming: All hardware encoders in use, falling back to VP8 software encoding."));
			// Set a default expire duration
			Info.ExpireDuration = 5.0f;
			// And call Add Notification
			FSlateNotificationManager::Get().AddNotification(Info);
		});
	}
}

void FPixelStreamingEditorModule::StartStreaming(UE::EditorPixelStreaming::EStreamTypes InStreamType)
{
	// Activate our editor streamer
	if (!EditorStreamer.IsValid())
	{
		return;
	}

	// Add custom handle for { type: "Command", Resolution.Width: "1920", Resolution.Height: "1080" } when doing Editor streaming
	// because we cannot resize the game viewport, but instead want to resize the parent window.
	if (TSharedPtr<IPixelStreamingInputHandler> InputHandler = EditorStreamer->GetInputHandler().Pin())
	{
		InputHandler->SetCommandHandler("Resolution.Width",
			[](FString, FString Descriptor, FString WidthString) {
				bool bSuccess;
				FString HeightString;
				UE::PixelStreaming::ExtractJsonFromDescriptor(Descriptor, TEXT("Resolution.Height"), HeightString, bSuccess);
				int Width = FCString::Atoi(*WidthString);
				int Height = FCString::Atoi(*HeightString);
				if (Width < 1 || Height < 1)
				{
					return;
				}

				if (GIsEditor)
				{
					TSharedPtr<SWindow> ParentWindow = IMainFrameModule::Get().GetParentWindow();
					ParentWindow->Resize(FVector2D(Width, Height));
					FSlateApplication::Get().OnSizeChanged(ParentWindow->GetNativeWindow().ToSharedRef(), Width, Height);
					// Triggers the NullApplication to rebuild its DisplayMetrics with the new resolution and inform slate 
					// about the updated virtual desktop size
					FSystemResolution::RequestResolutionChange(Width, Height, GSystemResolution.WindowMode);
					IConsoleManager::Get().CallAllConsoleVariableSinks();					
				}
			});
	}

	switch (InStreamType)
	{
		case UE::EditorPixelStreaming::EStreamTypes::LevelEditorViewport:
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
			if (!ActiveLevelViewport.IsValid())
			{
				return;
			}

			FLevelEditorViewportClient& LevelViewportClient = ActiveLevelViewport->GetLevelViewportClient();
			FSceneViewport* SceneViewport = static_cast<FSceneViewport*>(LevelViewportClient.Viewport);
			EditorStreamer->SetTargetViewport(SceneViewport->GetViewportWidget());
			EditorStreamer->SetTargetWindow(SceneViewport->FindWindow());
			EditorStreamer->SetInputHandlerType(EPixelStreamingInputType::RouteToWindow);
			EditorStreamer->SetVideoInput(FPixelStreamingVideoInputViewport::Create(EditorStreamer));
		}
		break;
		case UE::EditorPixelStreaming::EStreamTypes::Editor:
		{
			EditorStreamer->SetTargetViewport(nullptr);
			EditorStreamer->SetTargetWindow(nullptr);
			EditorStreamer->SetInputHandlerType(EPixelStreamingInputType::RouteToWindow);

			TSharedPtr<FPixelStreamingVideoInputBackBufferComposited> VideoInput = FPixelStreamingVideoInputBackBufferComposited::Create();
			VideoInput->OnFrameSizeChanged.AddSP(EditorStreamer.ToSharedRef(), &IPixelStreamingStreamer::SetTargetScreenRect);
			EditorStreamer->SetVideoInput(VideoInput);
		}
		break;
		default:
		{
			UE_LOGF(LogPixelStreamingEditor, Warning, "Specified Stream Type doesn't have an associated FPixelStreamingVideoInput");
		}
			// Return here as we don't want to start streaming if we didn't set a viewport
			return;
	}

	if (!bUseExternalSignallingServer)
	{
		EditorStreamer->SetSignallingServerURL(FString::Printf(TEXT("%s:%d"), *SignallingDomain, StreamerPort));
		StartSignalling();
	}

	EditorStreamer->StartStreaming();
}

void FPixelStreamingEditorModule::StopStreaming()
{
	if (!EditorStreamer.IsValid())
	{
		return;
	}

	if (!bUseExternalSignallingServer)
	{
		StopSignalling();
	}

	EditorStreamer->SetTargetViewport(nullptr);
	EditorStreamer->SetTargetWindow(nullptr);

	EditorStreamer->StopStreaming();
}

void FPixelStreamingEditorModule::StartSignalling()
{
	bool bAlreadyLaunched = SignallingServer.IsValid() && SignallingServer->HasLaunched();
	if (bAlreadyLaunched)
	{
		return;
	}

	// Download Pixel Streaming servers/frontend if we want to use a browser to view Pixel Streaming output
	// but only attempt this is we haven't already started a download before.
	if (!DownloadProcess.IsValid())
	{
		// We set bSkipIfPresent to false, which means the get_ps_servers script will always be run, that script will choose whether to download or not
		DownloadProcess = UE::PixelStreamingServers::DownloadPixelStreamingServers(/*bSkipIfPresent*/ false);
		if (DownloadProcess.IsValid())
		{
			DownloadProcess->OnCompleted().BindLambda([this](int ExitCode) {
				StopSignalling();
				StartSignalling();
			});

			return;
		}
	}

	// Launch signalling server
	SignallingServer = UE::PixelStreamingServers::MakeSignallingServer();

	UE::PixelStreamingServers::FLaunchArgs LaunchArgs;
	LaunchArgs.bPollUntilReady = false;
	LaunchArgs.ReconnectionTimeoutSeconds = 30.0f;
	LaunchArgs.ReconnectionIntervalSeconds = 2.0f;
	LaunchArgs.ProcessArgs = FString::Printf(TEXT("--HttpPort=%d --StreamerPort=%d"), ViewerPort, StreamerPort);

	SignallingServer->Launch(LaunchArgs);
}

void FPixelStreamingEditorModule::StopSignalling()
{
	if (SignallingServer.IsValid())
	{
		SignallingServer->Stop();
		SignallingServer.Reset();
	}
}

TSharedPtr<UE::PixelStreamingServers::IServer> FPixelStreamingEditorModule::GetSignallingServer()
{
	if (SignallingServer.IsValid())
	{
		return SignallingServer;
	}
	return nullptr;
}

void FPixelStreamingEditorModule::SetSignallingDomain(const FString& InSignallingDomain)
{
	SignallingDomain = InSignallingDomain;
}

void FPixelStreamingEditorModule::SetStreamerPort(int32 InStreamerPort)
{
	StreamerPort = InStreamerPort;
}

void FPixelStreamingEditorModule::SetViewerPort(int32 InViewerPort)
{
	ViewerPort = InViewerPort;
}

bool FPixelStreamingEditorModule::UseExternalSignallingServer()
{
	return bUseExternalSignallingServer;
}
void FPixelStreamingEditorModule::UseExternalSignallingServer(bool bInUseExternalSignallingServer)
{
	bUseExternalSignallingServer = bInUseExternalSignallingServer;
}

bool FPixelStreamingEditorModule::ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY)
{
	if (*InResolution)
	{
		FString CmdString(InResolution);
		CmdString = CmdString.TrimStartAndEnd().ToLower();

		// Retrieve the X dimensional value
		const uint32 X = FMath::Max(FCString::Atof(*CmdString), 0.0f);

		// Determine whether the user has entered a resolution and extract the Y dimension.
		FString YString;

		// Find separator between values (Example of expected format: 1280x768)
		const TCHAR* YValue = NULL;
		if (FCString::Strchr(*CmdString, 'x'))
		{
			YValue = const_cast<TCHAR*>(FCString::Strchr(*CmdString, 'x') + 1);
			YString = YValue;
			// Remove any whitespace from the end of the string
			YString = YString.TrimStartAndEnd();
		}

		// If the Y dimensional value exists then setup to use the specified resolution.
		uint32 Y = 0;
		if (YValue && YString.Len() > 0)
		{
			if (YString.IsNumeric())
			{
				Y = FMath::Max(FCString::Atof(YValue), 0.0f);
				OutX = X;
				OutY = Y;
				return true;
			}
		}
	}
	return false;
}

void FPixelStreamingEditorModule::MaybeResizeEditor(TSharedPtr<SWindow> RootWindow)
{
	uint32 ResolutionX, ResolutionY = 0;
	FString ResolutionStr;
	bool bSuccess = FParse::Value(FCommandLine::Get(), TEXT("EditorPixelStreamingRes="), ResolutionStr);
	if (bSuccess)
	{
		bSuccess = ParseResolution(*ResolutionStr, ResolutionX, ResolutionY);
	}
	else
	{
		bool UserSpecifiedWidth = FParse::Value(FCommandLine::Get(), TEXT("EditorPixelStreamingResX="), ResolutionX);
		bool UserSpecifiedHeight = FParse::Value(FCommandLine::Get(), TEXT("EditorPixelStreamingResY="), ResolutionY);
		bSuccess = UserSpecifiedWidth | UserSpecifiedHeight;

		const float AspectRatio = 16.0f / 9.0f;
		if (UserSpecifiedWidth && !UserSpecifiedHeight)
		{
			ResolutionY = int32(ResolutionX / AspectRatio);
		}
		else if (UserSpecifiedHeight && !UserSpecifiedWidth)
		{
			ResolutionX = int32(ResolutionY * AspectRatio);
		}
	}

	if (bSuccess)
	{
        // Update editor window size
		RootWindow->Resize(FVector2D(ResolutionX, ResolutionY));
		FSlateApplication::Get().OnSizeChanged(RootWindow->GetNativeWindow().ToSharedRef(), ResolutionX, ResolutionY);
        // Triggers the NullApplication to rebuild its DisplayMetrics with the new resolution and inform slate 
        // about the updated virtual desktop size
        FSystemResolution::RequestResolutionChange(ResolutionX, ResolutionY, GSystemResolution.WindowMode);
	    IConsoleManager::Get().CallAllConsoleVariableSinks();
	}
}

void FPixelStreamingEditorModule::RestoreCPUThrottlingSetting()
{
	UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();
	Settings->bThrottleCPUWhenNotForeground = bOldCPUThrottlingSetting;
	Settings->PostEditChange();
}

void FPixelStreamingEditorModule::DisableCPUThrottlingSetting()
{
	// Update editor settings so that editor won't slow down if not in focus
	UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();

	// Store whatever value the user had in here so we can restore it when we are done streaming.
	bOldCPUThrottlingSetting = Settings->bThrottleCPUWhenNotForeground;

	if(Settings->bThrottleCPUWhenNotForeground)
	{
		Settings->bThrottleCPUWhenNotForeground = false;
		Settings->PostEditChange();

		// Let the user know we are forcing this editor setting (so they know why their setting is not working potentially)
		FNotificationInfo Info(LOCTEXT("PixelStreamingEditorModule_CPUThrottlingNotification", "Pixel Streaming: Disabling setting \"Use less CPU in background\" for streaming performance."));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

#undef LOCTEXT_NAMESPACE
IMPLEMENT_MODULE(FPixelStreamingEditorModule, PixelStreamingEditor)
