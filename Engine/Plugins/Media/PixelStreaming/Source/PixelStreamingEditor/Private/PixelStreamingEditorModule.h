// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingModule.h"
#include "PixelStreamingServers.h"
#include "PixelStreamingEditorUtils.h"
#include "IPixelStreamingAudioInput.h"
#include "IPixelStreamingEditorModule.h"
#include "EditorSubmixListener.h"
#include "AudioDeviceHandle.h"

namespace UE::EditorPixelStreaming
{
	class FPixelStreamingToolbar;
}

class FPixelStreamingEditorModule : public IPixelStreamingEditorModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual void StartStreaming(UE::EditorPixelStreaming::EStreamTypes InStreamType) override;
	virtual void StopStreaming() override;

	virtual void StartSignalling() override;
	virtual void StopSignalling() override;
	virtual TSharedPtr<UE::PixelStreamingServers::IServer> GetSignallingServer() override;

	virtual void SetSignallingDomain(const FString& InSignallingDomain) override;
	virtual FString GetSignallingDomain() override { return SignallingDomain; };
	virtual void SetStreamerPort(int32 InStreamerPort) override;
	virtual int32 GetStreamerPort() override { return StreamerPort; };
	virtual void SetViewerPort(int32 InViewerPort) override;
	virtual int32 GetViewerPort() override { return ViewerPort; };

	virtual bool UseExternalSignallingServer() override;
	virtual void UseExternalSignallingServer(bool bInUseExternalSignallingServer) override;
	/** End IModuleInterface implementation */

	virtual FString GetEditorStreamerID() const { return EditorStreamerID; }

private:
	void InitEditorStreaming(IPixelStreamingModule& Module);
	bool ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY);
	void MaybeResizeEditor(TSharedPtr<SWindow> RootWindow);
	void OnFrameSizeChanged(TWeakPtr<FIntRect> NewTargetRect);
	void DisableCPUThrottlingSetting();
	void RestoreCPUThrottlingSetting();

	TSharedPtr<UE::EditorPixelStreaming::FPixelStreamingToolbar> Toolbar;
	// Signalling/webserver
	TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer;
	// Download process for PS web frontend files (if we want to view output in the browser)
	TSharedPtr<FMonitoredProcess> DownloadProcess;
	// The signalling server host: eg ws://127.0.0.1
	FString SignallingDomain;
	// The ID for the Editor stream, eg "Editor"
	FString EditorStreamerID;
	// The port the streamer will connect to. eg 8888
	int32 StreamerPort;
	// The port the streams can be viewed at on the browser. eg 80 or 8080
#if PLATFORM_LINUX
	int32 ViewerPort = 8080; // ports <1000 require superuser privileges on Linux
#else
	int32 ViewerPort = 80;
#endif
	// The streamer used by the PixelStreamingEditor module
	TSharedPtr<IPixelStreamingStreamer> EditorStreamer;

	bool bUseExternalSignallingServer = false;
	bool bOldCPUThrottlingSetting = false;

	TMap<Audio::FDeviceId, TSharedPtr<UE::EditorPixelStreaming::FEditorSubmixListener, ESPMode::ThreadSafe>> AudioInputs;
};
