// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingEditorUtils.h"

#include "GenericPlatform/GenericWindowDefinition.h"
#include "Widgets/SWindow.h"

namespace UE::EditorPixelStreaming
{
	FString ToString(EStreamTypes StreamType)
	{
		switch (StreamType)
		{
			case EStreamTypes::LevelEditorViewport:
				return TEXT("the Level Editor");
			case EStreamTypes::Editor:
				return TEXT("the Full Editor");
			default:
				return TEXT("Unknown stream type!");
		}
	}

	const TCHAR* ToString(EWindowType Type)
	{
		switch (Type)
		{
			case EWindowType::Normal:
				return TEXT("Normal");

			case EWindowType::Menu:
				return TEXT("Menu");

			case EWindowType::ToolTip:
				return TEXT("ToolTip");

			case EWindowType::Notification:
				return TEXT("Notification");

			case EWindowType::CursorDecorator:
				return TEXT("CursorDecorator");

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			case EWindowType::GameWindow:
				return TEXT("GameWindow");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			default:
				return TEXT("Unknown Window Type!");
		}
	}

	const FString HashWindow(SWindow& SlateWindow, const FTextureRHIRef& FrameBuffer)
	{
		return FString::Printf(TEXT("%s-%s-%dx%d"), ToString(SlateWindow.GetType()), *SlateWindow.GetTitle().ToString(), FrameBuffer->GetSizeXY().X, FrameBuffer->GetSizeXY().Y);
	}
} // namespace UE::EditorPixelStreaming