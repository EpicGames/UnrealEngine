// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateGameResources.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

namespace UE::EditorPixelStreaming
{
    TSharedPtr<FSlateStyleSet> FPixelStreamingStyle::StyleInstance = NULL;
    void FPixelStreamingStyle::Initialize()
    {
        if(!StyleInstance.IsValid())
        {
            StyleInstance = Create();
            FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
        }
    }

    void FPixelStreamingStyle::Shutdown()
    {
        FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
    	ensure(StyleInstance.IsUnique());
	    StyleInstance.Reset();
    }

    FName FPixelStreamingStyle::GetStyleSetName()
    {
        static FName StyleSetName(TEXT("PixelStreamingStyle"));
        return StyleSetName;
    }

    const FVector2D Icon16x16(16.0f, 16.0f);
    const FVector2D Icon20x20(20.0f, 20.0f);
    const FVector2D Icon64x64(64.0f, 64.0f);
    
    TSharedRef<FSlateStyleSet> FPixelStreamingStyle::Create()
    {
        TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("PixelStreamingStyle"));
        Style->SetContentRoot(IPluginManager::Get().FindPlugin("PixelStreaming")->GetBaseDir() / TEXT("Resources"));
        Style->Set("PixelStreaming.Icon", new IMAGE_BRUSH_SVG("PixelStreaming_16", Icon16x16));
        return Style;
    }
    
    void FPixelStreamingStyle::ReloadTextures()
    {
        if(FSlateApplication::IsInitialized())
        {
            FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
        }
    }
    
    FSlateStyleSet& FPixelStreamingStyle::Get()
    {
        return *StyleInstance;
    }
}

#undef RootToContentDir