// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingToolbar.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "LevelEditor.h"
#include "ToolMenus.h"
#include "PixelStreamingCommands.h"
#include "PixelStreamingStyle.h"
#include "Framework/SlateDelegates.h"
#include "ToolMenuContext.h"
#include "IPixelStreamingModule.h"
#include "IPixelStreamingStreamer.h"
#include "Editor/EditorEngine.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SWindow.h"
#include "EditorViewportClient.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SViewport.h"
#include "Widgets/Layout/SSpacer.h"
#include "Styling/AppStyle.h"
#include "PixelStreamingCodec.h"
#include "PixelStreamingEditorModule.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Types/SlateEnums.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "PixelStreamingServers.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include <SocketSubsystem.h>
#include <IPAddress.h>
#include "PixelStreamingEditorUtils.h"
#include "SlateFwd.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "EditorPixelStreamingSettings.h"

#include "PixelStreamingCoderUtils.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"

#define LOCTEXT_NAMESPACE "PixelStreamingEditor"

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreamingToolbar, Log, All);
DEFINE_LOG_CATEGORY(LogPixelStreamingToolbar);

namespace UE::EditorPixelStreaming
{
	FPixelStreamingToolbar::FPixelStreamingToolbar()
	{
		FPixelStreamingCommands::Register();

		PluginCommands = MakeShared<FUICommandList>();

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().ExternalSignalling,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingEditorModule::Get().UseExternalSignallingServer(!IPixelStreamingEditorModule::Get().UseExternalSignallingServer());
				IPixelStreamingEditorModule::Get().StopSignalling();
			}),
			FCanExecuteAction::CreateLambda([] {
				TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = IPixelStreamingEditorModule::Get().GetSignallingServer();
				if (!SignallingServer.IsValid() || !SignallingServer->HasLaunched())
				{
					return true;
				}
				return false;
			}),
			FIsActionChecked::CreateLambda([]() {
				return IPixelStreamingEditorModule::Get().UseExternalSignallingServer();
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().StreamLevelEditor,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingEditorModule::Get().StartStreaming(EStreamTypes::LevelEditorViewport);
			}),
			FCanExecuteAction::CreateLambda([] {
				FPixelStreamingEditorModule& EditorModule = static_cast<FPixelStreamingEditorModule&>(IPixelStreamingEditorModule::Get());
				if (TSharedPtr<IPixelStreamingStreamer> Streamer = IPixelStreamingModule::Get().FindStreamer(EditorModule.GetEditorStreamerID()))
				{
					return !Streamer->IsStreaming();
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().StreamEditor,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingEditorModule::Get().StartStreaming(EStreamTypes::Editor);
			}),
			FCanExecuteAction::CreateLambda([] {
				FPixelStreamingEditorModule& EditorModule = static_cast<FPixelStreamingEditorModule&>(IPixelStreamingEditorModule::Get());
				if (TSharedPtr<IPixelStreamingStreamer> Streamer = IPixelStreamingModule::Get().FindStreamer(EditorModule.GetEditorStreamerID()))
				{
					return !Streamer->IsStreaming();
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().StartSignalling,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingEditorModule::Get().StartSignalling();
			}),
			FCanExecuteAction::CreateLambda([] {
				TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = IPixelStreamingEditorModule::Get().GetSignallingServer();
				if (!SignallingServer.IsValid() || !SignallingServer->HasLaunched())
				{
					return true;
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().StopSignalling,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingEditorModule::Get().StopSignalling();
			}),
			FCanExecuteAction::CreateLambda([] {
				TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = IPixelStreamingEditorModule::Get().GetSignallingServer();
				if (SignallingServer.IsValid() && SignallingServer->HasLaunched())
				{
					return true;
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().VP8,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingModule::Get().SetCodec(EPixelStreamingCodec::VP8);
			}),
			FCanExecuteAction::CreateLambda([] {
				bool bCanChangeCodec = true;
				IPixelStreamingModule::Get().ForEachStreamer([&bCanChangeCodec](TSharedPtr<IPixelStreamingStreamer> Streamer) {
					bCanChangeCodec &= !Streamer->IsStreaming();
				});

				return bCanChangeCodec;
			}),
			FIsActionChecked::CreateLambda([]() {
				return IPixelStreamingModule::Get().GetCodec() == EPixelStreamingCodec::VP8;
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().VP9,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingModule::Get().SetCodec(EPixelStreamingCodec::VP9);
			}),
			FCanExecuteAction::CreateLambda([] {
				bool bCanChangeCodec = true;
				IPixelStreamingModule::Get().ForEachStreamer([&bCanChangeCodec](TSharedPtr<IPixelStreamingStreamer> Streamer) {
					bCanChangeCodec &= !Streamer->IsStreaming();
				});

				return bCanChangeCodec;
			}),
			FIsActionChecked::CreateLambda([]() {
				return IPixelStreamingModule::Get().GetCodec() == EPixelStreamingCodec::VP9;
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().H264,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingModule::Get().SetCodec(EPixelStreamingCodec::H264);
			}),
			FCanExecuteAction::CreateLambda([] {
				bool bCanChangeCodec = UE::PixelStreaming::IsEncoderSupported<FVideoEncoderConfigH264>();
				IPixelStreamingModule::Get().ForEachStreamer([&bCanChangeCodec](TSharedPtr<IPixelStreamingStreamer> Streamer) {
					bCanChangeCodec &= !Streamer->IsStreaming();
				});

				return bCanChangeCodec;
			}),
			FIsActionChecked::CreateLambda([]() {
				return IPixelStreamingModule::Get().GetCodec() == EPixelStreamingCodec::H264;
			}));

		PluginCommands->MapAction(
			FPixelStreamingCommands::Get().AV1,
			FExecuteAction::CreateLambda([]() {
				IPixelStreamingModule::Get().SetCodec(EPixelStreamingCodec::AV1);
			}),
			FCanExecuteAction::CreateLambda([] {
				bool bCanChangeCodec = UE::PixelStreaming::IsEncoderSupported<FVideoEncoderConfigAV1>();
				IPixelStreamingModule::Get().ForEachStreamer([&bCanChangeCodec](TSharedPtr<IPixelStreamingStreamer> Streamer) {
					bCanChangeCodec &= !Streamer->IsStreaming();
				});

				return bCanChangeCodec;
			}),
			FIsActionChecked::CreateLambda([]() {
				return IPixelStreamingModule::Get().GetCodec() == EPixelStreamingCodec::AV1;
			}));

		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FPixelStreamingToolbar::RegisterMenus));
	}

	FPixelStreamingToolbar::~FPixelStreamingToolbar()
	{
		FPixelStreamingCommands::Unregister();
	}

	void FPixelStreamingToolbar::RegisterMenus()
	{
		if (!GetDefault<UPixelStreamingSettings>()->EnablePixelStreamingToolbar)
		{
			return;
		}

		FToolMenuOwnerScoped OwnerScoped(this);
		{
			UToolMenu* CustomToolBar = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
			{
				FToolMenuSection& Section = CustomToolBar->AddSection("PixelStreaming");
				Section.AddSeparator("PixelStreamingSeperator");
				{
					// Settings dropdown
					FToolMenuEntry SettingsEntry = FToolMenuEntry::InitComboButton(
						"PixelStreamingMenus",
						FUIAction(),
						FOnGetContent::CreateLambda(
							[&]() {
								FMenuBuilder MenuBuilder(true, PluginCommands);

								// Use external signalling server option
								MenuBuilder.BeginSection("Signalling Server Location", LOCTEXT("PixelStreamingSSLocation", "Signalling Server Location"));
								MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().ExternalSignalling);
								MenuBuilder.EndSection();

								if (!IPixelStreamingEditorModule::Get().UseExternalSignallingServer())
								{
									// Embedded Signalling Server Config (streamer port & http port)
									RegisterEmbeddedSignallingServerConfig(MenuBuilder);

									// Signalling Server Viewer URLs
									TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = IPixelStreamingEditorModule::Get().GetSignallingServer();
									if (SignallingServer.IsValid() && SignallingServer->HasLaunched())
									{
										RegisterSignallingServerURLs(MenuBuilder);
									}
								}
								else
								{
									// Remote Signalling Server Config (URL)
									RegisterRemoteSignallingServerConfig(MenuBuilder);
								}

								// Pixel Streaming streamer controls
								RegisterStreamerControls(MenuBuilder);

								// Codec Config
								RegisterCodecConfig(MenuBuilder);

								return MenuBuilder.MakeWidget();
							}),
						LOCTEXT("PixelStreamingMenu", "Pixel Streaming"),
						LOCTEXT("PixelStreamingMenuTooltip", "Configure Pixel Streaming"),
						FSlateIcon(FPixelStreamingStyle::GetStyleSetName(), "PixelStreaming.Icon"),
						false,
						"PixelStreamingMenu");
					SettingsEntry.StyleNameOverride = "CalloutToolbar";
					SettingsEntry.SetCommandList(PluginCommands);
					Section.AddEntry(SettingsEntry);
				}
			}
		}
	}

	void FPixelStreamingToolbar::RegisterEmbeddedSignallingServerConfig(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("Signalling Server Options", LOCTEXT("PixelStreamingEmbeddedSSOptions", "Embedded Signalling Server Options"));

		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = IPixelStreamingEditorModule::Get().GetSignallingServer();
		if (!SignallingServer.IsValid() || !SignallingServer->HasLaunched())
		{
			TSharedRef<SWidget> StreamerPortInputBlock = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					  .AutoWidth()
					  .VAlign(VAlign_Center)
					  .Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
						  [SNew(STextBlock)
								  .Text(FText::FromString(TEXT("Streamer Port: ")))
								  .ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))]
				+ SHorizontalBox::Slot()
					  .AutoWidth()
						  [SNew(SNumericEntryBox<int32>)
								  .MinValue(1)
								  .Value_Lambda([]() {
									  return IPixelStreamingEditorModule::Get().GetStreamerPort();
								  })
								  .OnValueChanged_Lambda([](int32 InStreamerPort) {
									  IPixelStreamingEditorModule::Get().SetStreamerPort(InStreamerPort);
								  })
								  .OnValueCommitted_Lambda([](int32 InStreamerPort, ETextCommit::Type InCommitType) {
									  IPixelStreamingEditorModule::Get().SetStreamerPort(InStreamerPort);
								  })];
			MenuBuilder.AddWidget(StreamerPortInputBlock, FText(), true);
			TSharedRef<SWidget> ViewerPortInputBlock = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					  .AutoWidth()
					  .VAlign(VAlign_Center)
					  .Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
						  [SNew(STextBlock)
								  .Text(FText::FromString(TEXT("Viewer Port: ")))
								  .ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))]
				+ SHorizontalBox::Slot()
					  .AutoWidth()
						  [SNew(SNumericEntryBox<int32>)
								  .MinValue(1)
								  .Value_Lambda([]() {
									  return IPixelStreamingEditorModule::Get().GetViewerPort();
								  })
								  .OnValueChanged_Lambda([](int32 InViewerPort) {
									  IPixelStreamingEditorModule::Get().SetViewerPort(InViewerPort);
								  })
								  .OnValueCommitted_Lambda([](int32 InViewerPort, ETextCommit::Type InCommitType) {
									  IPixelStreamingEditorModule::Get().SetViewerPort(InViewerPort);
								  })];
			MenuBuilder.AddWidget(ViewerPortInputBlock, FText(), true);
			MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().StartSignalling);
		}
		else
		{
			MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().StopSignalling);
		}

		MenuBuilder.EndSection();
	}

	void FPixelStreamingToolbar::RegisterRemoteSignallingServerConfig(FMenuBuilder& MenuBuilder)
	{
		TSharedPtr<UE::PixelStreamingServers::IServer> SignallingServer = IPixelStreamingEditorModule::Get().GetSignallingServer();
		MenuBuilder.BeginSection("Remote Signalling Server Options", LOCTEXT("PixelStreamingRemoteSSOptions", "Remote Signalling Server Options"));
		{
			TSharedRef<SWidget> URLInputBlock = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					  .AutoWidth()
					  .VAlign(VAlign_Center)
					  .Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
						  [SNew(STextBlock)
								  .Text(FText::FromString(TEXT("Remote Signalling Server URL")))
								  .ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))]
				+ SHorizontalBox::Slot()
					  .AutoWidth()
						  [SNew(SEditableTextBox)
								  .Text_Lambda([]() {
									  FPixelStreamingEditorModule& EditorModule = static_cast<FPixelStreamingEditorModule&>(IPixelStreamingEditorModule::Get());
									  TSharedPtr<IPixelStreamingStreamer> Streamer = IPixelStreamingModule::Get().FindStreamer(EditorModule.GetEditorStreamerID());
									  return FText::FromString(Streamer->GetSignallingServerURL());
								  })
								  .OnTextChanged_Lambda([](const FText& InText) {
									  IPixelStreamingModule::Get().ForEachStreamer([InText](TSharedPtr<IPixelStreamingStreamer> Streamer) {
										  Streamer->SetSignallingServerURL(InText.ToString());
									  });
								  })
								  .OnTextCommitted_Lambda([](const FText& InText, ETextCommit::Type InTextCommit) {
									  IPixelStreamingModule::Get().ForEachStreamer([InText](TSharedPtr<IPixelStreamingStreamer> Streamer) {
										  Streamer->SetSignallingServerURL(InText.ToString());
									  });
								  })
								  .IsEnabled_Lambda([]() {
									  bool bCanChangeURL = true;
									  IPixelStreamingModule::Get().ForEachStreamer([&bCanChangeURL](TSharedPtr<IPixelStreamingStreamer> Streamer) {
										  bCanChangeURL &= !Streamer->IsStreaming();
									  });
									  return bCanChangeURL;
								  })];
			MenuBuilder.AddWidget(URLInputBlock, FText(), true);
		}
		MenuBuilder.EndSection();
	}

	void FPixelStreamingToolbar::RegisterSignallingServerURLs(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("Signalling Server URLs", LOCTEXT("PixelStreamingSignallingURLs", "Signalling Server URLs"));
		{
			MenuBuilder.AddWidget(SNew(SBox)
									  .Padding(FMargin(16.0f, 3.0f))
										  [SNew(STextBlock)
												  .ColorAndOpacity(FSlateColor::UseSubduedForeground())
												  .Text(LOCTEXT("SignallingTip", "The Signalling Server is running and may be accessed via the following URLs (network settings permitting):"))
												  .WrapTextAt(400)],
				FText());

			MenuBuilder.AddWidget(SNew(SBox)
									  .Padding(FMargin(32.0f, 3.0f))
										  [SNew(STextBlock)
												  .ColorAndOpacity(FSlateColor::UseSubduedForeground())
												  .Text(FText::FromString(FString::Printf(TEXT("127.0.0.1:%d"), IPixelStreamingEditorModule::Get().GetViewerPort())))],
				FText());

			TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
			if (ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(AdapterAddresses))
			{
				for (TSharedPtr<FInternetAddr> AdapterAddress : AdapterAddresses)
				{
					MenuBuilder.AddWidget(SNew(SBox)
											  .Padding(FMargin(32.0f, 3.0f))
												  [SNew(STextBlock)
														  .ColorAndOpacity(FSlateColor::UseSubduedForeground())
														  .Text(FText::FromString(FString::Printf(TEXT("%s:%d"), *AdapterAddress->ToString(false), IPixelStreamingEditorModule::Get().GetViewerPort())))],
						FText());
				}
			}
		}
		MenuBuilder.EndSection();
	}

	void FPixelStreamingToolbar::RegisterStreamerControls(FMenuBuilder& MenuBuilder)
	{
		IPixelStreamingModule::Get().ForEachStreamer([&](TSharedPtr<IPixelStreamingStreamer> Streamer) {
			FString StreamerId = Streamer->GetId();
			MenuBuilder.BeginSection(FName(*StreamerId), FText::FromString(FString::Printf(TEXT("Streamer - %s"), *StreamerId)));
			{

				if (Streamer->IsStreaming())
				{
					FString VideoInput = TEXT("nothing (no video input)");
					if (TSharedPtr<FPixelStreamingVideoInput> Video = Streamer->GetVideoInput().Pin())
					{
						VideoInput = Video->ToString();
					}

					MenuBuilder.AddWidget(SNew(SBox)
											  .Padding(FMargin(16.0f, 3.0f))
												  [SNew(STextBlock)
														  .ColorAndOpacity(FSlateColor::UseSubduedForeground())
														  .Text(FText::FromString(FString::Printf(TEXT("Streaming %s"), *VideoInput)))
														  .WrapTextAt(400)],
						FText());

					MenuBuilder.AddMenuEntry(
						LOCTEXT("PixelStreaming_StopStreaming", "Stop Streaming"),
						LOCTEXT("PixelStreaming_StopStreamingToolTip", "Stop this streamer"),
						FSlateIcon(),
						FExecuteAction::CreateLambda([Streamer]() {
							Streamer->StopStreaming();
						}));
				}
				else
				{
					FPixelStreamingEditorModule& EditorModule = static_cast<FPixelStreamingEditorModule&>(IPixelStreamingEditorModule::Get());
					if (Streamer->GetId() == EditorModule.GetEditorStreamerID())
					{
						MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().StreamLevelEditor);
						MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().StreamEditor);
					}
					else
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("PixelStreaming_StartStreaming", "Start Streaming"),
							LOCTEXT("PixelStreaming_StartStreamingToolTip", "Start this streamer"),
							FSlateIcon(),
							FExecuteAction::CreateLambda([Streamer]() {
								Streamer->StartStreaming();
							}));
					}
				}
			}
			MenuBuilder.EndSection();
		});
	}

	void FPixelStreamingToolbar::RegisterCodecConfig(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("Codec", LOCTEXT("PixelStreamingCodecSettings", "Codec"));
		{
			MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().H264);
			MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().AV1);
			MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().VP8);
			MenuBuilder.AddMenuEntry(FPixelStreamingCommands::Get().VP9);
		}
		MenuBuilder.EndSection();
	}

	TSharedRef<SWidget> FPixelStreamingToolbar::GeneratePixelStreamingMenuContent(TSharedPtr<FUICommandList> InCommandList)
	{
		FToolMenuContext MenuContext(InCommandList);
		return UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.AddQuickMenu", MenuContext);
	}
} // namespace UE::EditorPixelStreaming

#undef LOCTEXT_NAMESPACE