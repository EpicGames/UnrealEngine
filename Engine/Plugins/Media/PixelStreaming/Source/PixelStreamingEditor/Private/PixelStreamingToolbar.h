// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"
#include "ToolMenus.h"
#include "IPixelStreamingModule.h"

namespace UE::EditorPixelStreaming
{
	class FPixelStreamingToolbar
	{
	public:
		FPixelStreamingToolbar();
		virtual ~FPixelStreamingToolbar();
		void StartStreaming();
		void StopStreaming();
		static TSharedRef<SWidget> GeneratePixelStreamingMenuContent(TSharedPtr<FUICommandList> InCommandList);
		static FText GetActiveViewportName();
		static const FSlateBrush* GetActiveViewportIcon();

	private:
		void RegisterMenus();
		void RegisterEmbeddedSignallingServerConfig(FMenuBuilder& MenuBuilder);
		void RegisterRemoteSignallingServerConfig(FMenuBuilder& MenuBuilder);
		void RegisterSignallingServerURLs(FMenuBuilder& MenuBuilder);
		void RegisterStreamerControls(FMenuBuilder& MenuBuilder);
		void RegisterVCamControls(FMenuBuilder& MenuBuilder);
		void RegisterCodecConfig(FMenuBuilder& MenuBuilder);

		TSharedPtr<class FUICommandList> PluginCommands;
	};
} // namespace UE::EditorPixelStreaming
