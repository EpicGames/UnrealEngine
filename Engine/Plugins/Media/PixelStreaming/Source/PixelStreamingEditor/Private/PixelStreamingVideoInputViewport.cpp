// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingVideoInputViewport.h"
#include "Async/Async.h"
#include "Settings.h"
#include "Utils.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureCapturerRHI.h"
#include "UnrealClient.h"
#include "Engine/GameViewportClient.h"
#include "RenderingThread.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "SLevelViewport.h"
#include "IPixelStreamingModule.h"

bool IsPIESessionRunning()
{
	return GEditor && GEditor->PlayWorld && GEditor->PlayWorld->WorldType == EWorldType::PIE;
}

TSharedPtr<FPixelStreamingVideoInputViewport> FPixelStreamingVideoInputViewport::Create(TSharedPtr<IPixelStreamingStreamer> InAssociatedStreamer)
{
	TSharedPtr<FPixelStreamingVideoInputViewport> NewInput = TSharedPtr<FPixelStreamingVideoInputViewport>(new FPixelStreamingVideoInputViewport());
	NewInput->AssociatedStreamer = InAssociatedStreamer;
	TWeakPtr<FPixelStreamingVideoInputViewport> WeakInput = NewInput;

	// Set up the callback on the game thread since FSlateApplication::Get() can only be used there
	AsyncTask(ENamedThreads::GameThread, [WeakInput]() {
		if (TSharedPtr<FPixelStreamingVideoInputViewport> Input = WeakInput.Pin())
		{
			// Bind to GameViewport delegate for PIE sessions
			Input->PIEDelegateHandle = UGameViewportClient::OnViewportRendered().AddSP(Input.ToSharedRef(), 
			&FPixelStreamingVideoInputViewport::OnPIEViewportRendered);

			// For non-PIE cases (just using level editor) then bind to slate window rendered
			FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
			if(!Renderer)
			{
				return;
			}

			Input->DelegateHandle = Renderer->OnSlateWindowRendered().AddSP(Input.ToSharedRef(), 
			&FPixelStreamingVideoInputViewport::OnWindowRendered);
		}
	});

	return NewInput;
}

FPixelStreamingVideoInputViewport::~FPixelStreamingVideoInputViewport()
{
	if (!IsEngineExitRequested())
	{
		AsyncTask(ENamedThreads::GameThread, [HandleCopy = DelegateHandle, PIEHandleCopy = PIEDelegateHandle]() {
			UGameViewportClient::OnViewportRendered().Remove(PIEHandleCopy);
			FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
			if(Renderer)
			{
				Renderer->OnSlateWindowRendered().Remove(HandleCopy);
			}
		});
	}
}

bool FPixelStreamingVideoInputViewport::FilterWindow(SWindow& InWindow)
{
	TSharedPtr<IPixelStreamingStreamer> Streamer = AssociatedStreamer.Pin();
	if (!Streamer || !Streamer->IsStreaming())
	{
		return false;
	}

	TSharedPtr<SWindow> TargetWindow = Streamer->GetTargetWindow().Pin();

	if (!TargetWindow)
	{
		return false;
	}

	// Check if the window we were passed is our target window.
	if (TargetWindow.Get() != &InWindow)
	{
		return false;
	}

	return true;
}

void FPixelStreamingVideoInputViewport::OnWindowRendered(SWindow& InWindow)
{
	// If there is a PIE session running then don't render level viewport
	if (IsPIESessionRunning())
	{
		return;
	}

	if (!FilterWindow(InWindow))
	{
		return;
	}

	TSharedPtr<IPixelStreamingStreamer> Streamer = AssociatedStreamer.Pin();
	if (!Streamer || !Streamer->IsStreaming())
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
	if (!ActiveLevelViewport.IsValid())
	{
		return;
	}

	FLevelEditorViewportClient& LevelViewportClient = ActiveLevelViewport->GetLevelViewportClient();
	FViewport* TargetViewport = LevelViewportClient.Viewport;

	SubmitViewport(TargetViewport);
}

void FPixelStreamingVideoInputViewport::OnPIEViewportRendered(FViewport* InViewport)
{
	// If gameclient viewport is rendered and it is not PIE we are not interested
	if (!IsPIESessionRunning() || !InViewport || !InViewport->IsPlayInEditorViewport())
	{
		return;
	}

	SubmitViewport(InViewport);
}

void FPixelStreamingVideoInputViewport::SubmitViewport(FViewport* InViewport)
{
	if(!InViewport)
	{
		return;
	}

	const FTextureRHIRef& FrameBuffer = InViewport->GetRenderTargetTexture();
	if (!FrameBuffer)
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(StreamViewportTextureCommand)
	([this, FrameBuffer](FRHICommandList& RHICmdList) {
		OnFrame(FPixelCaptureInputFrameRHI(FrameBuffer));
	});
}

FString FPixelStreamingVideoInputViewport::ToString()
{
	return TEXT("the Target Viewport");
}