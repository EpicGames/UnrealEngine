// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings.h"

namespace UE::EditorPixelStreaming::Settings
{
    template <typename T>
	inline void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<T>& CVar)
	{
		T Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value))
		{
			CVar->Set(Value, ECVF_SetByCommandline);
		}
	};

	inline void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<FString>& CVar, bool bStopOnSeparator = false)
	{
		FString Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value, bStopOnSeparator))
		{
			CVar->Set(*Value, ECVF_SetByCommandline);
		}
	};

	inline void CommandLineParseOption(const TCHAR* Match, TAutoConsoleVariable<bool>& CVar)
	{
		FString ValueMatch(Match);
		ValueMatch.Append(TEXT("="));
		FString Value;
		if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value))
		{
			if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
			{
				CVar->Set(true, ECVF_SetByCommandline);
			}
			else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
			{
				CVar->Set(false, ECVF_SetByCommandline);
			}
		}
		else if (FParse::Param(FCommandLine::Get(), Match))
		{
			CVar->Set(true, ECVF_SetByCommandline);
		}
	}
	
	TAutoConsoleVariable<bool> CVarEditorPixelStreamingStartOnLaunch(
		TEXT("PixelStreaming.Editor.StartOnLaunch"),
		false,
		TEXT("Start streaming the Editor as soon as it launches. Default: false"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarEditorPixelStreamingUseRemoteSignallingServer(
		TEXT("PixelStreaming.Editor.UseRemoteSignallingServer"),
		false,
		TEXT("Use a remote signalling server. Default: false"),
		ECVF_Default);

    TAutoConsoleVariable<FString> CVarEditorPixelStreamingSource(
		TEXT("PixelStreaming.Editor.Source"),
		TEXT("Editor"),
		TEXT("Editor PixelStreaming source. Supported values are `Editor`, `LevelEditor`"),
		ECVF_Default);

    void InitialiseSettings()
    {
        // Options parse (if these exist they are set to true)
		CommandLineParseOption(TEXT("EditorPixelStreamingStartOnLaunch"), CVarEditorPixelStreamingStartOnLaunch);
		CommandLineParseOption(TEXT("EditorPixelStreamingUseRemoteSignallingServer"), CVarEditorPixelStreamingUseRemoteSignallingServer);

        CommandLineParseValue(TEXT("EditorPixelStreamingSource="), CVarEditorPixelStreamingSource);
    }
}