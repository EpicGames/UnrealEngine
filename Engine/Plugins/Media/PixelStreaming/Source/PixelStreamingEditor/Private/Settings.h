// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"

namespace UE::EditorPixelStreaming::Settings
{
    extern void InitialiseSettings();

	// Begin Editor CVars
	extern TAutoConsoleVariable<bool> CVarEditorPixelStreamingStartOnLaunch;
	extern TAutoConsoleVariable<bool> CVarEditorPixelStreamingUseRemoteSignallingServer;
    extern TAutoConsoleVariable<FString> CVarEditorPixelStreamingSource;
    // End Editor CVars
}