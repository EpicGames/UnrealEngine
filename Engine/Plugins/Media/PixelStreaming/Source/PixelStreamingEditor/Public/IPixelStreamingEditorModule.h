// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PixelStreamingServers.h"
#include "PixelStreamingEditorUtils.h"

/**
 * The public interface of the Pixel Streaming Editor module.
 */
class PIXELSTREAMINGEDITOR_API IPixelStreamingEditorModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreamingEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreamingEditorModule>("PixelStreamingEditor");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("PixelStreamingEditor"); }

	/**
	 * Starts the editor specific streamer.
	 *
	 * @param InStreamType The stream type for this streamer.
	 */
	virtual void StartStreaming(UE::EditorPixelStreaming::EStreamTypes InStreamType) = 0;

	/**
	 * Starts the editor specific streamer.
	 *
	 */
	virtual void StopStreaming() = 0;

	/**
	 * Start the inbuilt C++ signalling server
	 *
	 */
	virtual void StartSignalling() = 0;

	/**
	 * Stop the inbuilt C++ signalling server
	 *
	 */
	virtual void StopSignalling() = 0;

	/**
	 * Get the inbuilt C++ signalling server
	 *
	 * @return TSharedPtr<UE::PixelStreamingServers::IServer> The signalling server
	 */
	virtual TSharedPtr<UE::PixelStreamingServers::IServer> GetSignallingServer() = 0;

	/**
	 * Set the domain for the inbuilt c++ signalling server
	 *
	 * @param InSignallingDomain The domain for the inbuilt c++ signalling server
	 */
	virtual void SetSignallingDomain(const FString& InSignallingDomain) = 0;

	/**
	 * Get the domain for the inbuilt c++ signalling server
	 *
	 * @return FString The inbuilt c++ signalling server's domain
	 */
	virtual FString GetSignallingDomain() = 0;

	/**
	 * Set the port streamers connect to for the inbuilt c++ signalling server
	 *
	 * @param InStreamerPort The port streamers connect to for the inbuilt c++ signalling server
	 */
	virtual void SetStreamerPort(int32 InStreamerPort) = 0;

	/**
	 * Get the port streamers connect to for the inbuilt c++ signalling server
	 *
	 * @return int32 The inbuilt c++ signalling server's port for streamers connect to
	 */
	virtual int32 GetStreamerPort() = 0;

	/**
	 * Set the port viewers connect to for the inbuilt c++ signalling server
	 *
	 * @param InViewerPort The port viewers connect to for the inbuilt c++ signalling server
	 */
	virtual void SetViewerPort(int32 InViewerPort) = 0;

	/**
	 * Get the port viewers connect to for the inbuilt c++ signalling server
	 *
	 * @return int32 The inbuilt c++ signalling server's port for viewers connect to
	 */
	virtual int32 GetViewerPort() = 0;

	/**
	 * @brief Get if streamers should use an external signalling server or the inbuilt c++ signalling server
	 *
	 * @return true Use an external signalling server
	 * @return false Use the inbuilt c++ signalling server
	 */
	virtual bool UseExternalSignallingServer() = 0;

	/**
	 * @brief Set if streamers should use an external signalling server or the inbuilt c++ signalling server
	 *
	 * @param bUseExternalSignallingServer Should streamers use an external signalling server
	 */
	virtual void UseExternalSignallingServer(bool bUseExternalSignallingServer) = 0;
};
