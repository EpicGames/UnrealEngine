// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "RHIResources.h"

class SWindow;

namespace UE::EditorPixelStreaming
{
	enum class EStreamTypes : uint8
	{
		LevelEditorViewport = 0,
		Editor = 1
	};

	FString ToString(EStreamTypes StreamType);
	const TCHAR* ToString(EWindowType Type);
	const FString HashWindow(SWindow& SlateWindow, const FTextureRHIRef& FrameBuffer);
}