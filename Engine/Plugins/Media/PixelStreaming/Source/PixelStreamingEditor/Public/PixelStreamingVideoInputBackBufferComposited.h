// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInputRHI.h"

#include "Widgets/SWindow.h"
#include "RendererInterface.h"
#include "Slate/SlateViewportProvider.h"
#include "Delegates/IDelegateInstance.h"

/*
 * Use this if you want to send the full UE editor as video input.
 */
class PIXELSTREAMINGEDITOR_API FPixelStreamingVideoInputBackBufferComposited : public FPixelStreamingVideoInputRHI
{
public:
    static TSharedPtr<FPixelStreamingVideoInputBackBufferComposited> Create();
    virtual ~FPixelStreamingVideoInputBackBufferComposited();

    virtual FString ToString() override;

    DECLARE_MULTICAST_DELEGATE_OneParam(FOnFrameSizeChanged, TWeakPtr<FIntRect>);
    FOnFrameSizeChanged OnFrameSizeChanged;

private:
    // Our class to keep window and texture information. Use of this struct prevents SWindow properties being updated composition as well
    // as preventing deletion of our staging textures pre composition
    class FTexturedWindow
    {
    public:
        FTexturedWindow(FVector2D InPositionInScreen, FVector2D InSizeInScreen, float InOpacity, EWindowType InType, SWindow *InOwningWindow)
            : PositionInScreen(InPositionInScreen), SizeInScreen(InSizeInScreen), Opacity(InOpacity), Type(InType), Texture(nullptr), OwningWindow(InOwningWindow)
        {
        }

        FVector2D GetPositionInScreen() { return PositionInScreen; }
        FVector2D GetSizeInScreen() { return SizeInScreen; }
        float GetOpacity() { return Opacity; }
        EWindowType GetType() { return Type; }
        SWindow *GetOwningWindow() { return OwningWindow; }
        TRefCountPtr<IPooledRenderTarget> &GetTexture() { return Texture; }
        void SetTexture(TRefCountPtr<IPooledRenderTarget> InTexture) { Texture = InTexture; }

    private:
        FVector2D PositionInScreen;
        FVector2D SizeInScreen;
        float Opacity;
        EWindowType Type;
        TRefCountPtr<IPooledRenderTarget> Texture;
        SWindow *OwningWindow;
    };

private:
    FPixelStreamingVideoInputBackBufferComposited();
    void CompositeWindows();

    void OnBackBufferReady(SWindow &SlateWindow, ISlateViewportProvider& ViewportProvider);
    void OnPreTick(float DeltaTime);

    FDelegateHandle OnBackBufferReadyToPresentHandle;
    FDelegateHandle OnPreTickHandle;

    TArray<FTexturedWindow> TopLevelWindows;

    FCriticalSection TopLevelWindowsCriticalSection;

    TSharedPtr<FIntRect> SharedFrameRect;

private:
    // Util functions for 2D vectors
    template <class T>
    T VectorMax(const T A, const T B)
    {
        // Returns the component-wise maximum of two vectors
        return T(FMath::Max(A.X, B.X), FMath::Max(A.Y, B.Y));
    }

    template <class T>
    T VectorMin(const T A, const T B)
    {
        // Returns the component-wise minimum of two vectors
        return T(FMath::Min(A.X, B.X), FMath::Min(A.Y, B.Y));
    }
};