// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingVideoInputRHI.h"
#include "IPixelStreamingStreamer.h"
#include "Delegates/IDelegateInstance.h"
#include "UnrealClient.h"

/*
 * Use this if you want to send the UE primary scene viewport as video input - will only work in editor.
 */
class PIXELSTREAMINGEDITOR_API FPixelStreamingVideoInputViewport : public FPixelStreamingVideoInputRHI
{
public:
	static TSharedPtr<FPixelStreamingVideoInputViewport> Create(TSharedPtr<IPixelStreamingStreamer> InAssociatedStreamer);
	virtual ~FPixelStreamingVideoInputViewport();

	virtual FString ToString() override;

private:
	FPixelStreamingVideoInputViewport() = default;

	bool FilterWindow(SWindow& Window);
	void OnWindowRendered(SWindow& Window);
	void OnPIEViewportRendered(FViewport* InViewport);
	void SubmitViewport(FViewport* InViewport);

	FDelegateHandle DelegateHandle;
	FDelegateHandle PIEDelegateHandle;

	FName TargetViewportType = FName(FString(TEXT("SceneViewport")));

	TWeakPtr<IPixelStreamingStreamer> AssociatedStreamer;
};
