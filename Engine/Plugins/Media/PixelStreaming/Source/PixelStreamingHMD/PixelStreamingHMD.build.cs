// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PixelStreamingHMD : ModuleRules
	{
		public PixelStreamingHMD(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"RHI",
					"RenderCore",
					"Renderer",
					"HeadMountedDisplay",
					"XRBase",
					"SlateCore"
				}
			);
		}
	}
}