// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingHMD.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "PostProcess/PostProcessHMD.h"
#include "GameFramework/WorldSettings.h"
#include "Settings.h"
#include "Widgets/SWindow.h"
#include "SceneView.h"

DEFINE_LOG_CATEGORY(LogPixelStreamingHMD);

FPixelStreamingHMD::FPixelStreamingHMD(const FAutoRegister& AutoRegister)
	: FHeadMountedDisplayBase(nullptr)
	, FHMDSceneViewExtension(AutoRegister)
	, CurHmdTransform(FTransform::Identity)
	, WorldToMeters(100.0f)
	, InterpupillaryDistance(0.0f)
	, bStereoEnabled(true)
{
}

FPixelStreamingHMD::~FPixelStreamingHMD()
{
}

void FPixelStreamingHMD::SetEyeViews(FTransform Left, FMatrix LeftProj, FTransform Right, FMatrix RightProj, FTransform HMD)
{
	// This is our intialization message we can use this set the base position if it hasn't been set.
	if (BasePosition == FVector::ZeroVector)
	{
		SetBasePosition(HMD.GetLocation());
	}

	// Set the HMD transform
	SetTransform(HMD);

	// Make left and right relative the HMD
	FTransform HMDInv = HMD.Inverse();
	FTransform LeftRelative = Left * HMDInv;
	FTransform RightRelative = Right * HMDInv;

	// Undo rotation of HMD, then find relative positional offset between eyes and HMD
	LeftEyePosOffset = LeftRelative.GetLocation();
	RightEyePosOffset = RightRelative.GetLocation();

	float IPD = FVector::Dist(RightEyePosOffset, LeftEyePosOffset);
	// Set the IPD (in meters)
	SetInterpupillaryDistance(IPD / 100.0f);

	// Calculate left/right view orientation relative to HMD
	LeftEyeRotOffset = LeftRelative.GetRotation();
	RightEyeRotOffset = RightRelative.GetRotation();

	// Calculate the horizontal and vertical FoV from the projection matrix (left and right eye will have same FoVs)
	HFoVRads = 2.0f * FMath::Atan(1.0f / LeftProj.M[0][0]);
	VFoVRads = 2.0f * FMath::Atan(1.0f / LeftProj.M[1][1]);

	// Extract the left/right eye projection offsets
	CurLeftEyeProjOffsetX = -LeftProj.M[0][2];	 // 0.242512569
	CurLeftEyeProjOffsetY = -LeftProj.M[1][2];	 // 0.193187475
	CurRightEyeProjOffsetX = -RightProj.M[0][2]; // -0.242512569
	CurRightEyeProjOffsetY = -RightProj.M[1][2]; // 0.193187475

	// Extract near and farclip planes
	NearClip = LeftProj.M[3][2] / (LeftProj.M[2][2] - 1);
	FarClip = LeftProj.M[3][2] / (LeftProj.M[2][2] + 1);
	SetClippingPlanes(NearClip, FarClip);

	// Calculate target aspect ratio from the projection matrix (left and right eye will have same aspect ratio)
	TargetAspectRatio = tan(HFoVRads * 0.5f) / tan(VFoVRads * 0.5f);

	TSharedPtr<SWindow> TargetWindow = GEngine->GameViewport->GetWindow();
	FVector2f			SizeInScreen = TargetWindow->GetSizeInScreen();
	const float			InWidth = SizeInScreen.X / 2.f;
	const float			InHeight = SizeInScreen.Y;
	const float			AspectRatio = InWidth / InHeight;

	// If current resolution does not match remote device aspect ratio, we will change resolution to match aspect ratio (though we rate limit res change to every 5s)
	if (UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDMatchAspectRatio.GetValueOnAnyThread() && FMath::Abs(AspectRatio - TargetAspectRatio) > 0.01 && !bReceivedTransforms)
	{
		int TargetHeight = InHeight;
		int TargetWidth = InHeight * TargetAspectRatio * 2.0f;
		UE_LOGF(LogPixelStreamingHMD, Warning, "XR Pixel Streaming streaming resolution not matching remote device aspect ratio. Changing resolution to %dx%d", TargetWidth, TargetHeight);
		FString ChangeResCommand = FString::Printf(TEXT("r.SetRes %dx%d"), TargetWidth, TargetHeight);
		GEngine->Exec(GEngine->GetWorld(), *ChangeResCommand);
	}

	// If we know we are doing XR update some CVars for Pixel Streaming to optimise for it.
	if (!bReceivedTransforms)
	{
		// Couple engine's render rate and streaming rate
		if (IConsoleVariable* DecoupleFramerateCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.DecoupleFramerate")))
		{
			DecoupleFramerateCVar->Set(false);
		}

		// Set the rate at which we will stream
		if (IConsoleVariable* StreamFPS = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.WebRTC.Fps")))
		{
			StreamFPS->Set(90);
		}

		// Set the MinQP to bound quality
		if (IConsoleVariable* MinQP = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.Encoder.MinQP")))
		{
			MinQP->Set(15);
		}

		// Necessary for coupled framerate
		if (IConsoleVariable* CaptureUseFence = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.CaptureUseFence")))
		{
			CaptureUseFence->Set(false);
		}

		// Disable keyframes interval, only send them as needed
		if (IConsoleVariable* KeyframeInterval = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.Encoder.KeyframeInterval")))
		{
			KeyframeInterval->Set(0);
		}
	}

	bReceivedTransforms = true;
}

float FPixelStreamingHMD::GetWorldToMetersScale() const
{
	return WorldToMeters;
}

bool FPixelStreamingHMD::IsHMDEnabled() const
{
	return UE::PixelStreamingHMD::Settings::CVarPixelStreamingEnableHMD.GetValueOnAnyThread();
}

void FPixelStreamingHMD::EnableHMD(bool enable)
{
	UE::PixelStreamingHMD::Settings::CVarPixelStreamingEnableHMD->Set(enable, ECVF_SetByCode);
}

bool FPixelStreamingHMD::GetHMDMonitorInfo(MonitorInfo& MonitorDesc)
{
	MonitorDesc.MonitorName = "PixelStreamingHMD";
	MonitorDesc.MonitorId = 0;
	MonitorDesc.DesktopX = MonitorDesc.DesktopY = MonitorDesc.ResolutionX = MonitorDesc.ResolutionY = 0;
	return false;
}

void FPixelStreamingHMD::GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const
{
	OutHFOVInDegrees = FMath::RadiansToDegrees(HFoVRads);
	OutVFOVInDegrees = FMath::RadiansToDegrees(VFoVRads);
}

bool FPixelStreamingHMD::GetRelativeEyePose(int32 DeviceId, int32 ViewIndex, FQuat& OutOrientation, FVector& OutPosition)
{
	if (DeviceId != IXRTrackingSystem::HMDDeviceId)
	{
		return false;
	}

	if (UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDApplyEyePosition.GetValueOnAnyThread())
	{
		// If not using override IPD get the actual translation of each eye from the HMD transform and apply that.
		OutPosition = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? LeftEyePosOffset : RightEyePosOffset;
	}

	// Apply eye rotation if this enabled (default: true)
	if (UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDApplyEyeRotation.GetValueOnAnyThread())
	{
		OutOrientation = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? LeftEyeRotOffset : RightEyeRotOffset;
	}

	return false;
}

bool FPixelStreamingHMD::EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type)
{
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::HeadMountedDisplay)
	{
		OutDevices.Add(IXRTrackingSystem::HMDDeviceId);
		return true;
	}
	return false;
}

void FPixelStreamingHMD::SetInterpupillaryDistance(float NewInterpupillaryDistance)
{
	InterpupillaryDistance = NewInterpupillaryDistance;
}

float FPixelStreamingHMD::GetInterpupillaryDistance() const
{
	return InterpupillaryDistance;
}

bool FPixelStreamingHMD::GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition)
{
	if (DeviceId != IXRTrackingSystem::HMDDeviceId)
	{
		return false;
	}
	CurrentOrientation = CurHmdTransform.GetRotation();
	CurrentPosition = CurHmdTransform.GetTranslation();
	return true;
}

bool FPixelStreamingHMD::IsChromaAbCorrectionEnabled() const
{
	return false;
}

void FPixelStreamingHMD::ResetOrientationAndPosition(float yaw)
{
	ResetOrientation(yaw);
	ResetPosition();
}

void FPixelStreamingHMD::DrawDistortionMesh_RenderThread(struct FHeadMountedDisplayPassContext& Context, const FIntPoint& TextureSize)
{
	// Note: Left intentionally blank as we do not want to do any distortion on the UE side, the device will distort the image for us.
}

bool FPixelStreamingHMD::IsStereoEnabled() const
{
	return bStereoEnabled;
}

bool FPixelStreamingHMD::EnableStereo(bool stereo)
{
	bStereoEnabled = stereo;
	return bStereoEnabled;
}

void FPixelStreamingHMD::AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const
{
	SizeX = SizeX / 2;
	X += SizeX * ViewIndex;
}

void FPixelStreamingHMD::CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation, const float InWorldToMeters, FVector& ViewLocation)
{
	if (ViewIndex == INDEX_NONE)
	{
		return;
	}

	float OverrideIPD = UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDIPD.GetValueOnAnyThread();

	// If not received any transforms yet, just do default offset of half IPD
	if (!bReceivedTransforms)
	{
		float		IPDCentimeters = OverrideIPD > 0.0f ? OverrideIPD : InterpupillaryDistance * 100.0f;
		const float PassOffset = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? -IPDCentimeters * 0.5f : IPDCentimeters * 0.5f;
		ViewLocation += ViewRotation.Quaternion().RotateVector(FVector(0, PassOffset, 0));
	}
	else
	{
		if (OverrideIPD > 0.0f)
		{
			// If using override IPD only translate along the horizontal plane.
			const float EyeTranslationOffset = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? -OverrideIPD * 0.5f : OverrideIPD * 0.5f;
			ViewLocation += ViewRotation.Quaternion().RotateVector(FVector(0, EyeTranslationOffset, 0));
		}
		else if (UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDApplyEyePosition.GetValueOnAnyThread())
		{
			// If not using override IPD get the actual translation of each eye from the HMD transform and apply that.
			ViewLocation += (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? ViewRotation.Quaternion().RotateVector(LeftEyePosOffset) : ViewRotation.Quaternion().RotateVector(RightEyePosOffset);
		}

		// Apply eye rotation if this enabled (default: true)
		if (UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDApplyEyeRotation.GetValueOnAnyThread())
		{
			ViewRotation += (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? LeftEyeRotOffset.Rotator() : RightEyeRotOffset.Rotator();
		}
	}
}

FMatrix FPixelStreamingHMD::GetStereoProjectionMatrix(const int32 ViewIndex) const
{

	float ProjOffsetX = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? CurLeftEyeProjOffsetX : CurRightEyeProjOffsetX;
	float ProjOffsetY = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? CurLeftEyeProjOffsetY : CurRightEyeProjOffsetY;

	// Check OverrideProjectOffset X & Y if they have been set by the user, use them instead of the values from WebXR
	{
		float OverrideProjectionOffsetX = UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDProjectionOffsetX.GetValueOnAnyThread();
		float OverrideProjectionOffsetY = UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDProjectionOffsetY.GetValueOnAnyThread();

		if (OverrideProjectionOffsetX >= 0.0f)
		{
			ProjOffsetX = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? OverrideProjectionOffsetX : -OverrideProjectionOffsetX;
		}
		if (OverrideProjectionOffsetY >= 0.0f)
		{
			ProjOffsetY = OverrideProjectionOffsetY;
		}
	}

	const float HFoVOverride = UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDHFOV.GetValueOnAnyThread();
	const float VFoVOverride = UE::PixelStreamingHMD::Settings::CVarPixelStreamingHMDVFOV.GetValueOnAnyThread();
	// FoV's are either passed in from the remote device or taken from the FoV override CVars.
	const float HalfVFov = VFoVOverride > 0.0f ? FMath::DegreesToRadians(VFoVOverride) * 0.5f : VFoVRads * 0.5f;
	const float HalfHFov = HFoVOverride > 0.0f ? FMath::DegreesToRadians(HFoVOverride) * 0.5f : HFoVRads * 0.5f;

	const float InNearZ = GNearClippingPlane;
	const float TanHalfHFov = tan(HalfHFov);
	const float TanHalfVFov = tan(HalfVFov);
	const float XS = 1.0f / TanHalfHFov;
	const float YS = 1.0f / TanHalfVFov;

	// Apply eye off-center translation
	const FTranslationMatrix OffCenterProjection = FTranslationMatrix(FVector(ProjOffsetX, ProjOffsetY, 0));
	const float				 ZNear = GNearClippingPlane_RenderThread;

	FMatrix ProjMatrix = FMatrix(
		FPlane(XS, 0.0f, 0.0f, 0.0f),
		FPlane(0.0f, YS, 0.0f, 0.0f),
		FPlane(0.0f, 0.0f, 0.0f, 1.0f),
		FPlane(0.0f, 0.0, ZNear, 0.0f));

	const FMatrix OutMatrix = ProjMatrix * OffCenterProjection;
	return OutMatrix;
}

void FPixelStreamingHMD::GetEyeRenderParams_RenderThread(const FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const
{
	EyeToSrcUVOffsetValue = FVector2D::ZeroVector;
	EyeToSrcUVScaleValue = FVector2D(1.0f, 1.0f);
}

void FPixelStreamingHMD::SetupViewFamily(FSceneViewFamily& InViewFamily)
{
	InViewFamily.EngineShowFlags.MotionBlur = 0;
	// Note: We do not want to apply any distortion on the UE side.
	InViewFamily.EngineShowFlags.HMDDistortion = false;
	InViewFamily.EngineShowFlags.StereoRendering = IsStereoEnabled();

	if (UWorld* World = GWorld)
	{
		WorldToMeters = World->GetWorldSettings()->WorldToMeters;
	}
}