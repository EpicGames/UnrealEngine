// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingHMDModule.h"
#include "IXRTrackingSystem.h"
#include "PixelStreamingHMD.h"
#include "Settings.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

namespace UE::PixelStreamingHMD
{
	/**
	 * IModuleInterface implementation
	 */
	void FPixelStreamingHMDModule::StartupModule()
	{
		Settings::InitialiseSettings();
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
		ActiveXRSystem = EPixelStreamingXRSystem::Unknown;
	}

	void FPixelStreamingHMDModule::ShutdownModule()
	{
		// Remove the modules hold of the ptr
		// HMD.Reset();
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	}
	/**
	 * End IModuleInterface implementation
	 */

	/**
	 *
	 */
	TSharedPtr<IXRTrackingSystem, ESPMode::ThreadSafe> FPixelStreamingHMDModule::CreateTrackingSystem()
	{
		if (!Settings::CVarPixelStreamingEnableHMD.GetValueOnAnyThread())
		{
			return nullptr;
		}

		TSharedRef<FPixelStreamingHMD> PixelStreamingHMD = FSceneViewExtensions::NewExtension<FPixelStreamingHMD>();
		if (PixelStreamingHMD->IsInitialized())
		{
			FGenericCrashContext::SetEngineData(TEXT("XR"), TEXT("XR PixelStreaming"));
			return PixelStreamingHMD;
		}
		return nullptr;
	}

	FPixelStreamingHMD* FPixelStreamingHMDModule::GetPixelStreamingHMD() const
	{
		static FName SystemName(TEXT("PixelStreamingHMD"));
		if (GEngine->XRSystem.IsValid() && (GEngine->XRSystem->GetSystemName() == SystemName))
		{
			return static_cast<FPixelStreamingHMD*>(GEngine->XRSystem.Get());
		}

		return nullptr;
	}
} // namespace UE::PixelStreamingHMD

IMPLEMENT_MODULE(UE::PixelStreamingHMD::FPixelStreamingHMDModule, PixelStreamingHMD)