// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingHMDModule.h"
#include "PixelStreamingHMD.h"

class IXRTrackingSystem;

namespace UE::PixelStreamingHMD
{
	/*
	 * This module allows HMD input to be used with pixel streaming
	 */
	class FPixelStreamingHMDModule : public IPixelStreamingHMDModule
	{
	public:
		FPixelStreamingHMD* GetPixelStreamingHMD() const;
		EPixelStreamingXRSystem GetActiveXRSystem() { return ActiveXRSystem; }
		void SetActiveXRSystem(EPixelStreamingXRSystem System) { ActiveXRSystem = System; }

	private:
		/** IModuleInterface implementation */
		void StartupModule() override;
		void ShutdownModule() override;
		/** End IModuleInterface implementation */

		/** IHeadMountedDisplayModule implementation */
		virtual TSharedPtr<IXRTrackingSystem, ESPMode::ThreadSafe> CreateTrackingSystem() override;
		FString GetModuleKeyName() const override { return FString(TEXT("PixelStreamingHMD")); }
		bool IsHMDConnected() override { return true; }
		/** IHeadMountedDisplayModule implementation */

		TSharedPtr<FPixelStreamingHMD, ESPMode::ThreadSafe> HMD;
		EPixelStreamingXRSystem ActiveXRSystem;
	};
} // namespace UE::PixelStreamingHMD