// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings.h"
#include "Misc/DefaultValueHelper.h"

namespace UE::PixelStreamingHMD::Settings
{
	void CommandLineParseOption(const TCHAR* Match, TAutoConsoleVariable<bool>& CVar)
	{
		FString ValueMatch(Match);
		ValueMatch.Append(TEXT("="));
		FString Value;
		if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value))
		{
			if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
			{
				CVar->Set(true, ECVF_SetByCommandline);
			}
			else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
			{
				CVar->Set(false, ECVF_SetByCommandline);
			}
		}
		else if (FParse::Param(FCommandLine::Get(), Match))
		{
			CVar->Set(true, ECVF_SetByCommandline);
		}
	}

	template <typename T>
	void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<T>& CVar)
	{
		T Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value))
		{
			CVar->Set(Value, ECVF_SetByCommandline);
		}
	};

	inline void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<FString>& CVar, bool bStopOnSeparator = false)
	{
		FString Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value, bStopOnSeparator))
		{
			CVar->Set(*Value, ECVF_SetByCommandline);
		}
	};

	TAutoConsoleVariable<bool> CVarPixelStreamingEnableHMD(
		TEXT("PixelStreaming.HMD.Enable"),
		false,
		TEXT("Enables HMD specific functionality for Pixel Streaming. Namely input handling and stereoscopic rendering"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingHMDMatchAspectRatio(
		TEXT("PixelStreaming.HMD.MatchAspectRatio"),
		true,
		TEXT("If true automatically resize the rendering resolution to match the aspect ratio determined by the HFoV and VFoV"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingHMDApplyEyePosition(
		TEXT("PixelStreaming.HMD.ApplyEyePosition"),
		true,
		TEXT("If true automatically position each eye's rendering by whatever amount WebXR reports for each left-right XRView. If false do no eye positioning. Default: false"),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarPixelStreamingHMDApplyEyeRotation(
		TEXT("PixelStreaming.HMD.ApplyEyeRotation"),
		true,
		TEXT("If true automatically rotate each eye's rendering by whatever amount WebXR reports for each left-right XRView. If false do no eye rotation. Default: false"),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingHMDHFOV(
		TEXT("PixelStreaming.HMD.HFOV"),
		-1.0f,
		TEXT("Overrides the horizontal field of view for HMD rendering, values are in degrees and values less than 0.0f disable the override."),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingHMDVFOV(
		TEXT("PixelStreaming.HMD.VFOV"),
		-1.0f,
		TEXT("Overrides the vertical field of view for HMD rendering, values are in degrees and values less than 0.0f disable the override."),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingHMDIPD(
		TEXT("PixelStreaming.HMD.IPD"),
		-1.0f,
		TEXT("Overrides the HMD IPD (interpupillary distance), values are in centimeters and values less than 0.0f disable the override."),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingHMDProjectionOffsetX(
		TEXT("PixelStreaming.HMD.ProjectionOffsetX"),
		-1.0f,
		TEXT("Overrides the left/right eye projection matrix x-offset, values are in clip space and values less than 0.0f disable the override."),
		ECVF_Default);

	TAutoConsoleVariable<float> CVarPixelStreamingHMDProjectionOffsetY(
		TEXT("PixelStreaming.HMD.ProjectionOffsetY"),
		-1.0f,
		TEXT("Overrides the left-right eye projection matrix y-offset, values are in clip space and values less than 0.0f disable the override."),
		ECVF_Default);

	void InitialiseSettings()
	{
		CommandLineParseOption(TEXT("PixelStreamingEnableHMD"), CVarPixelStreamingEnableHMD);
		CommandLineParseOption(TEXT("PixelStreamingHMDMatchAspectRatio"), CVarPixelStreamingHMDMatchAspectRatio);
		CommandLineParseOption(TEXT("PixelStreamingHMDApplyEyeRotation"), CVarPixelStreamingHMDApplyEyeRotation);
		CommandLineParseOption(TEXT("PixelStreamingHMDApplyEyePosition"), CVarPixelStreamingHMDApplyEyePosition);
		CommandLineParseValue(TEXT("PixelStreamingHMDHFOV="), CVarPixelStreamingHMDHFOV);
		CommandLineParseValue(TEXT("PixelStreamingHMDVFOV="), CVarPixelStreamingHMDVFOV);
		CommandLineParseValue(TEXT("PixelStreamingHMDVIPD="), CVarPixelStreamingHMDIPD);
		CommandLineParseValue(TEXT("PixelStreamingHMDProjectionOffsetX="), CVarPixelStreamingHMDProjectionOffsetX);
		CommandLineParseValue(TEXT("PixelStreamingHMDProjectionOffsetY="), CVarPixelStreamingHMDProjectionOffsetY);
	}
} // namespace UE::PixelStreamingHMD::Settings