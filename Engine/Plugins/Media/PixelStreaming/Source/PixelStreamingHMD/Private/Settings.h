// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"

namespace UE::PixelStreamingHMD::Settings
{
	extern void InitialiseSettings();
	extern void CommandLineParseOption();

	// Begin CVars
	extern TAutoConsoleVariable<bool> CVarPixelStreamingEnableHMD;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingHMDMatchAspectRatio;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingHMDApplyEyePosition;
	extern TAutoConsoleVariable<bool> CVarPixelStreamingHMDApplyEyeRotation;
	extern TAutoConsoleVariable<float> CVarPixelStreamingHMDHFOV;
	extern TAutoConsoleVariable<float> CVarPixelStreamingHMDVFOV;
	extern TAutoConsoleVariable<float> CVarPixelStreamingHMDIPD;
	extern TAutoConsoleVariable<float> CVarPixelStreamingHMDProjectionOffsetX;
	extern TAutoConsoleVariable<float> CVarPixelStreamingHMDProjectionOffsetY;
	// End CVars

} // namespace UE::PixelStreamingHMD::Settings
