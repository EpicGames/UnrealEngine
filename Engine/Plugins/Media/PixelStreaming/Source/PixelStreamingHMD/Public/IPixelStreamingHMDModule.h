// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IHeadMountedDisplayModule.h"
#include "Modules/ModuleManager.h"
#include "PixelStreamingHMDEnums.h"
#include "PixelStreamingHMD.h"

/**
 * The public interface of the Pixel Streaming HMD module.
 */
class PIXELSTREAMINGHMD_API IPixelStreamingHMDModule : public IHeadMountedDisplayModule
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreamingHMDModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreamingHMDModule>("PixelStreamingHMD");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("PixelStreamingHMD"); }

	/**
	 * @brief Get the Pixel Streaming HMD object
	 *
	 * @return FPixelStreamingHMD*
	 */
	virtual FPixelStreamingHMD* GetPixelStreamingHMD() const = 0;

	/**
	 * @brief Get the Active XR System
	 *
	 * @return EPixelStreamingXRSystem
	 */
	virtual EPixelStreamingXRSystem GetActiveXRSystem() = 0;

	virtual void SetActiveXRSystem(EPixelStreamingXRSystem System) = 0;
};
