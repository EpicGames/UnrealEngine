// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HeadMountedDisplayBase.h"
#include "SceneViewExtension.h"
#include "XRTrackingSystemBase.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreamingHMD, Log, All);

class APlayerController;
class FSceneView;
class FSceneViewFamily;
class UCanvas;

/**
 * Pixel Streamed Head Mounted Display
 */
class PIXELSTREAMINGHMD_API FPixelStreamingHMD : public FHeadMountedDisplayBase, public FHMDSceneViewExtension
{
public:
	/** IXRTrackingSystem interface */
	virtual FName GetSystemName() const override
	{
		static FName DefaultName(TEXT("PixelStreamingHMD"));
		return DefaultName;
	}

	int32 GetXRSystemFlags() const { return EXRSystemFlags::IsHeadMounted; }
	virtual bool GetRelativeEyePose(int32 DeviceId, int32 ViewIndex, FQuat& OutOrientation, FVector& OutPosition) override;
	virtual void SetBasePosition(const FVector& InBasePosition) override { BasePosition = InBasePosition; };
	virtual FVector GetBasePosition() const override { return BasePosition; }

	virtual bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type = EXRTrackedDeviceType::Any) override;

	virtual void SetInterpupillaryDistance(float NewInterpupillaryDistance) override;
	virtual float GetInterpupillaryDistance() const override;

	virtual void ResetOrientationAndPosition(float yaw = 0.f) override;
	virtual void ResetOrientation(float Yaw = 0.f) override {}
	virtual void ResetPosition() override {}

	virtual bool GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition) override;
	virtual void SetBaseRotation(const FRotator& BaseRot) override{};
	virtual FRotator GetBaseRotation() const override { return FRotator::ZeroRotator; }

	virtual void SetBaseOrientation(const FQuat& BaseOrient) override {}
	virtual FQuat GetBaseOrientation() const override { return FQuat::Identity; }

	virtual class IHeadMountedDisplay* GetHMDDevice() override { return this; }

	virtual class TSharedPtr<class IStereoRendering, ESPMode::ThreadSafe> GetStereoRenderingDevice() override
	{
		return SharedThis(this);
	}

	protected :
		/** FXRTrackingSystemBase protected interface */
		virtual float GetWorldToMetersScale() const override;

public:
	/** IHeadMountedDisplay interface */
	virtual bool IsHMDConnected() override { return true; }
	virtual bool IsHMDEnabled() const override;
	virtual void EnableHMD(bool allow = true) override;
	virtual bool GetHMDMonitorInfo(MonitorInfo&) override;
	virtual void GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const override;
	virtual bool IsChromaAbCorrectionEnabled() const override;
	virtual bool GetHMDDistortionEnabled(EShadingPath ShadingPath) const override { return false; }
	virtual void DrawDistortionMesh_RenderThread(struct FHeadMountedDisplayPassContext& Context, const FIntPoint& TextureSize) override;

	/** IStereoRendering interface */
	virtual bool IsStereoEnabled() const override;
	virtual bool EnableStereo(bool stereo = true) override;
	virtual void AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const override;
	virtual void CalculateStereoViewOffset(const int32 ViewIndex,
		FRotator& ViewRotation,
		const float InWorldToMeters,
		FVector& ViewLocation) override;
	virtual FMatrix GetStereoProjectionMatrix(const int32 ViewIndex) const override;
	virtual void GetEyeRenderParams_RenderThread(const struct FHeadMountedDisplayPassContext& Context,
		FVector2D& EyeToSrcUVScaleValue,
		FVector2D& EyeToSrcUVOffsetValue) const override;

	/** ISceneViewExtension interface */
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override;

	/** Constructor */
	FPixelStreamingHMD(const FAutoRegister&);

	/** Destructor */
	virtual ~FPixelStreamingHMD();

	/** @return	True if the HMD was initialized OK */
	bool IsInitialized() const { return true; }

	void SetTransform(FTransform Transform) { CurHmdTransform = Transform; }
	void SetEyeViews(FTransform Left, FMatrix LeftProj, FTransform Right, FMatrix RightProj, FTransform HMD);

private:
	FVector BasePosition = FVector::ZeroVector;
	FTransform CurHmdTransform;
	FVector LeftEyePosOffset;
	FVector RightEyePosOffset;
	FQuat LeftEyeRotOffset;
	FQuat RightEyeRotOffset;
	float WorldToMeters;
	float InterpupillaryDistance;
	float HFoVRads = FMath::DegreesToRadians(90.0f);
	float VFoVRads = FMath::DegreesToRadians(90.0f);
	float CurLeftEyeProjOffsetX = 0.0f;
	float CurLeftEyeProjOffsetY = 0.0f;
	float CurRightEyeProjOffsetX = 0.0f;
	float CurRightEyeProjOffsetY = 0.0f;
	float TargetAspectRatio = 9.0f/16.0f;
	float NearClip = 10.0f;
	float FarClip = 10000.0f;
	bool bStereoEnabled;
	bool bReceivedTransforms = false;
	double LastResChangeSeconds = 0.0f;
};