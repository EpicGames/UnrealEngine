// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PixelStreamingInput : ModuleRules
	{
		public PixelStreamingInput(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InputDevice",
					"Json",
					"SlateCore",
					"Slate",
					"DeveloperSettings",
					"HeadMountedDisplay",
					"XRBase",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"PixelStreamingHMD",
					"InputCore",
					"Core"
				}
			);
		}
	}
}