// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplicationWrapper.h"
#include "EditorPixelStreamingSettings.h"

namespace UE::PixelStreamingInput
{
	FPixelStreamingApplicationWrapper::FPixelStreamingApplicationWrapper(TSharedPtr<GenericApplication> InWrappedApplication)
		: GenericApplication(MakeShareable(new FCursor()))
		, WrappedApplication(InWrappedApplication)
	{
		// Whether we want to always consider the mouse as attached. This allow
		// us to run Pixel Streaming on a machine which has no physical mouse
		// and just let the browser supply mouse positions.
		const UPixelStreamingSettings* Settings = GetDefault<UPixelStreamingSettings>();
		check(Settings);
		bMouseAlwaysAttached = Settings->bMouseAlwaysAttached;
	}

	void FPixelStreamingApplicationWrapper::SetTargetWindow(TWeakPtr<SWindow> InTargetWindow)
	{
		TargetWindow = InTargetWindow;
	}

	TSharedPtr<FGenericWindow> FPixelStreamingApplicationWrapper::GetWindowUnderCursor()
	{
		TSharedPtr<SWindow> Window = TargetWindow.Pin();
		if (Window.IsValid())
		{
			FVector2D CursorPosition = Cursor->GetPosition();
			FGeometry WindowGeometry = Window->GetWindowGeometryInScreen();

			FVector2D WindowOffset = WindowGeometry.GetAbsolutePosition();
			FVector2D WindowSize = WindowGeometry.GetAbsoluteSize();

			FBox2D WindowRect(WindowOffset, WindowSize);
			if (WindowRect.IsInside(CursorPosition))
			{
				return Window->GetNativeWindow();
			}
		}

		return WrappedApplication->GetWindowUnderCursor();
	}
} // namespace UE::PixelStreamingInput