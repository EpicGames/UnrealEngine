// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericApplication.h"
#include "Widgets/SWindow.h"

namespace UE::PixelStreamingInput
{
	/**
	 * Wrap the GenericApplication layer so we can replace the cursor and override
	 * certain behavior.
	 */
	class FPixelStreamingApplicationWrapper : public GenericApplication
	{
	public:
		FPixelStreamingApplicationWrapper(TSharedPtr<GenericApplication> InWrappedApplication);

		/**
		 * Functions passed directly to the wrapped application.
		 */
		virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) { WrappedApplication->SetMessageHandler(InMessageHandler); }
		virtual void PollGameDeviceState(const float TimeDelta) { WrappedApplication->PollGameDeviceState(TimeDelta); }
		virtual void PumpMessages(const float TimeDelta) { WrappedApplication->PumpMessages(TimeDelta); }
		virtual void ProcessDeferredEvents(const float TimeDelta) { WrappedApplication->ProcessDeferredEvents(TimeDelta); }
		virtual void Tick(const float TimeDelta) { WrappedApplication->Tick(TimeDelta); }
		virtual TSharedRef<FGenericWindow> MakeWindow() { return WrappedApplication->MakeWindow(); }
		virtual void InitializeWindow(const TSharedRef<FGenericWindow>& Window, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FGenericWindow>& InParent, const bool bShowImmediately) { WrappedApplication->InitializeWindow(Window, InDefinition, InParent, bShowImmediately); }
		virtual void SetCapture(const TSharedPtr<FGenericWindow>& InWindow) { WrappedApplication->SetCapture(InWindow); }
		virtual void* GetCapture(void) const { return WrappedApplication->GetCapture(); }
		virtual FModifierKeysState GetModifierKeys() const { return WrappedApplication->GetModifierKeys(); }
		virtual void SetHighPrecisionMouseMode(const bool Enable, const TSharedPtr<FGenericWindow>& InWindow) { WrappedApplication->SetHighPrecisionMouseMode(Enable, InWindow); };
		virtual bool IsUsingHighPrecisionMouseMode() const { return WrappedApplication->IsUsingHighPrecisionMouseMode(); }
		virtual bool IsUsingTrackpad() const { return WrappedApplication->IsUsingTrackpad(); }
		virtual bool IsGamepadAttached() const { return WrappedApplication->IsGamepadAttached(); }
		virtual void RegisterConsoleCommandListener(const FOnConsoleCommandListener& InListener) { WrappedApplication->RegisterConsoleCommandListener(InListener); }
		virtual void AddPendingConsoleCommand(const FString& InCommand) { WrappedApplication->AddPendingConsoleCommand(InCommand); }
		virtual FPlatformRect GetWorkArea(const FPlatformRect& CurrentWindow) const { return WrappedApplication->GetWorkArea(CurrentWindow); }
		virtual bool TryCalculatePopupWindowPosition(const FPlatformRect& InAnchor, const FVector2D& InSize, const FVector2D& ProposedPlacement, const EPopUpOrientation::Type Orientation, /*OUT*/ FVector2D* const CalculatedPopUpPosition) const { return WrappedApplication->TryCalculatePopupWindowPosition(InAnchor, InSize, ProposedPlacement, Orientation, CalculatedPopUpPosition); }
		virtual void GetInitialDisplayMetrics(FDisplayMetrics& OutDisplayMetrics) const { WrappedApplication->GetInitialDisplayMetrics(OutDisplayMetrics); }
		virtual EWindowTitleAlignment::Type GetWindowTitleAlignment() const { return WrappedApplication->GetWindowTitleAlignment(); }
		virtual EWindowTransparency GetWindowTransparencySupport() const { return WrappedApplication->GetWindowTransparencySupport(); }
		virtual void DestroyApplication() { WrappedApplication->DestroyApplication(); }
		virtual IInputInterface* GetInputInterface() { return WrappedApplication->GetInputInterface(); }
		virtual ITextInputMethodSystem* GetTextInputMethodSystem() { return WrappedApplication->GetTextInputMethodSystem(); }
		virtual void SendAnalytics(IAnalyticsProvider* Provider) { WrappedApplication->SendAnalytics(Provider); }
		virtual bool SupportsSystemHelp() const { return WrappedApplication->SupportsSystemHelp(); }
		virtual void ShowSystemHelp() { WrappedApplication->ShowSystemHelp(); }
		virtual bool ApplicationLicenseValid(FPlatformUserId PlatformUser = PLATFORMUSERID_NONE) { return WrappedApplication->ApplicationLicenseValid(PlatformUser); }

		/**
		 * Functions with overridden behavior.
		 */
		virtual bool IsMouseAttached() const { return bMouseAlwaysAttached ? true : WrappedApplication->IsMouseAttached(); }
		virtual bool IsCursorDirectlyOverSlateWindow() const { return true; }
		virtual TSharedPtr<FGenericWindow> GetWindowUnderCursor() override;

		/**
		 * Custom functions
		 */
		virtual void SetTargetWindow(TWeakPtr<SWindow> InTargetWindow);

		TSharedPtr<GenericApplication> WrappedApplication;
		TWeakPtr<SWindow> TargetWindow;
		bool bMouseAlwaysAttached;
	};

	/**
	 * When reading input from a browser then the cursor position will be sent
	 * across with mouse events. We want to use this position and avoid getting the
	 * cursor position from the operating system. This is not relevant to touch
	 * events.
	 */
	class FCursor : public ICursor
	{
	public:
		FCursor() {}
		virtual ~FCursor() = default;
		virtual FVector2D GetPosition() const override { return Position; }
		virtual void SetPosition(const int32 X, const int32 Y) override { Position = FVector2D(X, Y); };
		virtual void SetType(const EMouseCursor::Type InNewCursor) override{};
		virtual EMouseCursor::Type GetType() const override { return EMouseCursor::Type::Default; };
		virtual void GetSize(int32& Width, int32& Height) const override{};
		virtual void Show(bool bShow) override{};
		virtual void Lock(const RECT* const Bounds) override{};
		virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override{};

	private:
		/** The cursor position sent across with mouse events. */
		FVector2D Position;
	};
} // namespace UE::PixelStreamingInput