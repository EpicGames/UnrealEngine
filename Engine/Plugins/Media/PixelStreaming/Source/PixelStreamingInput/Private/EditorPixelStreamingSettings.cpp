// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorPixelStreamingSettings.h"

UPixelStreamingSettings::UPixelStreamingSettings(const FObjectInitializer& ObjectInitlaizer)
	: Super(ObjectInitlaizer)
{
}

FName UPixelStreamingSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UPixelStreamingSettings::GetSectionText() const
{
	return NSLOCTEXT("PixelStreamingPlugin", "PixelStreamingSettingsSection", "PixelStreaming");
}
#endif

#if WITH_EDITOR
void UPixelStreamingSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif