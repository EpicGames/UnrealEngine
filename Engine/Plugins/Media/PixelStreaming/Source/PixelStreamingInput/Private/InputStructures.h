// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"

namespace UE::PixelStreamingInput
{
    struct TPayloadNoParam
    {
        TPayloadNoParam()
        {
        }

        TPayloadNoParam(FArchive& Ar)
        {
        }

        TArray<uint8> AsData()
        {
            FBufferArchive MemAr;
            return MoveTemp(MemAr);
        }
    };

    template <typename ParamOneType>
    struct TPayloadOneParam
    {
        ParamOneType	Param1;

        TPayloadOneParam(FArchive& Ar)
        {
            Param1 = ParamOneType();
            Ar << Param1;
        }

        TPayloadOneParam(ParamOneType InParam1)
        {
            Param1 = InParam1;
        }

        TArray<uint8> AsData()
        {
            FBufferArchive MemAr;
            MemAr << Param1;
            return MoveTemp(MemAr);
        }
    };

    template <typename ParamOneType, typename ParamTwoType>
    struct TPayloadTwoParam
    {
        ParamOneType	Param1;
        ParamTwoType	Param2;

        TPayloadTwoParam(FArchive& Ar)
        {
            Param1 = ParamOneType();
            Param2 = ParamTwoType();
            Ar << Param1;
            Ar << Param2;
        }

        TPayloadTwoParam(ParamOneType InParam1, ParamTwoType InParam2)
        {
            Param1 = InParam1;
            Param2 = InParam2;
        }

        TArray<uint8> AsData()
        {
            FBufferArchive MemAr;
            MemAr << Param1 << Param2;
            return MoveTemp(MemAr);
        }
    };

    template <typename ParamOneType, typename ParamTwoType, typename ParamThreeType>
    struct TPayloadThreeParam
    {
        ParamOneType	Param1;
        ParamTwoType	Param2;
        ParamThreeType	Param3;

        TPayloadThreeParam(FArchive& Ar)
        {
            Param1 = ParamOneType();
            Param2 = ParamTwoType();
            Param3 = ParamThreeType();
            Ar << Param1;
            Ar << Param2;
            Ar << Param3;
        }

        TPayloadThreeParam(ParamOneType InParam1, ParamTwoType InParam2, ParamThreeType InParam3)
        {
            Param1 = InParam1;
            Param2 = InParam2;
            Param3 = InParam3;
        }

        TArray<uint8> AsData()
        {
            FBufferArchive MemAr;
            MemAr << Param1 << Param2 << Param3;
            return MoveTemp(MemAr);
        }
    };

    template <typename ParamOneType, typename ParamTwoType, typename ParamThreeType, typename ParamFourType>
    struct TPayloadFourParam
    {
        ParamOneType	Param1;
        ParamTwoType	Param2;
        ParamThreeType	Param3;
        ParamFourType	Param4;

        TPayloadFourParam(FArchive& Ar)
        {
            Param1 = ParamOneType();
            Param2 = ParamTwoType();
            Param3 = ParamThreeType();
            Param4 = ParamFourType();
            Ar << Param1;
            Ar << Param2;
            Ar << Param3;
            Ar << Param4;
        }

        TPayloadFourParam(ParamOneType InParam1, ParamTwoType InParam2, ParamThreeType InParam3, ParamFourType InParam4)
        {
            Param1 = InParam1;
            Param2 = InParam2;
            Param3 = InParam3;
            Param4 = InParam4;
        }

        TArray<uint8> AsData()
        {
            FBufferArchive MemAr;
            MemAr << Param1 << Param2 << Param3 << Param4;
            return MoveTemp(MemAr);
        }
    };

    template <typename ParamOneType, typename ParamTwoType, typename ParamThreeType, typename ParamFourType, typename ParamFiveType>
    struct TPayloadFiveParam
    {
        ParamOneType	Param1;
        ParamTwoType	Param2;
        ParamThreeType	Param3;
        ParamFourType	Param4;
        ParamFiveType	Param5;

        TPayloadFiveParam(FArchive& Ar)
        {
            Param1 = ParamOneType();
            Param2 = ParamTwoType();
            Param3 = ParamThreeType();
            Param4 = ParamFourType();
            Param5 = ParamFiveType();
            Ar << Param1;
            Ar << Param2;
            Ar << Param3;
            Ar << Param4;
            Ar << Param5;
        }

        TPayloadFiveParam(ParamOneType InParam1, ParamTwoType InParam2, ParamThreeType InParam3, ParamFourType InParam4, ParamFiveType InParam5)
        {
            Param1 = InParam1;
            Param2 = InParam2;
            Param3 = InParam3;
            Param4 = InParam4;
            Param5 = InParam5;
        }

        TArray<uint8> AsData()
        {
            FBufferArchive MemAr;
            MemAr << Param1 << Param2 << Param3 << Param4 << Param5;
            return MoveTemp(MemAr);
        }
    };


    /**
	* A touch is a specific finger placed on the canvas as a specific position.
    */
    struct FTouch
    {
    	uint16 PosX;	  // X position of finger.
		uint16 PosY;	  // Y position of finger.
		uint8 TouchIndex; // Index of finger for tracking multi-touch events.
		uint8 Force;	  // Amount of pressure being applied by the finger.
		uint8 Valid;	  // 1 if the touch was within bounds.
    };
} // namespace UE::PixelStreamingInput
