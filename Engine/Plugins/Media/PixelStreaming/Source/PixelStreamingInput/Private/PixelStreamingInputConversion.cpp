// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputConversion.h"

TMap<TTuple<EPixelStreamingXRSystem, EControllerHand, uint8, EPixelStreamingInputAction>, FKey> FPixelStreamingInputConverter::XRInputToFKey;
TMap<TTuple<uint8, EPixelStreamingInputAction>, FKey> FPixelStreamingInputConverter::GamepadInputToFKey;
