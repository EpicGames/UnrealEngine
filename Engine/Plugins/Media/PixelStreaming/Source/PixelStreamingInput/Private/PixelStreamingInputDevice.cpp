// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputDevice.h"
#include "Framework/Application/SlateApplication.h"

TSharedPtr<UE::PixelStreamingInput::FPixelStreamingInputDevice> UE::PixelStreamingInput::FPixelStreamingInputDevice::InputDevice;

namespace UE::PixelStreamingInput
{
	TSharedPtr<FPixelStreamingInputDevice> FPixelStreamingInputDevice::GetInputDevice()
	{
		if (InputDevice)
		{
			return InputDevice;
		}

		TSharedPtr<FPixelStreamingInputDevice> Device = TSharedPtr<FPixelStreamingInputDevice>(new FPixelStreamingInputDevice());
		if (Device)
		{
			InputDevice = Device;
		}
		return InputDevice;
	}

	FPixelStreamingInputDevice::FPixelStreamingInputDevice()
	{
		// This is imperative for editor streaming as when a modal is open or we've hit a BP breakpoint, the engine tick loop will not run, so instead we rely on this delegate to tick for us
		FSlateApplication::Get().OnPreTick().AddRaw(this, &FPixelStreamingInputDevice::Tick);
	}

	void FPixelStreamingInputDevice::AddInputHandler(TSharedPtr<IPixelStreamingInputHandler> InputHandler)
	{
		InputHandlers.Add(InputHandler);
	}

	void FPixelStreamingInputDevice::Tick(float DeltaTime)
	{
		for (TWeakPtr<IPixelStreamingInputHandler> WeakHandler : InputHandlers)
		{
			if (TSharedPtr<IPixelStreamingInputHandler> Handler = WeakHandler.Pin())
			{
				Handler->Tick(DeltaTime);
			}
		}
	}

	void FPixelStreamingInputDevice::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler)
	{
		for (TWeakPtr<IPixelStreamingInputHandler> WeakHandler : InputHandlers)
		{
			if (TSharedPtr<IPixelStreamingInputHandler> Handler = WeakHandler.Pin())
			{
				Handler->SetMessageHandler(InTargetHandler);
			}
		}
	}

	bool FPixelStreamingInputDevice::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		bool bRetVal = true;
		for (TWeakPtr<IPixelStreamingInputHandler> WeakHandler : InputHandlers)
		{
			if (TSharedPtr<IPixelStreamingInputHandler> Handler = WeakHandler.Pin())
			{
				bRetVal &= Handler->Exec(InWorld, Cmd, Ar);
			}
		}
		return bRetVal;
	}

	void FPixelStreamingInputDevice::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
	{
		for (TWeakPtr<IPixelStreamingInputHandler> WeakHandler : InputHandlers)
		{
			if (TSharedPtr<IPixelStreamingInputHandler> Handler = WeakHandler.Pin())
			{
				Handler->SetChannelValue(ControllerId, ChannelType, Value);
			}
		}
	}

	void FPixelStreamingInputDevice::SetChannelValues(int32 ControllerId, const FForceFeedbackValues& Values)
	{
		for (TWeakPtr<IPixelStreamingInputHandler> WeakHandler : InputHandlers)
		{
			if (TSharedPtr<IPixelStreamingInputHandler> Handler = WeakHandler.Pin())
			{
				Handler->SetChannelValues(ControllerId, Values);
			}
		}
	}

	uint8 FPixelStreamingInputDevice::OnControllerConnected()
	{
		uint8 NextControllerId = 0;
		while (ConnectedControllers.Contains(NextControllerId))
		{
			NextControllerId++;
		}

		ConnectedControllers.Add(NextControllerId);
		return NextControllerId;
	}

	void FPixelStreamingInputDevice::OnControllerDisconnected(uint8 DeleteControllerId)
	{
		ConnectedControllers.Remove(DeleteControllerId);
	}
} // namespace UE::PixelStreamingInput
