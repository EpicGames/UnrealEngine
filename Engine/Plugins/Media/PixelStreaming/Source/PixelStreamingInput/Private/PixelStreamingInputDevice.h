// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDevice.h"
#include "IPixelStreamingInputHandler.h"

namespace UE::PixelStreamingInput
{
	/**
	 * @brief The input device used to interface the multiple streamers and the single input device created by the OS
	 *
	 */
	class FPixelStreamingInputDevice : public IInputDevice
	{
	public:
		void Tick(float DeltaTime) override;
		/** Poll for controller state and send events if needed */
		virtual void SendControllerEvents() override{};

		/** Set which MessageHandler will route input  */
		virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler) override;

		/** Exec handler to allow console commands to be passed through for debugging */
		virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
		virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
		virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues& Values) override;

		void AddInputHandler(TSharedPtr<IPixelStreamingInputHandler> InputHandler);
		uint8 OnControllerConnected();
		void OnControllerDisconnected(uint8 DeleteControllerId);

		static TSharedPtr<FPixelStreamingInputDevice> GetInputDevice();

	private:
		FPixelStreamingInputDevice();

		/**
		 * A singleton pointer to the input device. We only want a single input device that has multiple input handlers.
		 * The reason for a single input device is that only one is created by the application, so make sure we always use that one
		 *
		 */
		static TSharedPtr<FPixelStreamingInputDevice> InputDevice;

		/**
		 * The array of input handlers. Each input handler belongs to a single streamer
		 *
		 * NOTE: We store each input handler as a weakptr as we don't want to be the reason
		 * the handler doesn't get deleted. Each handler should be tied to a Streamer's lifecycle
		 */
		TArray<TWeakPtr<IPixelStreamingInputHandler>> InputHandlers;

		/**
		 * The array of connected controllers. As each device can have multiple controllers, we want to make sure that each controller of each device is unique
		 * As such, a simple incrementer approach is not applicable and we must instead keep track of all the connected controllers
		 *
		 * NOTE: We store each input handler as a weakptr as we don't want to be the reason
		 * the handler doesn't get deleted. Each handler should be tied to a Streamer's lifecycle
		 */
		TArray<uint8> ConnectedControllers;
	};
} // namespace UE::PixelStreamingInput