// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputHandler.h"

#include "InputStructures.h"
#include "PixelStreamingInputProtocol.h"
#include "JavaScriptKeyCodes.inl"
#include "Settings.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SViewport.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Utils.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateUser.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Misc/CoreMiscDefines.h"
#include "Input/HittestGrid.h"
#include "PixelStreamingHMD.h"
#include "IPixelStreamingHMDModule.h"
#include "PixelStreamingInputEnums.h"
#include "PixelStreamingInputConversion.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "PixelStreamingInputDevice.h"

#if WITH_EDITOR
#include "UObject/UObjectGlobals.h"
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreamingInputHandler, Log, VeryVerbose);
DEFINE_LOG_CATEGORY(LogPixelStreamingInputHandler);

// TODO: Gesture recognition is moving to the browser, so add handlers for the gesture events.
// The gestures supported will be swipe, pinch,

namespace UE::PixelStreamingInput
{
	typedef EPixelStreamingInputAction Action;
	typedef EPixelStreamingXRSystem XRSystem;

	FPixelStreamingInputHandler::FPixelStreamingInputHandler(TSharedPtr<FPixelStreamingApplicationWrapper> InApplicationWrapper, const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler)
		: TargetViewport(nullptr)
		, NumActiveTouches(0)
		, bIsMouseActive(false)
		, MessageHandler(InTargetHandler)
		, PixelStreamerApplicationWrapper(InApplicationWrapper)
	{
		// Register this input handler as an IMotionController. The module handles the registering as an IInputDevice
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);

		RegisterMessageHandler("KeyPress", [this](FString SourceId, FMemoryReader Ar) { HandleOnKeyChar(Ar); });
		RegisterMessageHandler("KeyUp", [this](FString SourceId, FMemoryReader Ar) { HandleOnKeyUp(Ar); });
		RegisterMessageHandler("KeyDown", [this](FString SourceId, FMemoryReader Ar) { HandleOnKeyDown(Ar); });

		RegisterMessageHandler("TouchStart", [this](FString SourceId, FMemoryReader Ar) { HandleOnTouchStarted(Ar); });
		RegisterMessageHandler("TouchMove", [this](FString SourceId, FMemoryReader Ar) { HandleOnTouchMoved(Ar); });
		RegisterMessageHandler("TouchEnd", [this](FString SourceId, FMemoryReader Ar) { HandleOnTouchEnded(Ar); });

		RegisterMessageHandler("GamepadConnected", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerConnected(Ar); });
		RegisterMessageHandler("GamepadAnalog", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerAnalog(Ar); });
		RegisterMessageHandler("GamepadButtonPressed", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerButtonPressed(Ar); });
		RegisterMessageHandler("GamepadButtonReleased", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerButtonReleased(Ar); });
		RegisterMessageHandler("GamepadDisconnected", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerDisconnected(Ar); });

		RegisterMessageHandler("MouseEnter", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseEnter(Ar); });
		RegisterMessageHandler("MouseLeave", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseLeave(Ar); });
		RegisterMessageHandler("MouseUp", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseUp(Ar); });
		RegisterMessageHandler("MouseDown", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseDown(Ar); });
		RegisterMessageHandler("MouseMove", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseMove(Ar); });
		RegisterMessageHandler("MouseWheel", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseWheel(Ar); });
		RegisterMessageHandler("MouseDouble", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseDoubleClick(Ar); });

		RegisterMessageHandler("XREyeViews", [this](FString SourceId, FMemoryReader Ar) { HandleOnXREyeViews(Ar); });
		RegisterMessageHandler("XRHMDTransform", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRHMDTransform(Ar); });
		RegisterMessageHandler("XRControllerTransform", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRControllerTransform(Ar); });
		RegisterMessageHandler("XRButtonPressed", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonPressed(Ar); });
		RegisterMessageHandler("XRButtonTouched", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonTouched(Ar); });
		RegisterMessageHandler("XRButtonTouchReleased", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonTouchReleased(Ar); });
		RegisterMessageHandler("XRButtonReleased", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonReleased(Ar); });
		RegisterMessageHandler("XRAnalog", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRAnalog(Ar); });
		RegisterMessageHandler("XRSystem", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRSystem(Ar); });

		RegisterMessageHandler("Command", [this](FString SourceId, FMemoryReader Ar) { HandleOnCommand(SourceId, Ar); });
		RegisterMessageHandler("UIInteraction", [this](FString SourceId, FMemoryReader Ar) { HandleUIInteraction(Ar); });
		RegisterMessageHandler("TextboxEntry", [this](FString SourceId, FMemoryReader Ar) { HandleOnTextboxEntry(Ar); });

		// Populate map
		// Button indices found in: https://github.com/immersive-web/webxr-input-profiles/tree/master/packages/registry/profiles
		// HTC Vive - Left Hand
		// Buttons
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Left, 0, Action::Click), EKeys::Vive_Left_Trigger_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Left, 0, Action::Axis), EKeys::Vive_Left_Trigger_Axis);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Left, 1, Action::Click), EKeys::Vive_Left_Grip_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Left, 2, Action::Click), EKeys::Vive_Left_Trackpad_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Left, 2, Action::Touch), EKeys::Vive_Left_Trackpad_Touch);
		// Axes
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Left, 0, Action::X), EKeys::Vive_Left_Trackpad_X);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Left, 1, Action::Y), EKeys::Vive_Left_Trackpad_Y);
		// HTC Vive - Right Hand
		// Buttons
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Right, 0, Action::Click), EKeys::Vive_Right_Trigger_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Right, 0, Action::Axis), EKeys::Vive_Right_Trigger_Axis);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Right, 1, Action::Click), EKeys::Vive_Right_Grip_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Right, 2, Action::Click), EKeys::Vive_Right_Trackpad_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Right, 2, Action::Touch), EKeys::Vive_Right_Trackpad_Touch);
		// Axes
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Right, 0, Action::X), EKeys::Vive_Right_Trackpad_X);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::HTCVive, EControllerHand::Right, 1, Action::Y), EKeys::Vive_Right_Trackpad_Y);

		// Quest - Left Hand
		// Buttons
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 0, Action::Click), EKeys::OculusTouch_Left_Trigger_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 0, Action::Axis), EKeys::OculusTouch_Left_Trigger_Axis);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 0, Action::Touch), EKeys::OculusTouch_Left_Trigger_Touch);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 1, Action::Click), EKeys::OculusTouch_Left_Grip_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 1, Action::Axis), EKeys::OculusTouch_Left_Grip_Axis);
		// Index 1 (grip) touch not supported in UE
		// Index 2 not supported by WebXR
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 3, Action::Click), EKeys::OculusTouch_Left_Thumbstick_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 3, Action::Touch), EKeys::OculusTouch_Left_Thumbstick_Touch);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 4, Action::Click), EKeys::OculusTouch_Left_X_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 4, Action::Touch), EKeys::OculusTouch_Left_X_Touch);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 5, Action::Click), EKeys::OculusTouch_Left_Y_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 5, Action::Touch), EKeys::OculusTouch_Left_Y_Touch);
		// Index 6 (thumbrest) not supported in UE

		// Axes
		// Indices 0 and 1 not supported in WebXR
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 2, Action::X), EKeys::OculusTouch_Left_Thumbstick_X);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Left, 3, Action::Y), EKeys::OculusTouch_Left_Thumbstick_Y);

		// Quest - Right Hand
		// Buttons
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 0, Action::Click), EKeys::OculusTouch_Right_Trigger_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 0, Action::Axis), EKeys::OculusTouch_Right_Trigger_Axis);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 0, Action::Touch), EKeys::OculusTouch_Right_Trigger_Touch);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 1, Action::Click), EKeys::OculusTouch_Right_Grip_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 1, Action::Axis), EKeys::OculusTouch_Right_Grip_Axis);
		// Index 1 (grip) touch not supported in UE
		// Index 2 not supported by WebXR
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 3, Action::Click), EKeys::OculusTouch_Right_Thumbstick_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 3, Action::Touch), EKeys::OculusTouch_Right_Thumbstick_Touch);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 4, Action::Click), EKeys::OculusTouch_Right_A_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 4, Action::Touch), EKeys::OculusTouch_Right_A_Touch);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 5, Action::Click), EKeys::OculusTouch_Right_B_Click);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 5, Action::Touch), EKeys::OculusTouch_Right_B_Touch);
		// Index 6 (thumbrest) not supported in UE

		// Axes
		// Indices 0 and 1 not supported in WebXR
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 2, Action::X), EKeys::OculusTouch_Right_Thumbstick_X);
		FPixelStreamingInputConverter::XRInputToFKey.Add(MakeTuple(XRSystem::Quest, EControllerHand::Right, 3, Action::Y), EKeys::OculusTouch_Right_Thumbstick_Y);

		// Gamepad Axes
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(1, Action::Axis), EKeys::Gamepad_LeftX);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(2, Action::Axis), EKeys::Gamepad_LeftY);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(3, Action::Axis), EKeys::Gamepad_RightX);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(4, Action::Axis), EKeys::Gamepad_RightY);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(5, Action::Axis), EKeys::Gamepad_LeftTriggerAxis);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(6, Action::Axis), EKeys::Gamepad_RightTriggerAxis);
		// Gamepad Buttons
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(0, Action::Click), EKeys::Gamepad_FaceButton_Bottom);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(1, Action::Click), EKeys::Gamepad_FaceButton_Right);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(2, Action::Click), EKeys::Gamepad_FaceButton_Left);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(3, Action::Click), EKeys::Gamepad_FaceButton_Top);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(4, Action::Click), EKeys::Gamepad_LeftShoulder);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(5, Action::Click), EKeys::Gamepad_RightShoulder);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(6, Action::Click), EKeys::Gamepad_LeftTrigger);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(7, Action::Click), EKeys::Gamepad_RightTrigger);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(8, Action::Click), EKeys::Gamepad_Special_Left);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(9, Action::Click), EKeys::Gamepad_Special_Right);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(10, Action::Click), EKeys::Gamepad_LeftThumbstick);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(11, Action::Click), EKeys::Gamepad_RightThumbstick);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(12, Action::Click), EKeys::Gamepad_DPad_Up);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(13, Action::Click), EKeys::Gamepad_DPad_Down);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(14, Action::Click), EKeys::Gamepad_DPad_Left);
		FPixelStreamingInputConverter::GamepadInputToFKey.Add(MakeTuple(15, Action::Click), EKeys::Gamepad_DPad_Right);

		PopulateDefaultCommandHandlers();
	}

	FPixelStreamingInputHandler::~FPixelStreamingInputHandler()
	{
	}

	void FPixelStreamingInputHandler::RegisterMessageHandler(const FString& MessageType, const MessageHandlerFn& Handler)
	{
		DispatchTable.Add(FPixelStreamingInputProtocol::ToStreamerProtocol.Find(MessageType)->GetID(), Handler);
	}

	IPixelStreamingInputHandler::MessageHandlerFn FPixelStreamingInputHandler::FindMessageHandler(const FString& MessageType)
	{
		return DispatchTable.FindRef(FPixelStreamingInputProtocol::ToStreamerProtocol.Find(MessageType)->GetID());
	}

	FName FPixelStreamingInputHandler::GetMotionControllerDeviceTypeName() const
	{
		return FName(TEXT("PixelStreamingXRController"));
	}

	bool FPixelStreamingInputHandler::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const
	{
		if (FPixelStreamingHMD* HMD = IPixelStreamingHMDModule::Get().GetPixelStreamingHMD(); (HMD == nullptr || ControllerIndex == INDEX_NONE))
		{
			return false;
		}

		EControllerHand DeviceHand;
		if (GetHandEnumForSourceName(MotionSource, DeviceHand))
		{
			FPixelStreamingXRController Controller = XRControllers.FindRef(DeviceHand);
			OutOrientation = Controller.Transform.Rotator();
			OutPosition = Controller.Transform.GetTranslation();
			return true;
		}
		return false;
	}

	ETrackingStatus FPixelStreamingInputHandler::GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const
	{
		EControllerHand DeviceHand;
		if (GetHandEnumForSourceName(MotionSource, DeviceHand))
		{
			const FPixelStreamingXRController* Controller = XRControllers.Find(DeviceHand);
			return (Controller != nullptr) ? ETrackingStatus::Tracked : ETrackingStatus::NotTracked;
		}
		return ETrackingStatus::NotTracked;
	}

	void FPixelStreamingInputHandler::EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const
	{
		SourcesOut.Add(FName(TEXT("AnyHand")));
		SourcesOut.Add(FName(TEXT("Left")));
		SourcesOut.Add(FName(TEXT("Right")));
		SourcesOut.Add(FName(TEXT("LeftGrip")));
		SourcesOut.Add(FName(TEXT("RightGrip")));
		SourcesOut.Add(FName(TEXT("LeftAim")));
		SourcesOut.Add(FName(TEXT("RightAim")));
	}

	void FPixelStreamingInputHandler::Tick(const float InDeltaTime)
	{
#if WITH_EDITOR
		/* No routing input while saving ... this is relevant for auto-save and can cause an incredibly rare crash...
		 * 
		 * The gist is that the auto-save system calls FSlateApplication::Tick(), which executes its OnPreTick() containing
		 * our FPixelStreamingInputHandler::Tick. Routing any input executes Slate delegates. Again, the gist is that
		 * the delegates can do anything including calling StaticConstructObject(), which will crash the editor
		 * ("Illegal call to StaticConstructObject() while serializing object data!").
		 * An example of a StaticConstructObject call is a UMG widget calling CreateWidget in response to a button's OnClick (which we routed!).
		 *
		 * If you're curious why our Tick gets called by auto-save:
		 * The auto save starts in FPackageAutoSaver::AttemptAutoSave, which calls FEditorFileUtils::AutosaveMapEx.
		 * This causes the world package to be saved (UEditorEngine::SavePackage) with a FSlowTask.
		 * The slow task calls FFeedbackContextEditor::ProgressReported... which ticks slate so the progres bar modal window updates.
		 * Consult with FPixelStreamingInputDevice::FPixelStreamingInputDevice, which explicitly wants to tick when a modal window is open.
		 *
		 * TLDR: if we're auto-saving, we'll postbone routing input until the auto save is done.
		 */
		if (UE::IsSavingPackage())
		{
			return;
		}
#endif

		TouchIndicesProcessedThisFrame.Reset();

		FMessage Message;
		while (Messages.Dequeue(Message))
		{
			FMemoryReader Ar(Message.Data);
			(*Message.Handler)(Message.SourceId, Ar);
		}

		ProcessLatestAnalogInputFromThisTick();
		BroadcastActiveTouchMoveEvents();
	}

	void FPixelStreamingInputHandler::OnMessage(FString SourceId, TArray<uint8> Buffer)
	{
		uint8 MessageType = Buffer[0];
		// Remove the message type. The remaining data in the buffer is now purely
		// the message data
		Buffer.RemoveAt(0);

		TFunction<void(FString, FMemoryReader)>* Handler = DispatchTable.Find(MessageType);
		if (Handler != nullptr)
		{
			FMessage Message = {
				SourceId, // Who sent this message
				Handler, // The function to call
				Buffer	 // The message data
			};
			Messages.Enqueue(Message);
		}
		else
		{
			UE_LOGF(LogPixelStreamingInputHandler, Warning, "No handler registered for message with id %d", MessageType);
		}
	}

	void FPixelStreamingInputHandler::SetTargetWindow(TWeakPtr<SWindow> InWindow)
	{
		TargetWindow = InWindow;
		PixelStreamerApplicationWrapper->SetTargetWindow(InWindow);
	}

	TWeakPtr<SWindow> FPixelStreamingInputHandler::GetTargetWindow()
	{
		return TargetWindow;
	}

	void FPixelStreamingInputHandler::SetTargetScreenRect(TWeakPtr<FIntRect> InScreenRect)
	{
		TargetScreenRect = InScreenRect;
	}

	TWeakPtr<FIntRect> FPixelStreamingInputHandler::GetTargetScreenRect()
	{
		return TargetScreenRect;
	}

	void FPixelStreamingInputHandler::SetTargetViewport(TWeakPtr<SViewport> InViewport)
	{
		TargetViewport = InViewport;
	}

	TWeakPtr<SViewport> FPixelStreamingInputHandler::GetTargetViewport()
	{
		return TargetViewport;
	}

	void FPixelStreamingInputHandler::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler)
	{
		MessageHandler = InTargetHandler;
	}

	bool FPixelStreamingInputHandler::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		return GEngine->Exec(InWorld, Cmd, Ar);
	}

	void FPixelStreamingInputHandler::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
	{
		// TODO: Implement FFB
	}

	void FPixelStreamingInputHandler::SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values)
	{
		// TODO: Implement FFB
	}

	void FPixelStreamingInputHandler::HandleOnKeyChar(FMemoryReader Ar)
	{
		TPayloadOneParam<TCHAR> Payload(Ar);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "KEY_PRESSED: Character = '%c'", Payload.Param1);
		// A key char event is never repeated, so set it to false. It's value
		// ultimately doesn't matter as this paramater isn't used later
		MessageHandler->OnKeyChar(Payload.Param1, false);
	}

	void FPixelStreamingInputHandler::HandleOnKeyDown(FMemoryReader Ar)
	{
		TPayloadTwoParam<uint8, uint8> Payload(Ar);

		bool bIsRepeat = Payload.Param2 != 0;
		const FKey* AgnosticKey = JavaScriptKeyCodeToFKey[Payload.Param1];
		if (FilterKey(*AgnosticKey))
		{
			const uint32* KeyPtr;
			const uint32* CharacterPtr;
			FInputKeyManager::Get().GetCodesFromKey(*AgnosticKey, KeyPtr, CharacterPtr);
			uint32 Key = KeyPtr ? *KeyPtr : 0;
			uint32 Character = CharacterPtr ? *CharacterPtr : 0;

			UE_LOGF(LogPixelStreamingInputHandler, Verbose, "KEY_DOWN: Key = %d; Character = %d; IsRepeat = %ls", Key, Character, bIsRepeat ? TEXT("True") : TEXT("False"));
			MessageHandler->OnKeyDown((int32)Key, (int32)Character, bIsRepeat);
		}
	}

	void FPixelStreamingInputHandler::HandleOnKeyUp(FMemoryReader Ar)
	{
		TPayloadOneParam<uint8> Payload(Ar);
		const FKey* AgnosticKey = JavaScriptKeyCodeToFKey[Payload.Param1];
		if (FilterKey(*AgnosticKey))
		{
			const uint32* KeyPtr;
			const uint32* CharacterPtr;
			FInputKeyManager::Get().GetCodesFromKey(*AgnosticKey, KeyPtr, CharacterPtr);
			uint32 Key = KeyPtr ? *KeyPtr : 0;
			uint32 Character = CharacterPtr ? *CharacterPtr : 0;

			UE_LOGF(LogPixelStreamingInputHandler, Verbose, "KEY_UP: Key = %d; Character = %d", Key, Character);
			MessageHandler->OnKeyUp((int32)Key, (int32)Character, false);
		}
	}

	void FPixelStreamingInputHandler::HandleOnTouchStarted(FMemoryReader Ar)
	{
		TPayloadOneParam<uint8> Payload(Ar);

		uint8 NumTouches = Payload.Param1;
		for (uint8 TouchIdx = 0; TouchIdx < NumTouches; TouchIdx++)
		{
			//                 PosX    PoxY    IDX   Force  Valid
			TPayloadFiveParam<uint16, uint16, uint8, uint8, uint8> Touch(Ar);
			// If Touch is valid
			if (Touch.Param5 != 0)
			{
				//                                                                           convert range from 0,65536 -> 0,1
				FVector2D TouchLocation = ConvertFromNormalizedScreenLocation(FVector2D(Touch.Param1 / uint16_MAX, Touch.Param2 / uint16_MAX));
				const int32 TouchIndex = Touch.Param3;
				const float TouchForce = Touch.Param4 / 255.0f;
				UE_LOGF(LogPixelStreamingInputHandler, Verbose, "TOUCH_START: TouchIndex = %d; Pos = (%d, %d); CursorPos = (%d, %d); Force = %.3f", TouchIndex, Touch.Param1, Touch.Param2, static_cast<int>(TouchLocation.X), static_cast<int>(TouchLocation.Y), TouchForce);

				if (InputType == EPixelStreamingInputType::RouteToWidget)
				{
					// TouchLocation = TouchLocation - TargetViewport.Pin()->GetCachedGeometry().GetAbsolutePosition();
					FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchLocation);

					if (WidgetPath.IsValid())
					{
						FScopedSwitchWorldHack SwitchWorld(WidgetPath);
						FPointerEvent PointerEvent(0, TouchIndex, TouchLocation, TouchLocation, TouchForce, true);
						FSlateApplication::Get().RoutePointerDownEvent(WidgetPath, PointerEvent);
					}
				}
				else if (InputType == EPixelStreamingInputType::RouteToWindow)
				{
					if (NumActiveTouches == 0 && !bIsMouseActive)
					{
						FSlateApplication::Get().OnCursorSet();
						// Make sure the application is active.
						FSlateApplication::Get().ProcessApplicationActivationEvent(true);

						FVector2D OldCursorLocation = PixelStreamerApplicationWrapper->WrappedApplication->Cursor->GetPosition();
						PixelStreamerApplicationWrapper->Cursor->SetPosition(OldCursorLocation.X, OldCursorLocation.Y);
						FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper);
					}

					// We must update the user cursor position explicitly before updating the application cursor position
					// as if there's a delta between them, when the touch event is started it will trigger a move
					// resulting in a large 'drag' across the screen
					TSharedPtr<FSlateUser> User = FSlateApplication::Get().GetCursorUser();
					User->SetCursorPosition(TouchLocation);
					PixelStreamerApplicationWrapper->Cursor->SetPosition(TouchLocation.X, TouchLocation.Y);
					PixelStreamerApplicationWrapper->WrappedApplication->Cursor->SetPosition(TouchLocation.X, TouchLocation.Y);

					MessageHandler->OnTouchStarted(PixelStreamerApplicationWrapper->GetWindowUnderCursor(), TouchLocation, TouchForce, TouchIndex, 0); // TODO: ControllerId?
				}

				NumActiveTouches++;
			}
		}

		FindFocusedWidget();
	}

	void FPixelStreamingInputHandler::HandleOnTouchMoved(FMemoryReader Ar)
	{
		TPayloadOneParam<uint8> Payload(Ar);

		uint8 NumTouches = Payload.Param1;
		for (uint8 TouchIdx = 0; TouchIdx < NumTouches; TouchIdx++)
		{
			//                 PosX    PoxY    IDX   Force  Valid
			TPayloadFiveParam<uint16, uint16, uint8, uint8, uint8> Touch(Ar);
			// If Touch is valid
			if (Touch.Param5 != 0)
			{
				//                                                                           convert range from 0,65536 -> 0,1
				FVector2D TouchLocation = ConvertFromNormalizedScreenLocation(FVector2D(Touch.Param1 / uint16_MAX, Touch.Param2 / uint16_MAX));
				const int32 TouchIndex = Touch.Param3;
				const float TouchForce = Touch.Param4 / 255.0f;
				UE_LOGF(LogPixelStreamingInputHandler, Verbose, "TOUCH_MOVE: TouchIndex = %d; Pos = (%d, %d); CursorPos = (%d, %d); Force = %.3f", TouchIndex, Touch.Param1, Touch.Param2, static_cast<int>(TouchLocation.X), static_cast<int>(TouchLocation.Y), TouchForce);

				FCachedTouchEvent& TouchEvent = CachedTouchEvents.FindOrAdd(TouchIndex);
				TouchEvent.Force = TouchForce;
				TouchEvent.ControllerIndex = 0;

				if (InputType == EPixelStreamingInputType::RouteToWidget)
				{
					// TouchLocation = TouchLocation - TargetViewport.Pin()->GetCachedGeometry().GetAbsolutePosition();
					TouchEvent.Location = TouchLocation;
					FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchLocation);

					if (WidgetPath.IsValid())
					{
						FScopedSwitchWorldHack SwitchWorld(WidgetPath);
						FPointerEvent PointerEvent(0, TouchIndex, TouchLocation, LastTouchLocation, TouchForce, true);
						FSlateApplication::Get().RoutePointerMoveEvent(WidgetPath, PointerEvent, false);
					}

					LastTouchLocation = TouchLocation;
				}
				else if (InputType == EPixelStreamingInputType::RouteToWindow)
				{
					TouchEvent.Location = TouchLocation;
					MessageHandler->OnTouchMoved(TouchEvent.Location, TouchEvent.Force, TouchIndex, TouchEvent.ControllerIndex); // TODO: ControllerId?
				}
				TouchIndicesProcessedThisFrame.Add(TouchIndex);
			}
		}
	}

	void FPixelStreamingInputHandler::HandleOnTouchEnded(FMemoryReader Ar)
	{
		TPayloadOneParam<uint8> Payload(Ar);
		uint8 NumTouches = Payload.Param1;
		for (uint8 TouchIdx = 0; TouchIdx < NumTouches; TouchIdx++)
		{
			//                 PosX    PoxY    IDX   Force  Valid
			TPayloadFiveParam<uint16, uint16, uint8, uint8, uint8> Touch(Ar);
			// Always allowing the "up" events regardless of in or outside the valid region so
			// states aren't stuck "down". Might want to uncomment this if it causes other issues.
			// if(Touch.Param5 != 0)
			{
				//                                                                           convert range from 0,65536 -> 0,1
				FVector2D TouchLocation = ConvertFromNormalizedScreenLocation(FVector2D(Touch.Param1 / uint16_MAX, Touch.Param2 / uint16_MAX));
				const int32 TouchIndex = Touch.Param3;
				UE_LOGF(LogPixelStreamingInputHandler, Verbose, "TOUCH_END: TouchIndex = %d; Pos = (%d, %d); CursorPos = (%d, %d)", Touch.Param3, Touch.Param1, Touch.Param2, static_cast<int>(TouchLocation.X), static_cast<int>(TouchLocation.Y));

				if (InputType == EPixelStreamingInputType::RouteToWidget)
				{
					// TouchLocation = TouchLocation - TargetViewport.Pin()->GetCachedGeometry().GetAbsolutePosition();
					FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchLocation);

					if (WidgetPath.IsValid())
					{
						FScopedSwitchWorldHack SwitchWorld(WidgetPath);
						float TouchForce = 0.0f;
						FPointerEvent PointerEvent(0, TouchIndex, TouchLocation, TouchLocation, TouchForce, true);
						FSlateApplication::Get().RoutePointerUpEvent(WidgetPath, PointerEvent);
					}
				}
				else if (InputType == EPixelStreamingInputType::RouteToWindow)
				{
					MessageHandler->OnTouchEnded(TouchLocation, TouchIndex, 0); // TODO: ControllerId?
				}

				CachedTouchEvents.Remove(TouchIndex);
				NumActiveTouches = (NumActiveTouches > 0) ? NumActiveTouches - 1 : NumActiveTouches;
			}
		}

		// If there's no remaining touches, and there is also no mouse over the player window
		// then set the platform application back to its default. We need to set it back to default
		// so that people using the editor (if editor streaming) can click on buttons outside the target window
		// and also have the correct cursor (pixel streaming forces default cursor)
		if (NumActiveTouches == 0 && !bIsMouseActive && InputType == EPixelStreamingInputType::RouteToWindow)
		{
			FVector2D OldCursorLocation = PixelStreamerApplicationWrapper->Cursor->GetPosition();
			PixelStreamerApplicationWrapper->WrappedApplication->Cursor->SetPosition(OldCursorLocation.X, OldCursorLocation.Y);
			FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper->WrappedApplication);
		}
	}

	void FPixelStreamingInputHandler::HandleOnControllerConnected(FMemoryReader Ar)
	{
		TSharedPtr<FPixelStreamingInputDevice> InputDevice = FPixelStreamingInputDevice::GetInputDevice();
		uint8 NextControllerId = InputDevice->OnControllerConnected();

		FString Descriptor = FString::Printf(TEXT("{ \"controllerId\": %d }"), NextControllerId);

		FBufferArchive Buffer;
		Buffer << Descriptor;
		TArray<uint8> Data(Buffer.GetData(), Buffer.Num());
		// Specific implementation for this method is handled per streamer
		OnSendMessage.Broadcast("GamepadResponse", FMemoryReader(Data));

		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "GAMEPAD_CONNECTED: ControllerId = %d", NextControllerId);
	}

	void FPixelStreamingInputHandler::HandleOnControllerAnalog(FMemoryReader Ar)
	{
		const TPayloadThreeParam<uint8, uint8, double> Payload(Ar);
		const FInputDeviceId ControllerId = FInputDeviceId::CreateFromInternalId((int32)Payload.Param1);

		FKeyId KeyId = Payload.Param2;
		double AxisValue = Payload.Param3;

		FKey* AnalogKeyPtr = FPixelStreamingInputConverter::GamepadInputToFKey.Find(MakeTuple(KeyId, Action::Axis));
		if(!AnalogKeyPtr)
		{
			return;
		}

		FAnalogValue AnalogValue;
		AnalogValue.Value = AxisValue;
		// Only send axes values continuously in the case of gamepad triggers
		AnalogValue.bKeepUnlessZero = (KeyId == 5 || KeyId == 6);

		// Overwrite the last data: every tick only process the latest
		AnalogEventsReceivedThisTick.FindOrAdd(ControllerId).FindOrAdd(AnalogKeyPtr) = AnalogValue;
	}

	void FPixelStreamingInputHandler::HandleOnControllerButtonPressed(FMemoryReader Ar)
	{
		TPayloadThreeParam<uint8, uint8, uint8> Payload(Ar);
		FKey* ButtonPtr = FPixelStreamingInputConverter::GamepadInputToFKey.Find(MakeTuple(Payload.Param2, Action::Click));
		if (ButtonPtr == nullptr)
		{
			return;
		}

		FInputDeviceId ControllerId = FInputDeviceId::CreateFromInternalId((int32)Payload.Param1);
		bool bIsRepeat = Payload.Param3 != 0;
		FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser();

		FKeyEvent KeyEvent(
			*ButtonPtr,															  /* InKey */
			FSlateApplication::Get().GetPlatformApplication()->GetModifierKeys(), /* InModifierKeys */
			ControllerId,														  /* InDeviceId */
			bIsRepeat,															  /* bInIsRepeat */
			0,																	  /* InCharacterCode */
			0,																	  /* InKeyCode */
			// TODO (william.belcher): This user idx should be the playerId
			0 /* InUserIndex*/
		);
		FSlateApplication::Get().ProcessKeyDownEvent(KeyEvent);

		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "GAMEPAD_PRESSED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls;", ControllerId.GetId(), *ButtonPtr->ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));
	}

	void FPixelStreamingInputHandler::HandleOnControllerButtonReleased(FMemoryReader Ar)
	{
		TPayloadTwoParam<uint8, uint8> Payload(Ar);
		FKey* ButtonPtr = FPixelStreamingInputConverter::GamepadInputToFKey.Find(MakeTuple(Payload.Param2, Action::Click));
		if (ButtonPtr == nullptr)
		{
			return;
		}

		FInputDeviceId ControllerId = FInputDeviceId::CreateFromInternalId((int32)Payload.Param1);
		FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser();

		FKeyEvent KeyEvent(
			*ButtonPtr,															  /* InKey */
			FSlateApplication::Get().GetPlatformApplication()->GetModifierKeys(), /* InModifierKeys */
			ControllerId,														  /* InDeviceId */
			false,																  /* bInIsRepeat */
			0,																	  /* InCharacterCode */
			0,																	  /* InKeyCode */
			// TODO (william.belcher): This user idx should be the playerId
			0 /* InUserIndex*/
		);
		FSlateApplication::Get().ProcessKeyUpEvent(KeyEvent);

		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "GAMEPAD_RELEASED: ControllerId = %d; KeyName = %ls;", ControllerId.GetId(), *ButtonPtr->ToString());
	}

	void FPixelStreamingInputHandler::HandleOnControllerDisconnected(FMemoryReader Ar)
	{
		TPayloadOneParam<uint8> Payload(Ar);

		uint8 DisconnectedControllerId = Payload.Param1;

		TSharedPtr<FPixelStreamingInputDevice> InputDevice = FPixelStreamingInputDevice::GetInputDevice();
		InputDevice->OnControllerDisconnected(DisconnectedControllerId);

		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "GAMEPAD_DISCONNECTED: ControllerId = %d", DisconnectedControllerId);
	}

	/**
	 * Mouse events
	 */
	void FPixelStreamingInputHandler::HandleOnMouseEnter(FMemoryReader Ar)
	{
		if (NumActiveTouches == 0 && !bIsMouseActive)
		{
			FSlateApplication::Get().OnCursorSet();
			FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper);
			// Make sure the application is active.
			FSlateApplication::Get().ProcessApplicationActivationEvent(true);
		}

		bIsMouseActive = true;
		UE_LOG(LogPixelStreamingInputHandler, Verbose, TEXT("MOUSE_ENTER"));
	}

	void FPixelStreamingInputHandler::HandleOnMouseLeave(FMemoryReader Ar)
	{
		if (NumActiveTouches == 0)
		{
			// Restore normal application layer if there is no active touches and MouseEnter hasn't been triggered
			FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper->WrappedApplication);
		}
		bIsMouseActive = false;
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "MOUSE_LEAVE");
	}

	void FPixelStreamingInputHandler::HandleOnMouseUp(FMemoryReader Ar)
	{
		// Ensure we have wrapped the slate application at this point
		if (!bIsMouseActive)
		{
			HandleOnMouseEnter(Ar);
		}

		TPayloadThreeParam<uint8, uint16, uint16> Payload(Ar);

		EMouseButtons::Type Button = static_cast<EMouseButtons::Type>(Payload.Param1);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "MOUSE_UP: Button = %d", Button);

		if (InputType == EPixelStreamingInputType::RouteToWidget)
		{
			FSlateApplication& SlateApplication = FSlateApplication::Get();
			FWidgetPath WidgetPath = FindRoutingMessageWidget(SlateApplication.GetCursorPos());

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);
				FKey Key = TranslateMouseButtonToKey(Button);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetLastCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					Key,
					0,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				SlateApplication.RoutePointerUpEvent(WidgetPath, MouseEvent);
			}
		}
		else if (InputType == EPixelStreamingInputType::RouteToWindow)
		{
			if (Button != EMouseButtons::Type::Invalid)
			{
				MessageHandler->OnMouseUp(Button);
			}
		}
	}

	void FPixelStreamingInputHandler::HandleOnMouseDown(FMemoryReader Ar)
	{
		// Ensure we have wrapped the slate application at this point
		if (!bIsMouseActive)
		{
			HandleOnMouseEnter(Ar);
		}

		TPayloadThreeParam<uint8, uint16, uint16> Payload(Ar);
		//                                                                           convert range from 0,65536 -> 0,1
		FVector2D ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Param2 / uint16_MAX, Payload.Param3 / uint16_MAX));
		EMouseButtons::Type Button = static_cast<EMouseButtons::Type>(Payload.Param1);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "MOUSE_DOWN: Button = %d; Pos = (%.4f, %.4f)", Button, ScreenLocation.X, ScreenLocation.Y);
		// Set cursor pos on mouse down - we may not have moved if this is the very first click
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		SlateApplication.OnCursorSet();
		PixelStreamerApplicationWrapper->Cursor->SetPosition(ScreenLocation.X, ScreenLocation.Y);
		// Force window focus
		SlateApplication.ProcessApplicationActivationEvent(true);

		if (InputType == EPixelStreamingInputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);

				FKey Key = TranslateMouseButtonToKey(Button);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					ScreenLocation,
					SlateApplication.GetLastCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					Key,
					0,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				SlateApplication.RoutePointerDownEvent(WidgetPath, MouseEvent);
			}
		}
		else if (InputType == EPixelStreamingInputType::RouteToWindow)
		{
			MessageHandler->OnMouseDown(PixelStreamerApplicationWrapper->GetWindowUnderCursor(), Button, ScreenLocation);
		}
	}

	void FPixelStreamingInputHandler::HandleOnMouseMove(FMemoryReader Ar)
	{
		TPayloadFourParam<uint16, uint16, int16, int16> Payload(Ar);
		//                                                                           convert range from 0,65536 -> 0,1
		FIntPoint ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Param1 / uint16_MAX, Payload.Param2 / uint16_MAX));
		//                                                                 convert range from -32,768 to 32,767 -> -1,1
		FIntPoint Delta = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Param3 / int16_MAX, Payload.Param4 / int16_MAX), false);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "MOUSE_MOVE: Pos = (%d, %d); Delta = (%d, %d)", ScreenLocation.X, ScreenLocation.Y, Delta.X, Delta.Y);
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		SlateApplication.OnCursorSet();
		PixelStreamerApplicationWrapper->Cursor->SetPosition(ScreenLocation.X, ScreenLocation.Y);

		if (InputType == EPixelStreamingInputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetLastCursorPos(),
					FVector2D(Delta.X, Delta.Y),
					SlateApplication.GetPressedMouseButtons(),
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				SlateApplication.RoutePointerMoveEvent(WidgetPath, MouseEvent, false);
			}
		}
		else if (InputType == EPixelStreamingInputType::RouteToWindow)
		{
			MessageHandler->OnRawMouseMove(Delta.X, Delta.Y);
		}
	}

	void FPixelStreamingInputHandler::HandleOnMouseWheel(FMemoryReader Ar)
	{
		TPayloadThreeParam<int16, uint16, uint16> Payload(Ar);
		//                                                                           convert range from 0,65536 -> 0,1
		FIntPoint ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Param2 / uint16_MAX, Payload.Param3 / uint16_MAX));
		const float SpinFactor = 1 / 120.0f;
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "MOUSE_WHEEL: Delta = %d; Pos = (%d, %d)", Payload.Param1, ScreenLocation.X, ScreenLocation.Y);

		if (InputType == EPixelStreamingInputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);

				FSlateApplication& SlateApplication = FSlateApplication::Get();

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					EKeys::Invalid,
					Payload.Param1 * SpinFactor,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				SlateApplication.RouteMouseWheelOrGestureEvent(WidgetPath, MouseEvent, nullptr);
			}
		}
		else if (InputType == EPixelStreamingInputType::RouteToWindow)
		{
			MessageHandler->OnMouseWheel(Payload.Param1 * SpinFactor, ScreenLocation);
		}
	}

	void FPixelStreamingInputHandler::HandleOnMouseDoubleClick(FMemoryReader Ar)
	{
		TPayloadThreeParam<uint8, uint16, uint16> Payload(Ar);
		//                                                                           convert range from 0,65536 -> 0,1
		FVector2D ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Param2 / uint16_MAX, Payload.Param3 / uint16_MAX));
		EMouseButtons::Type Button = static_cast<EMouseButtons::Type>(Payload.Param1);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "MOUSE_DOWN: Button = %d; Pos = (%.4f, %.4f)", Button, ScreenLocation.X, ScreenLocation.Y);
		// Force window focus
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		SlateApplication.ProcessApplicationActivationEvent(true);

		if (InputType == EPixelStreamingInputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);
				FKey Key = TranslateMouseButtonToKey(Button);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetLastCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					Key,
					0,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				SlateApplication.RoutePointerDoubleClickEvent(WidgetPath, MouseEvent);
			}
		}
		else if (InputType == EPixelStreamingInputType::RouteToWindow)
		{
			MessageHandler->OnMouseDoubleClick(PixelStreamerApplicationWrapper->GetWindowUnderCursor(), Button, ScreenLocation);
		}
	}

	FMatrix FPixelStreamingInputHandler::ExtractWebXRMatrix(FMemoryReader& Ar)
	{
		FMatrix OutMat;
		for (int32 Row = 0; Row < 4; ++Row)

		{
			float Col0 = 0.0f, Col1 = 0.0f, Col2 = 0.0f, Col3 = 0.0f;
			Ar << Col0 << Col1 << Col2 << Col3;
			OutMat.M[Row][0] = Col0;
			OutMat.M[Row][1] = Col1;
			OutMat.M[Row][2] = Col2;
			OutMat.M[Row][3] = Col3;
		}
		OutMat.DiagnosticCheckNaN();
		return OutMat;
	}

	FTransform FPixelStreamingInputHandler::WebXRMatrixToUETransform(FMatrix Mat)
	{
		// Rows and columns are swapped between raw mat and FMat
		FMatrix UEMatrix = FMatrix(
			FPlane(Mat.M[0][0], Mat.M[1][0], Mat.M[2][0], Mat.M[3][0]),
			FPlane(Mat.M[0][1], Mat.M[1][1], Mat.M[2][1], Mat.M[3][1]),
			FPlane(Mat.M[0][2], Mat.M[1][2], Mat.M[2][2], Mat.M[3][2]),
			FPlane(Mat.M[0][3], Mat.M[1][3], Mat.M[2][3], Mat.M[3][3]));

		// Extract scale vector and reorder coordinates to be UE coordinate system.
		FVector ScaleVectorRaw = UEMatrix.GetScaleVector();
		// Note: We do not invert Z scaling here because we already handle that when we rebuild translation/rot below.
		FVector ScaleVector = FVector(ScaleVectorRaw.Z, ScaleVectorRaw.X, ScaleVectorRaw.Y);

		// Temporarily remove scaling component as we need rotation axes to be unit length for proper quat conversion
		UEMatrix.RemoveScaling();

		// Extract & convert translation component to UE coordinate syste,
		FVector Translation = FVector(-UEMatrix.M[3][2], UEMatrix.M[3][0], UEMatrix.M[3][1]) * 100.0f;

		// Extract & convert rotation component to UE coordinate system
		FQuat RawRotation(UEMatrix);
		FQuat Rotation(-RawRotation.Z, RawRotation.X, RawRotation.Y, -RawRotation.W);
		return FTransform(Rotation, Translation, ScaleVector);
	}

	/**
	 * XR Handling
	 */
	void FPixelStreamingInputHandler::HandleOnXREyeViews(FMemoryReader Ar)
	{
		// The `Ar` buffer contains the left eye transform matrix stored as 16 floats
		FTransform LeftEyeTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		// The `Ar` buffer contains the left eye projection matrix stored as 16 floats
		FMatrix LeftEyeProjectionMatrix = ExtractWebXRMatrix(Ar);

		// The `Ar` buffer contains the right eye transform matrix stored as 16 floats
		FTransform RightEyeTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		// The `Ar` buffer contains the right eye projection matrix stored as 16 floats
		FMatrix RightEyeProjectionMatrix = ExtractWebXRMatrix(Ar);

		// The `Ar` buffer contains the right eye projection matrix stored as 16 floats
		FTransform HMDTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		if (FPixelStreamingHMD* HMD = IPixelStreamingHMDModule::Get().GetPixelStreamingHMD(); HMD != nullptr)
		{
			HMD->SetEyeViews(LeftEyeTransform, LeftEyeProjectionMatrix, RightEyeTransform, RightEyeProjectionMatrix, HMDTransform);
		}
	}

	void FPixelStreamingInputHandler::HandleOnXRHMDTransform(FMemoryReader Ar)
	{
		// The `Ar` buffer contains the transform matrix stored as 16 floats
		FTransform HMDTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		if (FPixelStreamingHMD* HMD = IPixelStreamingHMDModule::Get().GetPixelStreamingHMD(); HMD != nullptr)
		{
			HMD->SetTransform(HMDTransform);
		}
	}

	void FPixelStreamingInputHandler::HandleOnXRControllerTransform(FMemoryReader Ar)
	{
		// The `Ar` buffer contains the transform matrix stored as 16 floats
		FTransform ControllerTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		// The `Ar` buffer contains a UInt8 for the handedness
		EControllerHand Handedness = EControllerHand::Left;
		Ar << Handedness;

		FPixelStreamingXRController Controller;
		Controller.Transform = ControllerTransform;
		Controller.Handedness = Handedness;
		XRControllers.Add(Handedness, Controller);
	}

	void FPixelStreamingInputHandler::HandleOnXRButtonTouched(FMemoryReader Ar)
	{
		TPayloadThreeParam<uint8, uint8, uint8> Payload(Ar);
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId ControllerId = DeviceMapper.GetDefaultInputDevice();

		XRSystem System = IPixelStreamingHMDModule::Get().GetActiveXRSystem();
		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Param1);
		uint8 ButtonIdx = Payload.Param2;
		bool bIsRepeat = Payload.Param3 != 0;

		FKey* ButtonPtr = FPixelStreamingInputConverter::XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, Action::Touch));
		if (ButtonPtr == nullptr)
		{
			return;
		}

		MessageHandler->OnControllerButtonPressed(ButtonPtr->GetFName(), DeviceMapper.GetPrimaryPlatformUser(), ControllerId, bIsRepeat);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "XR_TOUCHED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls;", ControllerId.GetId(), *ButtonPtr->ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));
	}

	void FPixelStreamingInputHandler::HandleOnXRButtonTouchReleased(FMemoryReader Ar)
	{
		TPayloadThreeParam<uint8, uint8, uint8> Payload(Ar);
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId ControllerId = DeviceMapper.GetDefaultInputDevice();

		XRSystem System = IPixelStreamingHMDModule::Get().GetActiveXRSystem();
		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Param1);
		uint8 ButtonIdx = Payload.Param2;
		bool bIsRepeat = Payload.Param3 != 0;

		FKey* ButtonPtr = FPixelStreamingInputConverter::XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, Action::Touch));
		if (ButtonPtr == nullptr)
		{
			return;
		}

		MessageHandler->OnControllerButtonReleased(ButtonPtr->GetFName(), DeviceMapper.GetPrimaryPlatformUser(), ControllerId, bIsRepeat);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "XR_TOUCH_RELEASED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls;", ControllerId.GetId(), *ButtonPtr->ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));
	}

	void FPixelStreamingInputHandler::HandleOnXRButtonPressed(FMemoryReader Ar)
	{
		TPayloadFourParam<uint8, uint8, uint8, double> Payload(Ar);
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId ControllerId = DeviceMapper.GetDefaultInputDevice();

		XRSystem System = IPixelStreamingHMDModule::Get().GetActiveXRSystem();
		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Param1);
		uint8 ButtonIdx = Payload.Param2;
		bool bIsRepeat = Payload.Param3 != 0;

		FKey* ButtonPtr = FPixelStreamingInputConverter::XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, Action::Click));

		// Try and see if there is an axis associated with this button (usually the case for triggers)
		FKey* AxisPtr = FPixelStreamingInputConverter::XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, Action::Axis));

		// Only send button click if there is no axis associated with this button or this the first press of the axis
		if (ButtonPtr)
		{
			MessageHandler->OnControllerButtonPressed(ButtonPtr->GetFName(), DeviceMapper.GetPrimaryPlatformUser(), ControllerId, bIsRepeat);
			UE_LOGF(LogPixelStreamingInputHandler, Verbose, "XR_PRESSED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls", ControllerId.GetId(), *ButtonPtr->ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));
		}

		// If we have axis associate with this press then set axis value to the button press value
		if(AxisPtr)
		{
			// Trigger axes are not robust to some inputs missing across frames.
			// So to protect against this case (where PS is too slow to transmit them)
			// we must set the `bKeepUnlessZero` flag, which allows the input to be continuously passed
			// until a release is fired.
			FAnalogValue AnalogValue;
			AnalogValue.bKeepUnlessZero = true;
			AnalogValue.Value = Payload.Param4;
			AnalogValue.bIsRepeat = false;
			UE_LOGF(LogPixelStreamingInputHandler, Verbose, "XR_ANALOG: ControllerId = %d; KeyName = %ls; IsRepeat = False; AnalogValue = %.4f; [Queued for Tick()]", ControllerId.GetId(), *AxisPtr->ToString(), AnalogValue.Value);
			AnalogEventsReceivedThisTick.FindOrAdd(ControllerId).FindOrAdd(AxisPtr) = AnalogValue;
		}
	}

	void FPixelStreamingInputHandler::HandleOnXRButtonReleased(FMemoryReader Ar)
	{
		TPayloadThreeParam<uint8, uint8, uint8> Payload(Ar);
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId ControllerId = DeviceMapper.GetDefaultInputDevice();

		XRSystem System = IPixelStreamingHMDModule::Get().GetActiveXRSystem();
		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Param1);
		uint8 ButtonIdx = Payload.Param2;
		bool bIsRepeat = Payload.Param3 != 0;

		// Try and see if there is an axis associated with this button (usually the case for triggers)
		FKey* AxisPtr = FPixelStreamingInputConverter::XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, Action::Axis));

		// If we have axis associate with this release then set axis value to 0.0
		if (AxisPtr)
		{
			// In the case of an axes release, we should clear any analog value that is being
			// applied across ticks.
			FAnalogValue AnalogValue;
			AnalogValue.bKeepUnlessZero = true;
			AnalogValue.Value = 0.0;
			AnalogValue.bIsRepeat = false;
			UE_LOGF(LogPixelStreamingInputHandler, Verbose, "XR_ANALOG: ControllerId = %d; KeyName = %ls; IsRepeat = False; AnalogValue = %.4f; [Queued for Tick()]", ControllerId.GetId(), *AxisPtr->ToString(), AnalogValue.Value);
			AnalogEventsReceivedThisTick.FindOrAdd(ControllerId).FindOrAdd(AxisPtr) = AnalogValue;
		}

		// Do the actual release after the analog trigger, as the release can cancel any further inputs
		FKey* ButtonPtr = FPixelStreamingInputConverter::XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, Action::Click));
		if (ButtonPtr)
		{
			MessageHandler->OnControllerButtonReleased(ButtonPtr->GetFName(), DeviceMapper.GetPrimaryPlatformUser(), ControllerId, bIsRepeat);
			UE_LOGF(LogPixelStreamingInputHandler, Verbose, "XR_RELEASED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls;", ControllerId.GetId(), *ButtonPtr->ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));
		}
	}

	void FPixelStreamingInputHandler::HandleOnXRAnalog(FMemoryReader Ar)
	{
		TPayloadThreeParam<uint8, uint8, double> Payload(Ar);
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId ControllerId = DeviceMapper.GetDefaultInputDevice();

		XRSystem System = IPixelStreamingHMDModule::Get().GetActiveXRSystem();
		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Param1);
		Action InputAction = static_cast<Action>(Payload.Param2 % 2);
		int AxisIndex = Payload.Param2;
		FKey* AnalogKeyPtr = FPixelStreamingInputConverter::XRInputToFKey.Find(MakeTuple(System, Handedness, AxisIndex, InputAction));
		if (AnalogKeyPtr == nullptr)
		{
			return;
		}

		// This codepath is used for XR joysticks, which seems to be robust to temporary drops in input transmission
		// so we can safely set `bKeepUnlessZero` to false. However, if we use this for more than joysticks we will have to conditionally set this.
		FAnalogValue AnalogValue;
		AnalogValue.bKeepUnlessZero = false;
		// Y-axis is inverted in WebXR Gamepad API compared to UE
		AnalogValue.Value = AxisIndex % 2 == 0 ? Payload.Param3 : -Payload.Param3;
		AnalogValue.bIsRepeat = false;

		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "XR_ANALOG: ControllerId = %d; KeyName = %ls; IsRepeat = False; AnalogValue = %.4f; [Queued for Tick()]", ControllerId.GetId(), *AnalogKeyPtr->ToString(), AnalogValue.Value);
		AnalogEventsReceivedThisTick.FindOrAdd(ControllerId).FindOrAdd(AnalogKeyPtr) = AnalogValue;
	}

	void FPixelStreamingInputHandler::HandleOnXRSystem(FMemoryReader Ar)
	{
		uint8 ActiveSystem = (uint8)EPixelStreamingXRSystem::Unknown;
		Ar << ActiveSystem;
		IPixelStreamingHMDModule::Get().SetActiveXRSystem(static_cast<XRSystem>(ActiveSystem));
	}

	void FPixelStreamingInputHandler::SetCommandHandler(const FString& CommandName, const CommandHandlerFn& Handler)
	{
		CommandHandlers.Add(CommandName, Handler);
	}

	void FPixelStreamingInputHandler::SetElevatedCheck(const TFunction<bool(FString)>& CheckFn)
	{
		ElevatedCheck = CheckFn;
	}

	bool FPixelStreamingInputHandler::IsElevated(const FString& Id)
	{
		return !ElevatedCheck || ElevatedCheck(Id);
	}

	void FPixelStreamingInputHandler::PopulateDefaultCommandHandlers()
	{
		// Execute console commands if passed "ConsoleCommand" and -PixelStreamingAllowConsoleCommands is on.
		CommandHandlers.Add(TEXT("ConsoleCommand"), [this](FString SourceId, FString Descriptor, FString ConsoleCommand) {
			if (!UE::PixelStreamingInput::Settings::CVarPixelStreamingInputAllowConsoleCommands.GetValueOnAnyThread()
				|| !IsElevated(SourceId))
			{
				return;
			}
			GEngine->Exec(GEngine->GetWorld(), *ConsoleCommand);
		});

		// Change width/height if sent { "Resolution.Width": 1920, "Resolution.Height": 1080 }
		CommandHandlers.Add(TEXT("Resolution.Width"), [this](FString SourceId, FString Descriptor, FString WidthString) {
			bool bSuccess = false;
			FString HeightString;
			UE::PixelStreamingInput::ExtractJsonFromDescriptor(Descriptor, TEXT("Resolution.Height"), HeightString, bSuccess);
			if (bSuccess && IsElevated(SourceId))
			{
				int Width = FCString::Atoi(*WidthString);
				int Height = FCString::Atoi(*HeightString);
				if (Width < 1 || Height < 1)
				{
					return;
				}

				FString ChangeResCommand = FString::Printf(TEXT("r.SetRes %dx%d"), Width, Height);
				GEngine->Exec(GEngine->GetWorld(), *ChangeResCommand);
			}
		});

		// Response to "Stat.FPS" by calling "stat fps"
		CommandHandlers.Add(TEXT("Stat.FPS"), [](FString SourceId, FString Descriptor, FString FPSCommand) {
			FString StatFPSCommand = FString::Printf(TEXT("stat fps"));
			GEngine->Exec(GEngine->GetWorld(), *StatFPSCommand);
		});
	}

	/**
	 * Command handling
	 */
	void FPixelStreamingInputHandler::HandleOnCommand(FString SourceId, FMemoryReader Ar)
	{
		FString Res;
		Res.GetCharArray().SetNumUninitialized(Ar.TotalSize() / 2 + 1);
		Ar.Serialize(Res.GetCharArray().GetData(), Ar.TotalSize());

		FString Descriptor = Res.Mid(MessageHeaderOffset);
		UE_LOGF(LogPixelStreamingInputHandler, Verbose, "Command: %ls", *Descriptor);

		// Iterate each command handler and see if the command we got matches any of the bound command names.
		for (auto& CommandHandlersPair : CommandHandlers)
		{
			FString CommandValue;
			bool bSuccess = false;
			UE::PixelStreamingInput::ExtractJsonFromDescriptor(Descriptor, CommandHandlersPair.Key, CommandValue, bSuccess);
			if (bSuccess)
			{
				// Execute bound command handler with descriptor and parsed command value
				CommandHandlersPair.Value(SourceId, Descriptor, CommandValue);
				return;
			}
		}
	}

	/**
	 * UI Interaction handling
	 */
	void FPixelStreamingInputHandler::HandleUIInteraction(FMemoryReader Ar)
	{
		// Actual implementation is in the Pixel Streaming module
	}

	/**
	 * Textbox Entry handling
	 */
	void FPixelStreamingInputHandler::HandleOnTextboxEntry(FMemoryReader Ar)
	{
		FString Res;
		Res.GetCharArray().SetNumUninitialized(Ar.TotalSize() / 2 + 1);
		Ar.Serialize(Res.GetCharArray().GetData(), Ar.TotalSize());
		FString Text = Res.Mid(1);

		FSlateApplication::Get().ForEachUser([this, Text](FSlateUser& User) {
			TSharedPtr<SWidget> FocusedWidget = User.GetFocusedWidget();

			bool bIsEditableTextType = FocusedWidget->GetType() == TEXT("SEditableText");
			bool bIsMultiLineEditableTextType = FocusedWidget->GetType() == TEXT("SMultiLineEditableText");
			bool bEditable = FocusedWidget && (bIsEditableTextType || bIsMultiLineEditableTextType);

			if (bEditable)
			{
				if (bIsEditableTextType)
				{
					SEditableText* TextBox = static_cast<SEditableText*>(FocusedWidget.Get());
					TextBox->SetText(FText::FromString(Text));
				}
				else if (bIsMultiLineEditableTextType)
				{
					SMultiLineEditableText* TextBox = static_cast<SMultiLineEditableText*>(FocusedWidget.Get());
					TextBox->SetText(FText::FromString(Text));
				}

				// We need to manually trigger an Enter key press so that the OnTextCommitted delegate gets fired
				const uint32* KeyPtr = nullptr;
				const uint32* CharacterPtr = nullptr;
				FInputKeyManager::Get().GetCodesFromKey(EKeys::Enter, KeyPtr, CharacterPtr);
				uint32 Key = KeyPtr ? *KeyPtr : 0;
				uint32 Character = CharacterPtr ? *CharacterPtr : 0;
				if(Key != 0 || Character != 0)
				{
					MessageHandler->OnKeyDown((int32)Key, (int32)Character, false);
					MessageHandler->OnKeyUp((int32)Key, (int32)Character, false);
				}
			}
		});
	}

	FIntPoint FPixelStreamingInputHandler::ConvertFromNormalizedScreenLocation(const FVector2D& ScreenLocation, bool bIncludeOffset)
	{
		FIntPoint OutVector((int32)ScreenLocation.X, (int32)ScreenLocation.Y);

		if (TSharedPtr<SWindow> ApplicationWindow = TargetWindow.Pin())
		{
			FVector2D WindowOrigin = ApplicationWindow->GetPositionInScreen();
			if (TSharedPtr<SViewport> ViewportWidget = TargetViewport.Pin())
			{
				FGeometry InnerWindowGeometry = ApplicationWindow->GetWindowGeometryInWindow();

				// Find the widget path relative to the window
				FArrangedChildren JustWindow(EVisibility::Visible);
				JustWindow.AddWidget(FArrangedWidget(ApplicationWindow.ToSharedRef(), InnerWindowGeometry));

				FWidgetPath PathToWidget(ApplicationWindow.ToSharedRef(), JustWindow);
				if (PathToWidget.ExtendPathTo(FWidgetMatcher(ViewportWidget.ToSharedRef()), EVisibility::Visible))
				{
					FArrangedWidget ArrangedWidget = PathToWidget.FindArrangedWidget(ViewportWidget.ToSharedRef()).Get(FArrangedWidget::GetNullWidget());

					FVector2D WindowClientOffset = ArrangedWidget.Geometry.GetAbsolutePosition();
					FVector2D WindowClientSize = ArrangedWidget.Geometry.GetAbsoluteSize();

					FVector2D OutTemp = bIncludeOffset
						? (ScreenLocation * WindowClientSize) + WindowOrigin + WindowClientOffset
						: (ScreenLocation * WindowClientSize);
					UE_LOGF(LogPixelStreamingInputHandler, Verbose, "%.4f, %.4f", ScreenLocation.X, ScreenLocation.Y);
					OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
				}
			}
			else
			{
				FVector2D SizeInScreen = ApplicationWindow->GetSizeInScreen();
				FVector2D OutTemp = bIncludeOffset
					? (SizeInScreen * ScreenLocation) + ApplicationWindow->GetPositionInScreen()
					: (SizeInScreen * ScreenLocation);
				OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
			}
		}
		else if (TSharedPtr<FIntRect> ScreenRectPtr = TargetScreenRect.Pin())
		{
			FIntRect ScreenRect = *ScreenRectPtr;
			FIntPoint SizeInScreen = ScreenRect.Max - ScreenRect.Min;
			FVector2D OutTemp = FVector2D(SizeInScreen.X, SizeInScreen.Y) * ScreenLocation + (bIncludeOffset ? FVector2D(ScreenRect.Min.X, ScreenRect.Min.Y) : FVector2D(0, 0));
			OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
		}
		else if (TSharedPtr<FIntPoint> ScreenSize = TargetScreenSize.Pin())
		{
			UE_LOGF(LogPixelStreamingInputHandler, Warning, "You're using deprecated functionality by setting a target screen size. This functionality will be removed in later versions. Please use SetTargetScreenRect instead!");
			FIntPoint SizeInScreen = *ScreenSize;
			FVector2D OutTemp = FVector2D(SizeInScreen) * ScreenLocation;
			OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
		}

		return OutVector;
	}

	bool FPixelStreamingInputHandler::FilterKey(const FKey& Key)
	{
		for (auto&& FilteredKey : Settings::FilteredKeys)
		{
			if (FilteredKey == Key)
				return false;
		}
		return true;
	}

	void FPixelStreamingInputHandler::ProcessLatestAnalogInputFromThisTick()
	{
		for (auto AnalogInputIt = AnalogEventsReceivedThisTick.CreateIterator(); AnalogInputIt; ++AnalogInputIt)
		{
			for (auto FKeyIt = AnalogInputIt->Value.CreateIterator(); FKeyIt; ++FKeyIt)
			{
				const FInputDeviceId& ControllerId = AnalogInputIt->Key;
				FKey* Key = FKeyIt->Key;
				FAnalogValue AnalogValue = FKeyIt->Value;
				bool bIsRepeat = AnalogValue.bIsRepeat;

				if (!Key)
				{
					return;
				}

				// Pass an analog input along the engine's input processing system
				FSlateApplication& SlateApplication = FSlateApplication::Get();
				const FAnalogInputEvent AnalogInputEvent(
					*Key,															/* InKey */
					SlateApplication.GetPlatformApplication()->GetModifierKeys(),	/* InModifierKeys */
					ControllerId,													/* InDeviceId */
					bIsRepeat,														/* bInIsRepeat*/
					0,																/* InCharacterCode */
					0,																/* InKeyCode */
					AnalogValue.Value,												/* InAnalogValue */
					// TODO (william.belcher): This user idx should be the playerId
					0 /* InUserIndex */
				);

				bool bHandled = FSlateApplication::Get().ProcessAnalogInputEvent(AnalogInputEvent);
				UE_LOGF(LogPixelStreamingInputHandler, Verbose, "TICKED ANALOG Input: ControllerId = %d; KeyName = %ls; IsRepeat = %ls; AnalogValue = %.4f; Handled = %ls; [Queued for Tick()]", ControllerId.GetId(), *Key->ToString(), bIsRepeat ? TEXT("True") : TEXT("False"), AnalogValue.Value, bHandled ? TEXT("True") : TEXT("False"));

				// Remove current analog key unless it has the special `bKeepUnlessZero` flag set.
				// This flag is used to continuously apply input values across ticks because
				// Pixel Streaming may not have transmitted an axis value in time for the next tick.
				// But in all ordinary cases where this flag is not set, the stored analog value should
				// be dropped from the map so the input for the axis (e.g. joystick) is only applied the frame
				// it is received. The `bKeepUnlessZero` is used for trigger axes, where a temporary drop in
				// input triggers UE into thinking a full press/release should occur.
				if(!AnalogValue.bKeepUnlessZero)
				{
					FKeyIt.RemoveCurrent();
				}
				else if(AnalogValue.bKeepUnlessZero && AnalogValue.Value == 0.0)
				{
					// HACK: If we have zero, send it again next frame to ensure we trigger a release internally
					// Without this release does not seem to get processed for axes inputs
					FKeyIt->Value.bIsRepeat = true;
					FKeyIt->Value.bKeepUnlessZero = false;
				}
				else
				{
					// We are resending the same input, signal this is the case on UE side
					FKeyIt->Value.bIsRepeat = true;
				}
			}
		}
	}

	void FPixelStreamingInputHandler::BroadcastActiveTouchMoveEvents()
	{
		if (!ensure(MessageHandler))
		{
			return;
		}

		for (TPair<int32, FCachedTouchEvent> CachedTouchEvent : CachedTouchEvents)
		{
			const int32& TouchIndex = CachedTouchEvent.Key;
			const FCachedTouchEvent& TouchEvent = CachedTouchEvent.Value;

			// Only broadcast events that haven't already been fired this frame
			if (!TouchIndicesProcessedThisFrame.Contains(TouchIndex))
			{
				if (InputType == EPixelStreamingInputType::RouteToWidget)
				{
					FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchEvent.Location);

					if (WidgetPath.IsValid())
					{
						FScopedSwitchWorldHack SwitchWorld(WidgetPath);
						FPointerEvent PointerEvent(0, TouchIndex, TouchEvent.Location, LastTouchLocation, TouchEvent.Force, true);
						FSlateApplication::Get().RoutePointerMoveEvent(WidgetPath, PointerEvent, false);
					}
				}
				else if (InputType == EPixelStreamingInputType::RouteToWindow)
				{
					MessageHandler->OnTouchMoved(TouchEvent.Location, TouchEvent.Force, TouchIndex, TouchEvent.ControllerIndex);
				}
			}
		}
	}

	FKey FPixelStreamingInputHandler::TranslateMouseButtonToKey(const EMouseButtons::Type Button)
	{
		FKey Key = EKeys::Invalid;

		switch (Button)
		{
			case EMouseButtons::Left:
				Key = EKeys::LeftMouseButton;
				break;
			case EMouseButtons::Middle:
				Key = EKeys::MiddleMouseButton;
				break;
			case EMouseButtons::Right:
				Key = EKeys::RightMouseButton;
				break;
			case EMouseButtons::Thumb01:
				Key = EKeys::ThumbMouseButton;
				break;
			case EMouseButtons::Thumb02:
				Key = EKeys::ThumbMouseButton2;
				break;
		}

		return Key;
	}

	void FPixelStreamingInputHandler::FindFocusedWidget()
	{
		FSlateApplication::Get().ForEachUser([this](FSlateUser& User) {
			TSharedPtr<SWidget> FocusedWidget = User.GetFocusedWidget();

			static FName SEditableTextType(TEXT("SEditableText"));
			static FName SMultiLineEditableTextType(TEXT("SMultiLineEditableText"));
			bool bEditable = FocusedWidget && (FocusedWidget->GetType() == SEditableTextType || FocusedWidget->GetType() == SMultiLineEditableTextType);
			if (bEditable)
			{
				if (FocusedWidget->GetType() == TEXT("SEditableText"))
				{
					SEditableText* TextBox = static_cast<SEditableText*>(FocusedWidget.Get());
					bEditable = !TextBox->IsTextReadOnly();
				}
				else if (FocusedWidget->GetType() == TEXT("SMultiLineEditableText"))
				{
					SMultiLineEditableText* TextBox = static_cast<SMultiLineEditableText*>(FocusedWidget.Get());
					bEditable = !TextBox->IsTextReadOnly();
				}
			}

			// Tell the browser that the focus has changed.
			TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
			JsonObject->SetStringField(TEXT("command"), TEXT("onScreenKeyboard"));
			JsonObject->SetBoolField(TEXT("showOnScreenKeyboard"), bEditable);

			if (bEditable)
			{
				FVector2D NormalizedLocation = FVector2D::ZeroVector;
				TSharedPtr<SWindow> ApplicationWindow = TargetWindow.Pin();
				if (ApplicationWindow.IsValid())
				{
					FVector2D WindowOrigin = ApplicationWindow->GetPositionInScreen();
					FVector2D Pos = FocusedWidget->GetCachedGeometry().GetAbsolutePosition();
					if (TargetViewport.IsValid())
					{
						TSharedPtr<SViewport> ViewportWidget = TargetViewport.Pin();

						if (ViewportWidget.IsValid())
						{
							FGeometry InnerWindowGeometry = ApplicationWindow->GetWindowGeometryInWindow();

							// Find the widget path relative to the window
							FArrangedChildren JustWindow(EVisibility::Visible);
							JustWindow.AddWidget(FArrangedWidget(ApplicationWindow.ToSharedRef(), InnerWindowGeometry));

							FWidgetPath PathToWidget(ApplicationWindow.ToSharedRef(), JustWindow);
							if (PathToWidget.ExtendPathTo(FWidgetMatcher(ViewportWidget.ToSharedRef()), EVisibility::Visible))
							{
								FArrangedWidget ArrangedWidget = PathToWidget.FindArrangedWidget(ViewportWidget.ToSharedRef()).Get(FArrangedWidget::GetNullWidget());

								FVector2D WindowClientOffset = ArrangedWidget.Geometry.GetAbsolutePosition();
								FVector2D WindowClientSize = ArrangedWidget.Geometry.GetAbsoluteSize();

								Pos = Pos - WindowClientOffset;
								NormalizedLocation = FVector2D(Pos / WindowClientSize);
							}
						}
					}
					else
					{
						FVector2D SizeInScreen = ApplicationWindow->GetSizeInScreen();
						NormalizedLocation = FVector2D(Pos / SizeInScreen);
					}
				}
				else if (TSharedPtr<FIntRect> ScreenRectPtr = TargetScreenRect.Pin())
				{
					FIntRect ScreenRect = *ScreenRectPtr;
					FIntPoint SizeInScreen = ScreenRect.Max - ScreenRect.Min;
					NormalizedLocation = FocusedWidget->GetCachedGeometry().GetAbsolutePosition() / SizeInScreen;
				}

				NormalizedLocation *= uint16_MAX;
				// ConvertToNormalizedScreenLocation(Pos, NormalizedLocation);
				JsonObject->SetNumberField(TEXT("x"), static_cast<uint16>(NormalizedLocation.X));
				JsonObject->SetNumberField(TEXT("y"), static_cast<uint16>(NormalizedLocation.Y));

				FText TextboxContents;
				if (FocusedWidget->GetType() == TEXT("SEditableText"))
				{
					SEditableText* TextBox = static_cast<SEditableText*>(FocusedWidget.Get());
					TextboxContents = TextBox->GetText();
				}
				else if (FocusedWidget->GetType() == TEXT("SMultiLineEditableText"))
				{
					SMultiLineEditableText* TextBox = static_cast<SMultiLineEditableText*>(FocusedWidget.Get());
					TextboxContents = TextBox->GetText();
				}

				JsonObject->SetStringField(TEXT("contents"), TextboxContents.ToString());
			}

			FString Descriptor;
			TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Descriptor);
			FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);

			FBufferArchive Buffer;
			Buffer << Descriptor;
			TArray<uint8> Data(Buffer.GetData(), Buffer.Num());
			// Specific implementation for this method is handled per streamer
			OnSendMessage.Broadcast("Command", FMemoryReader(Data));
		});
	}

	FWidgetPath FPixelStreamingInputHandler::FindRoutingMessageWidget(const FVector2D& Location) const
	{
		if (TSharedPtr<SWindow> PlaybackWindowPinned = TargetWindow.Pin())
		{
			if (PlaybackWindowPinned->AcceptsInput())
			{
				bool bIgnoreEnabledStatus = false;
				TArray<FWidgetAndPointer> WidgetsAndCursors = PlaybackWindowPinned->GetHittestGrid().GetBubblePath(Location, FSlateApplication::Get().GetCursorRadius(), bIgnoreEnabledStatus);
				return FWidgetPath(MoveTemp(WidgetsAndCursors));
			}
		}
		return FWidgetPath();
	}
} // namespace UE::PixelStreamingInput
