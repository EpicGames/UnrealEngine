// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SWindow.h"
#include "InputCoreTypes.h"
#include "ApplicationWrapper.h"
#include "IPixelStreamingInputHandler.h"
#include "XRMotionControllerBase.h"
#include "HeadMountedDisplayTypes.h"
#include "PixelStreamingHMDEnums.h"
#include "PixelStreamingInputConversion.h"

namespace UE::PixelStreamingInput
{
	class FPixelStreamingInputHandler : public IPixelStreamingInputHandler, public FXRMotionControllerBase
	{
	public:
		FPixelStreamingInputHandler(TSharedPtr<FPixelStreamingApplicationWrapper> InApplicationWrapper, const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler);

		virtual ~FPixelStreamingInputHandler();

		virtual void Tick(float DeltaTime) override;

		/** Poll for controller state and send events if needed */
		virtual void SendControllerEvents() override {};

		/** Set which MessageHandler will route input  */
		virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler) override;

		/** Register a custom function to execute when command JSON is received. */
		virtual void SetCommandHandler(const FString& CommandName, const CommandHandlerFn& Handler) override;

		virtual void SetElevatedCheck(const TFunction<bool(FString)>& CheckFn) override;
		virtual bool IsElevated(const FString& Id) override;

		/** Exec handler to allow console commands to be passed through for debugging */
		virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
		/**
		 * IInputInterface pass through functions
		 */
		virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
		virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values) override;
		virtual void OnMessage(FString SourceId, TArray<uint8> Buffer) override;
		virtual void SetTargetWindow(TWeakPtr<SWindow> InWindow) override;
		virtual TWeakPtr<SWindow> GetTargetWindow() override;
		virtual void SetTargetViewport(TWeakPtr<SViewport> InViewport) override;
		virtual TWeakPtr<SViewport> GetTargetViewport() override;
		virtual void SetTargetScreenRect(TWeakPtr<FIntRect> InScreenRect) override;
		virtual TWeakPtr<FIntRect> GetTargetScreenRect() override;
		virtual bool IsFakingTouchEvents() const override { return bFakingTouchEvents; }
		virtual void RegisterMessageHandler(const FString& MessageType, const MessageHandlerFn& Handler) override;
		virtual MessageHandlerFn FindMessageHandler(const FString& MessageType) override;
		virtual void SetInputType(EPixelStreamingInputType InInputType) override { InputType = InInputType; };
		// IMotionController Interface
		virtual FName GetMotionControllerDeviceTypeName() const override;
		virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const override;
		virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const override;
		virtual void EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const override;
		// End IMotionController Interface

	protected:
		
		/**
		 * Key press handling
		 */
		virtual void HandleOnKeyChar(FMemoryReader Ar);
		virtual void HandleOnKeyDown(FMemoryReader Ar);
		virtual void HandleOnKeyUp(FMemoryReader Ar);
		/**
		 * Touch handling
		 */
		virtual void HandleOnTouchStarted(FMemoryReader Ar);
		virtual void HandleOnTouchMoved(FMemoryReader Ar);
		virtual void HandleOnTouchEnded(FMemoryReader Ar);
		/**
		 * Controller handling
		 */
		virtual void HandleOnControllerConnected(FMemoryReader Ar);
		virtual void HandleOnControllerAnalog(FMemoryReader Ar);
		virtual void HandleOnControllerButtonPressed(FMemoryReader Ar);
		virtual void HandleOnControllerButtonReleased(FMemoryReader Ar);
		virtual void HandleOnControllerDisconnected(FMemoryReader Ar);
		/**
		 * Mouse handling
		 */
		virtual void HandleOnMouseEnter(FMemoryReader Ar);
		virtual void HandleOnMouseLeave(FMemoryReader Ar);
		virtual void HandleOnMouseUp(FMemoryReader Ar);
		virtual void HandleOnMouseDown(FMemoryReader Ar);
		virtual void HandleOnMouseMove(FMemoryReader Ar);
		virtual void HandleOnMouseWheel(FMemoryReader Ar);
		virtual void HandleOnMouseDoubleClick(FMemoryReader Ar);
		/**
		 * XR handling
		 */
		virtual void HandleOnXREyeViews(FMemoryReader Ar);
		virtual void HandleOnXRHMDTransform(FMemoryReader Ar);
		virtual void HandleOnXRControllerTransform(FMemoryReader Ar);
		virtual void HandleOnXRButtonPressed(FMemoryReader Ar);
		virtual void HandleOnXRButtonTouched(FMemoryReader Ar);
		virtual void HandleOnXRButtonTouchReleased(FMemoryReader Ar);
		virtual void HandleOnXRButtonReleased(FMemoryReader Ar);
		virtual void HandleOnXRAnalog(FMemoryReader Ar);
		virtual void HandleOnXRSystem(FMemoryReader Ar);
		/**
		 * Command handling
		 */
		virtual void HandleOnCommand(FString SourceId, FMemoryReader Ar);
		/**
		 * UI Interaction handling
		 */
		virtual void HandleUIInteraction(FMemoryReader Ar);
		/**
		 * Textbox Entry handling
		 */
		virtual void HandleOnTextboxEntry(FMemoryReader Ar);

		/**
		 * Populate default command handlers for data channel messages sent with "{ type: "Command" }".
		 */
		void PopulateDefaultCommandHandlers();

		/**
		 * Extract 4x4 WebXR ordered matrix and convert to FMatrix.
		*/
		FMatrix ExtractWebXRMatrix(FMemoryReader& Ar);

		/**
		 * Converts the 'Y up' 'right handed' WebXR coordinate system transform to Unreal's 'Z up'
		 * 'left handed' coordinate system. Note: Ignores scale.
		 * Assumes WebXR conforms to the following: (https://developer.mozilla.org/en-US/docs/Web/API/WebXR_Device_API/Geometry)
		 * @return A 4x4 z-up transform matrix for use with UE.
		*/
		FTransform WebXRMatrixToUETransform(FMatrix Mat);

		FIntPoint ConvertFromNormalizedScreenLocation(const FVector2D& ScreenLocation, bool bIncludeOffset = true);
		FWidgetPath FindRoutingMessageWidget(const FVector2D& Location) const;
		FKey TranslateMouseButtonToKey(const EMouseButtons::Type Button);

		struct FCachedTouchEvent
		{
			FVector2D Location;
			float Force;
			int32 ControllerIndex;
		};

		struct FAnalogValue
		{
			/* The actual analog value from the controller axis, typical 0.0..1.0 */
			double Value;
			/* If value is non-zero then keep applying this analog values across frames.
			 * This is useful for trigger axis inputs where if a value is not transmitted
			 * UE will assume a gap in input means a full trigger press (which is not accurate if we were still pressing).
			 */
			bool bKeepUnlessZero = false;
			/* Has this key event already been fired once? */
			bool bIsRepeat = false;
		};

		// Keep a cache of the last touch events as we need to fire Touch Moved every frame while touch is down
		TMap<int32, FCachedTouchEvent> CachedTouchEvents;

		using FKeyId = uint8;

		/**
		 * If more values are received in a single tick (e.g. could be temp network issue),
		 * then we only forward the latest value.
		 * 
		 * Reason: The input system seems to expect at most one raw analog value per FKey per Tick.
		 * If this is not done, the input system can get stuck on non-zero input value even if the user has
		 * already stopped moving the analog stick. It would stay stuck until the next time the user moves the stick.
		 * 
		 * The values arrive in the order of recording: that means once the player releases the analog,
		 * the last analog value would be 0.
		 */
		TMap<FInputDeviceId, TMap<FKey*, FAnalogValue>> AnalogEventsReceivedThisTick;

		/** Forwards the latest analog input received for each key this tick. */
		void ProcessLatestAnalogInputFromThisTick();

		// Track which touch events we processed this frame so we can avoid re-processing them
		TSet<int32> TouchIndicesProcessedThisFrame;

		// Sends Touch Moved events for any touch index which is currently down but wasn't already updated this frame
		void BroadcastActiveTouchMoveEvents();

		void FindFocusedWidget();
		bool FilterKey(const FKey& Key);

		struct FMessage
		{
			FString SourceId;
			TFunction<void(FString, FMemoryReader)>* Handler;
			TArray<uint8> Data;
		};

		TWeakPtr<SWindow> TargetWindow;
		TWeakPtr<SViewport> TargetViewport;
		TWeakPtr<FIntPoint> TargetScreenSize; // Deprecated functionality but remaining until it can be removed
		TWeakPtr<FIntRect> TargetScreenRect;  // Manual size override used when we don't have a single window/viewport target
		uint8 NumActiveTouches;
		bool bIsMouseActive;
		TQueue<FMessage> Messages;
		EPixelStreamingInputType InputType = EPixelStreamingInputType::RouteToWindow;
		FVector2D LastTouchLocation = FVector2D(EForceInit::ForceInitToZero);
		TMap<uint8, MessageHandlerFn> DispatchTable;

		/** Reference to the message handler which events should be passed to. */
		TSharedPtr<FGenericApplicationMessageHandler> MessageHandler;

		/** For convenience, we keep a reference to the application wrapper owned by the input channel */
		TSharedPtr<FPixelStreamingApplicationWrapper> PixelStreamerApplicationWrapper;

		/**
		 * Is the application faking touch events by dragging the mouse along
		 * the canvas? If so then we must put the browser canvas in a special
		 * state to replicate the behavior of the application.
		 */
		bool bFakingTouchEvents;

		/*
		 * Padding for string parsing when handling messages.
		 * 1 character for the actual message and then
		 * 2 characters for the length which are skipped
		 */
		const size_t MessageHeaderOffset = 1;

		struct FPixelStreamingXRController
		{
		public:
			FTransform Transform;
			EControllerHand Handedness;
		};

		TMap<EControllerHand, FPixelStreamingXRController> XRControllers;

		/**
		 * A map of named commands we respond to when we receive a datachannel message of type "command".
		 * Key = command name (e.g "Encoder.MaxQP")
		 * Value = The command handler lambda function whose parameters are as follows:
		 *  FString - the source id of the user who sent the message
		 *  FString - the descriptor (e.g. the full json payload of the command message)
		 *  FString - the parsed value of the command, e.g. if key was "Encoder.MaxQP" and descriptor was { type: "Command", "Encoder.MaxQP": 51 }, then parsed value is "51".
		 */
		TMap<FString, CommandHandlerFn> CommandHandlers;

		TFunction<bool(FString)> ElevatedCheck;

	private:
		float uint16_MAX = (float)UINT16_MAX;
		float int16_MAX = (float)SHRT_MAX;
	};
} // namespace UE::PixelStreamingInput