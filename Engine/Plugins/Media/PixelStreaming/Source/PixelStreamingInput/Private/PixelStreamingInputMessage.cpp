// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputMessage.h"
#include "PixelStreamingInputEnums.h"

uint8 FPixelStreamingInputMessage::CurrentId = 0;

FPixelStreamingInputMessage::FPixelStreamingInputMessage()
	: Id(++CurrentId)
	, Structure({})
{
}

FPixelStreamingInputMessage::FPixelStreamingInputMessage(uint8 InId)
	: Id(InId)
	, Structure({})
{
	// We set id to 255 for the Protocol message type.
	// We don't want to set the CurrentId to this value otherwise we will have no room for custom messages
	if (InId > CurrentId && InId != 255)
	{
		CurrentId = InId; // Ensure that any new message id starts after the highest specified Id
	}
}

FPixelStreamingInputMessage::FPixelStreamingInputMessage(TArray<EPixelStreamingMessageTypes> InStructure)
	: Id(++CurrentId)
	, Structure(InStructure)
{
}

FPixelStreamingInputMessage::FPixelStreamingInputMessage(uint8 InId, TArray<EPixelStreamingMessageTypes> InStructure)
	: Id(InId)
	, Structure(InStructure)
{
	// We set id to 255 for the Protocol message type.
	// We don't want to set the CurrentId to this value otherwise we will have no room for custom messages
	if (InId > CurrentId && InId != 255)
	{
		CurrentId = InId; // Ensure that any new message id starts after the highest specified Id
	}
}