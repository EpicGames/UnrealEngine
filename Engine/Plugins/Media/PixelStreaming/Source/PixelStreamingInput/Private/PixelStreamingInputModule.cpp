// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputModule.h"
#include "IPixelStreamingHMDModule.h"
#include "Framework/Application/SlateApplication.h"
#include "ApplicationWrapper.h"
#include "Settings.h"
#include "PixelStreamingInputHandler.h"

namespace UE::PixelStreamingInput
{
	typedef EPixelStreamingMessageTypes EType;

	void FPixelStreamingInputModule::StartupModule()
	{
		if (!FSlateApplication::IsInitialized())
		{
			return;
		}

		PopulateProtocol();

		InputDevice = FPixelStreamingInputDevice::GetInputDevice();

		Settings::InitialiseSettings();

		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	void FPixelStreamingInputModule::ShutdownModule()
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	}

	TSharedPtr<IPixelStreamingInputHandler> FPixelStreamingInputModule::CreateInputHandler()
	{
		TSharedPtr<FPixelStreamingApplicationWrapper> PixelStreamerApplicationWrapper = MakeShareable(new FPixelStreamingApplicationWrapper(FSlateApplication::Get().GetPlatformApplication()));
		TSharedPtr<FGenericApplicationMessageHandler> BaseHandler = FSlateApplication::Get().GetPlatformApplication()->GetMessageHandler();
		TSharedPtr<IPixelStreamingInputHandler> InputHandler = MakeShared<FPixelStreamingInputHandler>(PixelStreamerApplicationWrapper, BaseHandler);

		// Add this input handler to the input device's array of handlers. This ensures that it's ticked
		InputDevice->AddInputHandler(InputHandler);

		return InputHandler;
	}

	TSharedPtr<IInputDevice> FPixelStreamingInputModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	{
		return InputDevice;
	}

	void FPixelStreamingInputModule::PopulateProtocol()
	{
		// Old EToStreamerMsg Commands
		/*
		 * Control Messages.
		 */
		// Simple command with no payload
		// Note, we only specify the ID when creating these messages to preserve backwards compatability
		// when adding your own message type, you can simply do FPixelStreamingInputProtocol::Direction.Add("XXX");
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("IFrameRequest", FPixelStreamingInputMessage(0));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("RequestQualityControl", FPixelStreamingInputMessage(1));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("FpsRequest", FPixelStreamingInputMessage(2));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("AverageBitrateRequest", FPixelStreamingInputMessage(3));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("StartStreaming", FPixelStreamingInputMessage(4));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("StopStreaming", FPixelStreamingInputMessage(5));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("LatencyTest", FPixelStreamingInputMessage(6, { EType::String }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("RequestInitialSettings", FPixelStreamingInputMessage(7));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("TestEcho", FPixelStreamingInputMessage(8, { EType::String }));

		/*
		 * Input Messages.
		 */
		// Generic Input Messages.
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("UIInteraction", FPixelStreamingInputMessage(50, { EType::String }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("Command", FPixelStreamingInputMessage(51, { EType::String }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("TextboxEntry", FPixelStreamingInputMessage(52, { EType::String }));

		// Keyboard Input Message.
		// Complex command with payload, therefore we specify the length of the payload (bytes) as well as the structure of the payload
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("KeyDown", FPixelStreamingInputMessage(60, { EType::Uint8, EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("KeyUp", FPixelStreamingInputMessage(61, { EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("KeyPress", FPixelStreamingInputMessage(62, { EType::Uint16 }));

		// Mouse Input Messages.
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("MouseEnter", FPixelStreamingInputMessage(70));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("MouseLeave", FPixelStreamingInputMessage(71));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("MouseDown", FPixelStreamingInputMessage(72, { EType::Uint8, EType::Uint16, EType::Uint16 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("MouseUp", FPixelStreamingInputMessage(73, { EType::Uint8, EType::Uint16, EType::Uint16 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("MouseMove", FPixelStreamingInputMessage(74, { EType::Uint16, EType::Uint16, EType::Uint16, EType::Uint16 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("MouseWheel", FPixelStreamingInputMessage(75, { EType::Int16, EType::Uint16, EType::Uint16 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("MouseDouble", FPixelStreamingInputMessage(76, { EType::Uint8, EType::Uint16, EType::Uint16 }));

		// Touch Input Messages.
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("TouchStart", FPixelStreamingInputMessage(80, { EType::Uint8, EType::Uint16, EType::Uint16, EType::Uint8, EType::Uint8, EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("TouchEnd", FPixelStreamingInputMessage(81, { EType::Uint8, EType::Uint16, EType::Uint16, EType::Uint8, EType::Uint8, EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("TouchMove", FPixelStreamingInputMessage(82, { EType::Uint8, EType::Uint16, EType::Uint16, EType::Uint8, EType::Uint8, EType::Uint8 }));

		// Gamepad Input Messages.
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("GamepadButtonPressed", FPixelStreamingInputMessage(90, { EType::Uint8, EType::Uint8, EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("GamepadButtonReleased", FPixelStreamingInputMessage(91, { EType::Uint8, EType::Uint8, EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("GamepadAnalog", FPixelStreamingInputMessage(92, { EType::Uint8, EType::Uint8, EType::Double }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("GamepadConnected", FPixelStreamingInputMessage(93));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("GamepadDisconnected", FPixelStreamingInputMessage(94, { EType::Uint8 }));

		// XR Input Messages.
		// clang-format off
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XREyeViews", FPixelStreamingInputMessage(109,		{	// Left eye transform (4x4 matrix)
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													// Left eye perspective projection (4x4 matrix)
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													// Right eye transform (4x4 matrix)
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													// Right eye perspective projection (4x4 matrix)
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													// HMD 4x4 Transform
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																												}));

		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRHMDTransform", FPixelStreamingInputMessage(110, {	// 4x4 Transform
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																													EType::Float, EType::Float, EType::Float, EType::Float,
																												}));

		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRControllerTransform", FPixelStreamingInputMessage(111, {// 4x4 Transform
																														EType::Float, EType::Float, EType::Float, EType::Float, 
																														EType::Float, EType::Float, EType::Float, EType::Float, 
																														EType::Float, EType::Float, EType::Float, EType::Float, 
																														EType::Float, EType::Float, EType::Float, EType::Float,
																														// Handedness (L, R, Any)
																														EType::Uint8 
																														}));

		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRButtonPressed", FPixelStreamingInputMessage(112,// Handedness,  ButtonIdx,    IsRepeat,     PressedAmount
																												{ EType::Uint8, EType::Uint8, EType::Uint8, EType::Double }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRButtonTouched", FPixelStreamingInputMessage(113,// Handedness,  ButtonIdx,    IsRepeat
																												{ EType::Uint8, EType::Uint8, EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRButtonReleased", FPixelStreamingInputMessage(114,// Handedness, ButtonIdx,    IsRepeat
																												{ EType::Uint8, EType::Uint8, EType::Uint8 }));

		// clang-format on
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRAnalog", FPixelStreamingInputMessage(115, { EType::Uint8, EType::Uint8, EType::Double }));

		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRSystem", FPixelStreamingInputMessage(116, { EType::Uint8 }));
		FPixelStreamingInputProtocol::ToStreamerProtocol.Add("XRButtonTouchReleased", FPixelStreamingInputMessage(117,// Handedness,  ButtonIdx,    IsRepeat
																												{ EType::Uint8, EType::Uint8, EType::Uint8 }));

		// Old EToPlayerMsg commands
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("QualityControlOwnership", FPixelStreamingInputMessage(0));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("Response", FPixelStreamingInputMessage(1));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("Command", FPixelStreamingInputMessage(2));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("FreezeFrame", FPixelStreamingInputMessage(3));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("UnfreezeFrame", FPixelStreamingInputMessage(4));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("VideoEncoderAvgQP", FPixelStreamingInputMessage(5));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("LatencyTest", FPixelStreamingInputMessage(6));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("InitialSettings", FPixelStreamingInputMessage(7));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("FileExtension", FPixelStreamingInputMessage(8));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("FileMimeType", FPixelStreamingInputMessage(9));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("FileContents", FPixelStreamingInputMessage(10));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("TestEcho", FPixelStreamingInputMessage(11));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("InputControlOwnership", FPixelStreamingInputMessage(12));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("GamepadResponse", FPixelStreamingInputMessage(13));
		FPixelStreamingInputProtocol::FromStreamerProtocol.Add("Protocol", FPixelStreamingInputMessage(255));
	}
} // namespace UE::PixelStreamingInput

IMPLEMENT_MODULE(UE::PixelStreamingInput::FPixelStreamingInputModule, PixelStreamingInput)