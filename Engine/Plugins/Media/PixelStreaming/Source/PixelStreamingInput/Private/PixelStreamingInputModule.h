// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreamingInputModule.h"
#include "PixelStreamingInputDevice.h"

namespace UE::PixelStreamingInput
{
	class FPixelStreamingInputModule : public IPixelStreamingInputModule
	{
	public:
		virtual TSharedPtr<IPixelStreamingInputHandler> CreateInputHandler() override;

	private:
		/** IModuleInterface implementation */
		void StartupModule() override;
		void ShutdownModule() override;
		/** End IModuleInterface implementation */

		/** IInputDeviceModule implementation */
		virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
		/** End IInputDeviceModule implementation */

		TSharedPtr<FPixelStreamingInputDevice> InputDevice;

		void PopulateProtocol();
	};
} // namespace UE::PixelStreamingInput