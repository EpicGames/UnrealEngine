// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputProtocol.h"
#include "PixelStreamingInputMessage.h"
#include "Dom/JsonValue.h"

FInputProtocolMap FPixelStreamingInputProtocol::ToStreamerProtocol;
FInputProtocolMap FPixelStreamingInputProtocol::FromStreamerProtocol;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<FJsonObject> FPixelStreamingInputProtocol::ToJson(EPixelStreamingMessageDirection Direction)
{
	TSharedPtr<FJsonObject> ProtocolJson = MakeShareable(new FJsonObject());
	FInputProtocolMap MessageProtocol =
		(Direction == EPixelStreamingMessageDirection::ToStreamer)
		? FPixelStreamingInputProtocol::ToStreamerProtocol
		: FPixelStreamingInputProtocol::FromStreamerProtocol;

	ProtocolJson->SetField(TEXT("Direction"), MakeShared<FJsonValueNumber>(static_cast<uint8>(Direction)));
	MessageProtocol.Apply([ProtocolJson](FString Key, FPixelStreamingInputMessage Value) {
		TSharedPtr<FJsonObject> MessageJson = MakeShareable(new FJsonObject());

		MessageJson->SetField(TEXT("id"), MakeShared<FJsonValueNumber>(Value.GetID()));

        TArray<EPixelStreamingMessageTypes> Structure = Value.GetStructure();
		TArray<TSharedPtr<FJsonValue>> StructureJson;
		for (auto It = Structure.CreateIterator(); It; ++It)
		{
			FString Text;
			switch (*It)
			{
				case EPixelStreamingMessageTypes::Uint8:
					Text = "uint8";
					break;
				case EPixelStreamingMessageTypes::Uint16:
					Text = "uint16";
					break;
				case EPixelStreamingMessageTypes::Int16:
					Text = "int16";
					break;
				case EPixelStreamingMessageTypes::Float:
					Text = "float";
					break;
				case EPixelStreamingMessageTypes::Double:
					Text = "double";
					break;
            	case EPixelStreamingMessageTypes::String:
					Text = "string";
					break;
				case EPixelStreamingMessageTypes::Undefined:
				case EPixelStreamingMessageTypes::Variable:
				default:
					Text = "";
			}
			TSharedRef<FJsonValueString> JsonValue = MakeShareable(new FJsonValueString(*Text));
			StructureJson.Add(JsonValue);
		}
		MessageJson->SetArrayField(TEXT("structure"), StructureJson);

		ProtocolJson->SetField(*Key, MakeShared<FJsonValueObject>(MessageJson));
	});

	return ProtocolJson;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS