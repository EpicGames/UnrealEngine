// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingInputProtocolMap.h"
#include "IPixelStreamingInputModule.h"

FPixelStreamingInputMessage& FInputProtocolMap::Add(FString Key, const FPixelStreamingInputMessage& Value)
{
	IPixelStreamingInputModule::Get().OnProtocolUpdated.Broadcast();
	return InnerMap.Add(Key, Value);
}

int FInputProtocolMap::Remove(FString Key)
{
	IPixelStreamingInputModule::Get().OnProtocolUpdated.Broadcast();
	return InnerMap.Remove(Key);
}

FPixelStreamingInputMessage& FInputProtocolMap::GetOrAdd(FString Key)
{
	if (InnerMap.Contains(Key))
	{
		return InnerMap[Key];
	}

	IPixelStreamingInputModule::Get().OnProtocolUpdated.Broadcast();
	return InnerMap.Add(Key);
}

FPixelStreamingInputMessage* FInputProtocolMap::Find(FString Key)
{
	return InnerMap.Find(Key);
}

const FPixelStreamingInputMessage* FInputProtocolMap::Find(FString Key) const
{
	return InnerMap.Find(Key);
}

void FInputProtocolMap::Clear()
{
	IPixelStreamingInputModule::Get().OnProtocolUpdated.Broadcast();
	InnerMap.Empty();
}

bool FInputProtocolMap::IsEmpty() const
{
	return InnerMap.IsEmpty();
}

void FInputProtocolMap::Apply(const TFunction<void(FString, FPixelStreamingInputMessage)>& Visitor)
{
	for (auto&& [Key, Value] : InnerMap)
	{
		Visitor(Key, Value);
	}
}