// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings.h"
#include "Misc/DefaultValueHelper.h"

namespace UE::PixelStreamingInput::Settings
{
	TAutoConsoleVariable<bool> CVarPixelStreamingInputAllowConsoleCommands(
		TEXT("PixelStreaming.AllowPixelStreamingCommands"),
		false,
		TEXT("If true browser can send consoleCommand payloads that execute in UE's console."),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarPixelStreamingInputKeyFilter(
		TEXT("PixelStreaming.KeyFilter"),
		"",
		TEXT("Comma separated list of keys to ignore from streaming clients."),
		ECVF_Default);

	TArray<FKey> FilteredKeys;

	void CommandLineParseOption(const TCHAR* Match, TAutoConsoleVariable<bool>& CVar)
	{
		FString ValueMatch(Match);
		ValueMatch.Append(TEXT("="));
		FString Value;
		if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value))
		{
			if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
			{
				CVar->Set(true, ECVF_SetByCommandline);
			}
			else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
			{
				CVar->Set(false, ECVF_SetByCommandline);
			}
		}
		else if (FParse::Param(FCommandLine::Get(), Match))
		{
			CVar->Set(true, ECVF_SetByCommandline);
		}
	}

	template <typename T>
	void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<T>& CVar)
	{
		T Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value))
			CVar->Set(Value, ECVF_SetByCommandline);
	};

	void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<FString>& CVar, bool bStopOnSeparator)
	{
		FString Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value, bStopOnSeparator))
			CVar->Set(*Value, ECVF_SetByCommandline);
	};

	void OnFilteredKeysChanged(IConsoleVariable* Var)
	{
		FString CommaList = Var->GetString();
		TArray<FString> KeyStringArray;
		CommaList.ParseIntoArray(KeyStringArray, TEXT(","), true);
		FilteredKeys.Empty();
		for (auto&& KeyString : KeyStringArray)
		{
			FilteredKeys.Add(FKey(*KeyString));
		}
	}

	void InitialiseSettings()
	{
		CVarPixelStreamingInputKeyFilter.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnFilteredKeysChanged));

		// Values parse from commands line
		CommandLineParseValue(TEXT("PixelStreamingKeyFilter="), CVarPixelStreamingInputKeyFilter);

		// Options parse (if these exist they are set to true)
		CommandLineParseOption(TEXT("AllowPixelStreamingCommands"), CVarPixelStreamingInputAllowConsoleCommands);
	}
} // namespace UE::PixelStreamingInput::Settings