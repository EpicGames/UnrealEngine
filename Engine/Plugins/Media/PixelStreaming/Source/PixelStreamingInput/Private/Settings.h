// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "InputCoreTypes.h"

namespace UE::PixelStreamingInput::Settings
{
	// Begin CVars
	extern TAutoConsoleVariable<bool> CVarPixelStreamingInputAllowConsoleCommands;
	extern TAutoConsoleVariable<FString> CVarPixelStreamingInputKeyFilter;
	extern TArray<FKey> FilteredKeys;
	// End CVars

	extern void InitialiseSettings();
	extern void CommandLineParseOption();
	template <typename T>
	extern void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<T>& CVar);
	extern void CommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<FString>& CVar, bool bStopOnSeparator = false);
} // namespace UE::PixelStreamingInput::Settings
