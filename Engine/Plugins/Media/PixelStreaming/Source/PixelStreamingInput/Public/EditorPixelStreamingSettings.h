// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "EditorPixelStreamingSettings.generated.h"

// Config loaded/saved to an .ini file.
// It is also exposed through the plugin settings page in editor.
UCLASS(config = PixelStreaming, defaultconfig, meta = (DisplayName = "PixelStreaming"))
class PIXELSTREAMINGINPUT_API UPixelStreamingSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		ConfigRestartRequired = true,
		DisplayName = "Enable Pixel Streaming Toolbar",
		ToolTip = "Adds the Pixel Streaming toolbar to the main viewport toolbar."
		))
	bool EnablePixelStreamingToolbar = true;
	
	/**
	 * Pixel streaming always requires various software cursors so they will be
	 * visible in the video stream sent to the browser to allow the user to
	 * click and interact with UI elements.
	 */
	UPROPERTY(config, EditAnywhere, Category = PixelStreaming)
	FSoftClassPath DefaultCursorClassName;
	UPROPERTY(config, EditAnywhere, Category = PixelStreaming)
	FSoftClassPath TextEditBeamCursorClassName;

	/**
	 * Pixel Streaming can have a server-side cursor (where the cursor itself
	 * is shown as part of the video), or a client-side cursor (where the cursor
	 * is shown by the browser). In the latter case we need to turn the UE4
	 * cursor invisible.
	 */
	UPROPERTY(config, EditAnywhere, Category = PixelStreaming)
	FSoftClassPath HiddenCursorClassName;

	/**
	 * Pixel Streaming may be running on a machine which has no physical mouse
	 * attached, and yet the browser is sending mouse positions. As such, we
	 * fake the presence of a mouse.
	 */
	UPROPERTY(config, EditAnywhere, Category = PixelStreaming)
	bool bMouseAlwaysAttached = true;

	// Begin UDeveloperSettings Interface
	virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
#endif
	// END UDeveloperSettings Interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};