// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformMisc.h"
#include "IInputDevice.h"
#include "Widgets/SViewport.h"
#include "Widgets/SWindow.h"
#include "Templates/SharedPointer.h"
#include "Serialization/MemoryReader.h"
#include "PixelStreamingInputEnums.h"

/**
 * The IPixelStreamingInputHandler, used to handle input from a remote peer and pass it to UE accordingly. Setting the target viewport allows for
 * scaling of input from browser to application, and setting the target window ensure that if windows are tiled (eg editor)
 * that the streamed input only affect the target window.
 */
class PIXELSTREAMINGINPUT_API IPixelStreamingInputHandler : public IInputDevice
{
public:
	/**
	 * @brief Handle the message from the WebRTC data channel.
	 * @param SourceId A source ID for this message
	 * @param Buffer The data channel message
	 */
	virtual void OnMessage(FString SourceId, TArray<uint8> Buffer) = 0;

	/**
	 * @brief Set the viewport this input device is associated with.
	 * @param InTargetViewport The viewport to set
	 */
	virtual void SetTargetViewport(TWeakPtr<SViewport> InTargetViewport) = 0;

	/**
	 * @brief Get the viewport this input device is associated with.
	 * @return The viewport this input device is associated with
	 */
	virtual TWeakPtr<SViewport> GetTargetViewport() = 0;

	/**
	 * @brief Set the viewport this input device is associated with.
	 * @param InTargetWindow The viewport to set
	 */
	virtual void SetTargetWindow(TWeakPtr<SWindow> InTargetWindow) = 0;

	/**
	 * @brief Get the viewport this input device is associated with.
	 * @return The viewport this input device is associated with
	 */
	virtual TWeakPtr<SWindow> GetTargetWindow() = 0;

	/**
	 * @brief Set the target screen rectangle for this streamer. This is used to when the streamer doesn't have a singular target window / viewport
	 * and as such we just use the manual scale
	 * @param InTargetScreenRect The target screen rectangle
	 */
	virtual void SetTargetScreenRect(TWeakPtr<FIntRect> InTargetScreenRect) = 0;

	/**
	 * @brief Get the target screen rectangle for this streamer
	 * @return The target screen rectangle
	 */
	virtual TWeakPtr<FIntRect> GetTargetScreenRect() = 0;

	/**
	 * @brief Set whether the input devices is faking touch events using keyboard and mouse this can be useful for debugging.
	 * @return true
	 * @return false
	 */
	virtual bool IsFakingTouchEvents() const = 0;

	/**
	 * @brief The callback signature for handling a command sent to the data channel.
	 *
	 * @param SourceId The source id of the sender of this message.
	 * @param Message The full message in the form of a FMemoryReader.
	 */
	using MessageHandlerFn = TFunction<void(FString SourceId, FMemoryReader Message)>;

	/**
	 * @brief Register a function to be called whenever the specified message type is received.
	 *
	 * @param MessageType The human readable identifier for the message
	 * @param Handler The function called when this message type is received. This handler must take a single parameter (an FMemoryReader) and have a return type of void
	 */
	virtual void RegisterMessageHandler(const FString& MessageType, const MessageHandlerFn& Handler) = 0;

	/**
	 * @brief The callback signature for handling a command sent to the data channel.
	 *
	 * @param SourceId The source id of the sender of this message.
	 * @param Descriptor The full descriptor of the commaand.
	 * @param CommandString The relevant string parameters for the command.
	 */
	using CommandHandlerFn = TFunction<void(FString SourceId, FString Descriptor, FString CommandString)>;

	/**
	 * @brief Register a custom function to execute when command JSON is received over the data channel: "{ type: "Command", YourCommand: YourCommandValue }".
	 * Note: You can also override the default Pixel Streaming command handlers by setting handlers with the same name as those already used, e.g. "Stat.FPS".
	 */
	virtual void SetCommandHandler(const FString& CommandName, const CommandHandlerFn& Handler) = 0;

	/**
	 * @brief Some behaviours might want to be limited to a specific source or group of sources. This method sets a check function to test of a given source id is "elevated".
	 * 
	 * @param CheckFn A callback that takes a SourceId and returns true if the source id is an elevated user.
	 */
	virtual void SetElevatedCheck(const TFunction<bool(FString)>& CheckFn) = 0;

	/**
	 * @brief Checks whether the given id has elevated priviledges.
	 * 
	 * @return true The id is elevated.
	 * @return false The id is not elevated.
	 */
	virtual bool IsElevated(const FString& Id) = 0;

	/**
	 * @brief Find the function to be called whenever the specified message type is received.
	 *
	 * @param MessageType The human readable identifier for the message
	 * @return TFunction<void(FString, FMemoryReader)> The function called when this message type is received.
	 */
	virtual MessageHandlerFn FindMessageHandler(const FString& MessageType) = 0;

	/**
	 * @brief Set the input handlers type. This controls whether input is routed to widgets or windows
	 */
	virtual void SetInputType(EPixelStreamingInputType InputType) = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSendMessage, FString, FMemoryReader);
	FOnSendMessage OnSendMessage;
};