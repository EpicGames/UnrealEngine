// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "IInputDeviceModule.h"
#include "PixelStreamingInputProtocol.h"
#include "IPixelStreamingInputHandler.h"

/**
 * The public interface of the Pixel Streaming Input module.
 */
class PIXELSTREAMINGINPUT_API IPixelStreamingInputModule : public IInputDeviceModule
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreamingInputModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreamingInputModule>("PixelStreamingInput");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("PixelStreamingInput"); }

	/**
	 * @brief Create a Input Handler object
	 *
	 * @return TSharedPtr<IPixelStreamingInputHandler> the input handler for this streamer
	 */
	virtual TSharedPtr<IPixelStreamingInputHandler> CreateInputHandler() = 0;

	/**
	 * Attempts to create a new input device interface
	 *
	 * @return	Interface to the new input device, if we were able to successfully create one
	 */
	virtual TSharedPtr<class IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override = 0;

	DECLARE_MULTICAST_DELEGATE(FOnProtocolUpdated);
	FOnProtocolUpdated OnProtocolUpdated;
};
