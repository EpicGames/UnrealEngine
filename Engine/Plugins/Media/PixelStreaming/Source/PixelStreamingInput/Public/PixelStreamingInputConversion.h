// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "InputCoreTypes.h"
#include "PixelStreamingHMDEnums.h"
#include "PixelStreamingInputEnums.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

struct PIXELSTREAMINGINPUT_API FPixelStreamingInputConverter
{
public:
	static TMap<TTuple<EPixelStreamingXRSystem, EControllerHand, uint8, EPixelStreamingInputAction>, FKey> XRInputToFKey;
	static TMap<TTuple<uint8, EPixelStreamingInputAction>, FKey> GamepadInputToFKey;
};