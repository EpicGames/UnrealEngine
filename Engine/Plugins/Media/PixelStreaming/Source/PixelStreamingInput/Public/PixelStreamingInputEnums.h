// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * @brief The data types supported when sending messages across the data channel to/from peers
 *
 */
enum class EPixelStreamingMessageTypes
{
	Uint8,
	Uint16,
	Int16,
	Float,
	Double,
    Variable UE_DEPRECATED(5.3, "EPixelStreamingMessageTypes::Variable has been deprecated. Please use either String or Undefined instead"),
    String,
	Undefined
};

/**
 * @brief The message directions
 *
 */
enum class EPixelStreamingMessageDirection : uint8
{
	ToStreamer = 0,
	FromStreamer = 1
};

/**
 * @brief The possible actions pixel streaming supports. These actions help differentiate input received from the browser
 *
 */
enum class EPixelStreamingInputAction : uint8
{
	X = 0,
	Y = 1,
	Axis,
	Click,
	Touch,
	None,
};

/**
 * @brief The possible control schemes pixel streaming supports. RouteToWindow routes input at an application level. RouteToWidget routes input to a specific widget, ignoring the rest of the application.
 *
 */
enum class EPixelStreamingInputType : uint8
{
	RouteToWindow,
	RouteToWidget
};