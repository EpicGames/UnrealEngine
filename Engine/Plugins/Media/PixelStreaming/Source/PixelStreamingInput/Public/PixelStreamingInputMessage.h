// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingInputEnums.h"

#include "Containers/Array.h"
#include "HAL/Platform.h"

class PIXELSTREAMINGINPUT_API FPixelStreamingInputMessage
{
public:
	// Emtpy constructor. Sets ID to next available ID, and Structure to []
	FPixelStreamingInputMessage();

	// Constructor taking an ID. Sets Structure to []
	FPixelStreamingInputMessage(uint8 InId);

	// Constructor taking a Structure. Sets ID to next available ID
	FPixelStreamingInputMessage(TArray<EPixelStreamingMessageTypes> InStructure);

	// Constructor taking an ID and a Structure
	FPixelStreamingInputMessage(uint8 InId, TArray<EPixelStreamingMessageTypes> InStructure);

    UE_DEPRECATED(5.3, "ByteLength no longer exists on FPixelStreamingInputMessage.")
    uint8 GetByteLength() const { return 0; }
	uint8 GetID() const { return Id; }
	TArray<EPixelStreamingMessageTypes> GetStructure() const { return Structure; }

private:
	uint8 Id;
	TArray<EPixelStreamingMessageTypes> Structure;

	static uint8 CurrentId;
};