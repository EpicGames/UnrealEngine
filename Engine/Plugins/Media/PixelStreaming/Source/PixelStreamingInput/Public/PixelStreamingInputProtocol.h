// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Map.h"
#include "PixelStreamingInputMessage.h"
#include "PixelStreamingInputProtocolMap.h"
#include "Dom/JsonObject.h"

class FInputProtocolMap;

struct PIXELSTREAMINGINPUT_API FPixelStreamingInputProtocol
{
public:
	static FInputProtocolMap ToStreamerProtocol;
	static FInputProtocolMap FromStreamerProtocol;

	static TSharedPtr<FJsonObject> ToJson(EPixelStreamingMessageDirection Direction);
};