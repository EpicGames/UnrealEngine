// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/ScopeLock.h"
#include "PixelStreamingInputMessage.h"

/**
 * @brief An map type that broadcasts the OnProtocolUpdated whenever
 * it's inner map is updated
 */
class PIXELSTREAMINGINPUT_API FInputProtocolMap
{
public:
	FPixelStreamingInputMessage& Add(FString Key, const FPixelStreamingInputMessage& Value);
	int Remove(FString Key);
	FPixelStreamingInputMessage& GetOrAdd(FString Key);
	FPixelStreamingInputMessage* Find(FString Key);
	const FPixelStreamingInputMessage* Find(FString Key) const;

	void Clear();
	bool IsEmpty() const;

	void Apply(const TFunction<void(FString, FPixelStreamingInputMessage)>& Visitor);

private:
	TMap<FString, FPixelStreamingInputMessage> InnerMap;
};
