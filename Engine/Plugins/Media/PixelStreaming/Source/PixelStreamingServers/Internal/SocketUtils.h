// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"

namespace UE::PixelStreaming
{
	PIXELSTREAMINGSERVERS_API extern int GetNextAvailablePort(TOptional<int> StartingPort = FNullOpt(0));
} // namespace UE::PixelStreaming
