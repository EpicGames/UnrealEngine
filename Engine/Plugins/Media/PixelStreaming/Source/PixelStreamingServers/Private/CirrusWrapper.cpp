// Copyright Epic Games, Inc. All Rights Reserved.

#include "CirrusWrapper.h"
#include "PixelStreamingServersModule.h"
#include "ServerUtils.h"
#include "PixelStreamingServersLog.h"
#include "PixelStreamingServers.h"
#include "Misc/Paths.h"

namespace UE::PixelStreamingServers
{

	FString FCirrusWrapper::GetServerResourceDirectory()
	{
		return FString(TEXT("SignallingWebServer"));
	}

	void FCirrusWrapper::Stop()
	{
		UE_LOGF(LogPixelStreamingServers, Log, "Stopping Cirrus signalling server.");
		FMonitoredServerBase::Stop();
	}
	
	TSharedPtr<FMonitoredProcess> FCirrusWrapper::LaunchServerProcess(FLaunchArgs& InLaunchArgs, FString ServerAbsPath, TMap<EEndpoint, FURL>& OutEndPoints)
	{
		Utils::PopulateCirrusEndPoints(LaunchArgs, OutEndPoints);

		/* Launch the server process in-place. 
		* Note this has not handled waiting for server to become ready, 
		* that is server specific, and is therefore handled in `LaunchImpl` methods.
		*/
		bool bUseServerBinary = InLaunchArgs.ServerBinaryOverridePath.IsSet();

		TSharedPtr<FMonitoredProcess> CirrusProcess = Utils::LaunchChildProcess(
			ServerAbsPath, 
			LaunchArgs.ProcessArgs, 
			FPaths::GetPathLeaf(ServerAbsPath), 
			!bUseServerBinary);

		UE_LOGF(LogPixelStreamingServers, Log, "Cirrus signalling server running at: %ls", *OutEndPoints[EEndpoint::Signalling_Webserver].ToString(true));

		if (bPollUntilReady)
		{
			Probe = MakeUnique<FWebSocketProbe>(OutEndPoints[EEndpoint::Signalling_Streamer]);
		}

		return CirrusProcess;
	}

	bool FCirrusWrapper::TestConnection()
	{
		if (bIsReady)
		{
			return true;
		}
		else
		{
			bool bConnected = Probe->Probe();
			if (bConnected)
			{
				// Close the websocket connection so others can use it
				Probe.Reset();
				return true;
			}
			else
			{
				return false;
			}
		}
	}

} // UE::PixelStreamingServers