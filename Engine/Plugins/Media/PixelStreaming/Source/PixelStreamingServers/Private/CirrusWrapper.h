// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingServers.h"
#include "WebSocketProbe.h"
#include "MonitoredServerBase.h"

namespace UE::PixelStreamingServers
{
	/* The NodeJS Cirrus signalling server launched as a child process. */
	class FCirrusWrapper : public FMonitoredServerBase
	{
	public:
		FCirrusWrapper() = default;
		virtual ~FCirrusWrapper() = default;

	private:
		TUniquePtr<FWebSocketProbe> Probe;

	protected:
		/* Begin FMonitoredServerBase interface */
		void Stop() override;
		TSharedPtr<FMonitoredProcess> LaunchServerProcess(FLaunchArgs& InLaunchArgs, FString ServerAbsPath, TMap<EEndpoint, FURL>& OutEndPoints) override;
		bool TestConnection() override;
		FString GetServerResourceDirectory() override;
		/* End FMonitoredServerBase interface */
	};

} // UE::PixelStreamingServers
		