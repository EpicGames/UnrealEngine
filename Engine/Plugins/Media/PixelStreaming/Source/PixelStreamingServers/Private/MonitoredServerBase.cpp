// Copyright Epic Games, Inc. All Rights Reserved.

#include "MonitoredServerBase.h"
#include "PixelStreamingServersLog.h"
#include "HAL/PlatformFileManager.h"
#include "ServerUtils.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/CoreDelegates.h"

namespace UE::PixelStreamingServers
{

	FMonitoredServerBase::~FMonitoredServerBase()
	{
		Stop();
	}

	bool FMonitoredServerBase::FindServerAbsPath(FLaunchArgs& InLaunchArgs, FString& OutServerAbsPath)
	{
		bool bUseServerBinary = InLaunchArgs.ServerBinaryOverridePath.IsSet();

		if(!bUseServerBinary)
		{
			bool bFoundServer = Utils::GetDownloadedServer(OutServerAbsPath, GetServerResourceDirectory());
			if(bFoundServer)
			{
				return true;
			}
			UE_LOGF(LogPixelStreamingServers, Log, "Downloaded server not found at - %ls", *OutServerAbsPath);
		}
		else
		{
			OutServerAbsPath = InLaunchArgs.ServerBinaryOverridePath.GetValue();
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if(PlatformFile.FileExists(*OutServerAbsPath))
			{
				return true;
			}
			UE_LOGF(LogPixelStreamingServers, Log, "Server binary not found at - %ls", *OutServerAbsPath);
		}
		return false;
	}

	bool FMonitoredServerBase::LaunchImpl(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndpoints)
	{
		// Bind to OnEnginePreExit in case our child process is still running when engine is shutting down
		EngineShutdownHandle = FCoreDelegates::OnEnginePreExit.AddSP(this, &FMonitoredServerBase::Stop);

		bool bFoundServerPath = FindServerAbsPath(InLaunchArgs, ServerRootAbsPath);
		if(!bFoundServerPath)
		{
			return false;
		}

		ServerProcess = LaunchServerProcess(InLaunchArgs, ServerRootAbsPath, Endpoints);
		return ServerProcess.IsValid();
	}

	FString FMonitoredServerBase::GetPathOnDisk()
	{
		return ServerRootAbsPath;
	}

	void FMonitoredServerBase::Stop()
	{
		if (ServerProcess)
		{
			ServerProcess->Cancel();
			ServerProcess.Reset();
		}
		FCoreDelegates::OnEnginePreExit.Remove(EngineShutdownHandle);
	}

	void FMonitoredServerBase::GetNumStreamers(TFunction<void(uint16)> OnNumStreamersReceived)
	{
		// Todo (Luke.Bermingham): When the need arises for us to return the number of streamers for Cirrus
		// we should implement the following endpoints in Cirrus:
		// (1) health check
		// (2) statistic
		// We would then use the statistics HTTP endpoint to return number of streamers.
		unimplemented();
	}

} // UE::PixelStreamingServers
