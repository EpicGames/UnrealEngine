// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ServerBase.h"
#include "Templates/SharedPointer.h"
#include "Misc/MonitoredProcess.h"

namespace UE::PixelStreamingServers
{
	
	/**
	 *  Base class for all Pixel Streaming servers that are launched as child processes, e.g. calling some scripts or executables.
	 **/
	class FMonitoredServerBase : public FServerBase, public TSharedFromThis<FMonitoredServerBase>
	{
	public:
		virtual ~FMonitoredServerBase();

		/* Begin IServer */
		virtual void Stop() override;
		virtual FString GetPathOnDisk() override;
		virtual void GetNumStreamers(TFunction<void(uint16)> OnNumStreamersReceived) override;
		/* End IServer */

	protected:

		virtual bool LaunchImpl(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndpoints) override;
		virtual TSharedPtr<FMonitoredProcess> LaunchServerProcess(FLaunchArgs& InLaunchArgs, FString ServerAbsPath, TMap<EEndpoint, FURL>& OutEndpoints) = 0;

		/**
		 * Implemented by derived types. Implementation specific but somehow the server has been tested to see if it ready for connections.
		 * @return	True if the server is able to be connected to.
		 **/
		virtual bool TestConnection() = 0;

		/**
		 * @return	Get the directory after PixelStreaming/Resources/WebServers/{ReturnThisDirectoryName}.
		 **/
		virtual FString GetServerResourceDirectory() = 0;

		virtual bool FindServerAbsPath(FLaunchArgs& InLaunchArgs, FString& OutServerAbsPath);

	protected:
		TSharedPtr<FMonitoredProcess> ServerProcess;
		FString ServerRootAbsPath;
		FDelegateHandle EngineShutdownHandle;
	};

} // UE::PixelStreamingServers
