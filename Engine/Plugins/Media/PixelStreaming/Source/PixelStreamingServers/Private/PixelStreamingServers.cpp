// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingServers.h"
#include "CirrusWrapper.h"
#include "SignallingServer.h"
#include "SignallingServerLegacy.h"

namespace UE::PixelStreamingServers
{
	TSharedPtr<IServer> MakeCirrusServer()
	{
		return MakeShared<FCirrusWrapper>();
	}

	TSharedPtr<IServer> MakeSignallingServer()
	{
		return MakeShared<FSignallingServer>();
	}

	TSharedPtr<FMonitoredProcess> DownloadPixelStreamingServers(bool bSkipIfPresent)
	{
		return Utils::DownloadPixelStreamingServers(bSkipIfPresent);
	}

	TSharedPtr<IServer> MakeLegacySignallingServer()
	{
		return MakeShared<FSignallingServerLegacy>();
	}

} // namespace UE::PixelStreamingServers
