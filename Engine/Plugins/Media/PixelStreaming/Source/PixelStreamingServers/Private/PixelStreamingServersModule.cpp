// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingServersModule.h"

FPixelStreamingServersModule& FPixelStreamingServersModule::Get()
{
	return FModuleManager::LoadModuleChecked<FPixelStreamingServersModule>("PixelStreamingServers");
}

void FPixelStreamingServersModule::StartupModule()
{
	// Any startup logic require goes here.
}

IMPLEMENT_MODULE(FPixelStreamingServersModule, PixelStreamingServers)
