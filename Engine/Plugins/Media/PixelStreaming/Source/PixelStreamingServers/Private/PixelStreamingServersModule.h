// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Modules/ModuleManager.h"

class FPixelStreamingServersModule : public IModuleInterface
{
public:
	virtual ~FPixelStreamingServersModule() = default;
	virtual void StartupModule() override;

public:
	static FPixelStreamingServersModule& Get();

private:
	FThreadSafeCounter NextGeneratedPort = 0;

};