// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerBase.h"
#include "PixelStreamingServersLog.h"

namespace UE::PixelStreamingServers
{	
	FServerBase::FServerBase()
		: FTickableGameObject(ETickableTickType::Never)
	{
	}

	FServerBase::~FServerBase()
	{
		SetTickableTickType(ETickableTickType::Never);
	}

	bool FServerBase::Launch(FLaunchArgs& InLaunchArgs)
	{
		SetTickableTickType(ETickableTickType::Always);
		
		LaunchArgs = InLaunchArgs;
		bPollUntilReady = InLaunchArgs.bPollUntilReady;

		bHasLaunched = LaunchImpl(InLaunchArgs, Endpoints);
		PollingStartedSeconds = FPlatformTime::Seconds();
		return bHasLaunched;
	}

	bool FServerBase::IsReady()
	{
		return bIsReady;
	}

	bool FServerBase::IsTickableWhenPaused() const
	{
		return true;
	}

	bool FServerBase::IsTickableInEditor() const
	{
		return true;
	}

	bool FServerBase::IsTickable() const
	{
		return bAllowedToTick;
	}

	void FServerBase::Tick(float DeltaTime)
	{
		// No need to do polling if polling is turned off
		if (!bPollUntilReady)
		{
			return;
		}

		// No need to start polling if we have not launched or we have already concluded the server is ready.
		if (!bHasLaunched || bIsReady)
		{
			return;
		}

		float SecondsElapsedPolling = FPlatformTime::Seconds() - PollingStartedSeconds;
		float SecondsSinceReconnect = FPlatformTime::Seconds() - LastReconnectionTimeSeconds;

		if (SecondsElapsedPolling < LaunchArgs.ReconnectionTimeoutSeconds)
		{
			if (SecondsSinceReconnect < LaunchArgs.ReconnectionIntervalSeconds)
			{
				return;
			}

			if (TestConnection())
			{
				UE_LOGF(LogPixelStreamingServers, Log, "Connected to the server. Server is now ready.  - Broadcasting OnReady event...");
				OnReady.Broadcast(Endpoints);
				// No need to poll anymore we are connected
				bAllowedToTick = false;
				bIsReady = true;
				return;
			}
			else
			{
				UE_LOGF(LogPixelStreamingServers, Log, "Polling again in another %.f seconds for server to become ready...", LaunchArgs.ReconnectionIntervalSeconds);
				LastReconnectionTimeSeconds = FPlatformTime::Seconds();
			}
		}
		else
		{
			// No need to poll anymore we timed out
			UE_LOGF(LogPixelStreamingServers, Error, "Server was not ready after %.f seconds, polling timed out.", LaunchArgs.ReconnectionTimeoutSeconds);
			bAllowedToTick = false;
			bTimedOut = true;
			OnFailedToReady.Broadcast();
		}
	}

	bool FServerBase::HasLaunched()
	{
		return bHasLaunched;
	}

	bool FServerBase::IsTimedOut()
	{
		return bTimedOut;
	}
}