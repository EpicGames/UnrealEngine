// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerUtils.h"
#include "HAL/PlatformFileManager.h"
#include "PixelStreamingServers.h"
#include "PixelStreamingServersModule.h"
#include "Containers/UnrealString.h"
#include "PixelStreamingServersLog.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonReader.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Misc/Paths.h"
#include "SocketUtils.h"
#include "GenericPlatform/GenericPlatformFile.h"

namespace UE::PixelStreamingServers::Utils
{

	TSharedPtr<FMonitoredProcess> LaunchChildProcess(FString ExecutableAbsPath, FString Args, FString LogPrefix, bool bRunAsScript)
	{
		// Check if the binary actually exists
		IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
		if (!FileManager.FileExists(*ExecutableAbsPath))
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Cannot start child process - the specified file did not exist. File=%ls", *ExecutableAbsPath);
			return TSharedPtr<FMonitoredProcess>();
		}

		if(bRunAsScript)
		{
// Get the executable we will use to run the scripts (e.g. cmd.exe on Windows)
#if PLATFORM_WINDOWS
			Args = FString::Printf(TEXT("/c \"\"%s\" %s\""), *ExecutableAbsPath, *Args);
			ExecutableAbsPath = TEXT("cmd.exe");
			
#elif PLATFORM_LINUX
			Args = FString::Printf(TEXT(" -- \"%s\" %s --nosudo"), *ExecutableAbsPath, *Args);
			ExecutableAbsPath = TEXT("/usr/bin/bash");
#elif PLATFORM_MAC
            Args = FString::Printf(TEXT(" -- \"%s\" %s --nosudo"), *ExecutableAbsPath, *Args);
            ExecutableAbsPath = TEXT("/bin/bash");
#else
			UE_LOGF(LogPixelStreamingServers, Error, "Unsupported platform for Pixel Streaming.");
			return TSharedPtr<FMonitoredProcess>();
#endif
		}

		TSharedPtr<FMonitoredProcess> ChildProcess = MakeShared<FMonitoredProcess>(
			ExecutableAbsPath, 
			Args, 
			true, 
#if PLATFORM_MAC
			// Pipes cause UE to lockup when destroying on Mac
			false
#else
			true
#endif
		);
		// Bind to output so we can capture the output in the log
		ChildProcess->OnOutput().BindLambda([LogPrefix](FString Output) {
			UE_LOGF(LogPixelStreamingServers, Log, "%ls - %ls", *LogPrefix, *Output);
		});
		// Run the child process
		UE_LOGF(LogPixelStreamingServers, Log, "Launch child process - %ls %ls", *ExecutableAbsPath, *Args);
		ChildProcess->Launch();
		return ChildProcess;
	}

	bool ExtractValueFromArgs(FString ArgsString, FString ArgKey, FString FallbackValue, FString& OutValue)
	{
		// Tokenize string in single whitespace " ".
		TArray<FString> ArgTokens;
		ArgsString.ParseIntoArray(ArgTokens, TEXT(" "), true);

		for (FString& Token : ArgTokens)
		{
			Token.TrimStartAndEndInline();

			if (!Token.StartsWith(ArgKey, ESearchCase::Type::CaseSensitive))
			{
				continue;
			}

			// We have a matching token for our search "key" - split on it.
			FString RightStr;
			if (!Token.Split(TEXT("="), nullptr, &RightStr))
			{
				continue;
			}

			OutValue = RightStr;
			return true;
		}
		OutValue = FallbackValue;
		return false;
	}

	FString QueryOrSetProcessArgs(FLaunchArgs& LaunchArgs, FString ArgKey, FString FallbackArgValue)
	{
		FString OutValue;
		bool bExtractedValue = ExtractValueFromArgs(LaunchArgs.ProcessArgs, ArgKey, FallbackArgValue, OutValue);

		// No key was present so we will inject our own.
		if (!bExtractedValue)
		{
			LaunchArgs.ProcessArgs += FString::Printf(TEXT(" %s%s"), *ArgKey, *FallbackArgValue);
		}

		return OutValue;
	}

	bool GetResourcesDir(FString& OutResourcesDir)
	{
#if WITH_EDITOR
		OutResourcesDir = FPaths::EnginePluginsDir() / TEXT("Media") / TEXT("PixelStreaming") / TEXT("Resources");
#else
		OutResourcesDir = FPaths::ProjectDir() / TEXT("Samples") / TEXT("PixelStreaming");
#endif // WITH_EDITOR

		OutResourcesDir = FPaths::ConvertRelativePathToFull(OutResourcesDir);

		if(FPaths::DirectoryExists(OutResourcesDir))
		{
			return true;
		}
		return false;
	}

	bool GetWebServersDir(FString& OutWebServersAbsPath)
	{
		bool bResourceDirExists = GetResourcesDir(OutWebServersAbsPath);

		if(!bResourceDirExists)
		{
			return false;
		}

		OutWebServersAbsPath = OutWebServersAbsPath / TEXT("WebServers");

		if(FPaths::DirectoryExists(OutWebServersAbsPath))
		{
			return true;
		}
		return false;
	}

	bool GetDownloadedServer(FString& OutAbsPath, FString ServerDirectoryName)
	{
		bool bServersDirExists = GetWebServersDir(OutAbsPath);

		if(!bServersDirExists)
		{
			return false;
		}

		// Now add the {ServerDirectoryName}/plaform_script/{os_specific_path}/{run_local.bat|sh}
		OutAbsPath = OutAbsPath / ServerDirectoryName / TEXT("platform_scripts");
#if PLATFORM_WINDOWS
		OutAbsPath = OutAbsPath / TEXT("cmd") / TEXT("run_local.bat");
#elif PLATFORM_LINUX || PLATFORM_MAC
		OutAbsPath = OutAbsPath / TEXT("bash") / TEXT("run_local.sh");
#else
		UE_LOGF(LogPixelStreamingServers, Error, "Unsupported platform for Pixel Streaming scripts.");
		return false;
#endif

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if(PlatformFile.FileExists(*OutAbsPath))
		{
			return true;
		}
		return false;
	}

	TSharedPtr<FMonitoredProcess> DownloadPixelStreamingServers(bool bSkipIfPresent)
	{
		FString OutScriptPath;
		if(bSkipIfPresent && GetDownloadedServer(OutScriptPath, FString(TEXT("SignallingWebServer"))))
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Found pixel streaming servers, skipping download.");
			// empty process
			return TSharedPtr<FMonitoredProcess>();
		}
		
		bool bHasWebServersDir = GetWebServersDir(OutScriptPath);

		if(!bHasWebServersDir)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Could not download ps servers, no PixelStreaming/Resources/WebServers directory found.");
		}

		FString Args = TEXT("");

#if PLATFORM_WINDOWS
		OutScriptPath = OutScriptPath / TEXT("get_ps_servers.bat");
#elif PLATFORM_LINUX || PLATFORM_MAC
		OutScriptPath = OutScriptPath / TEXT("get_ps_servers.sh");
#else
		UE_LOGF(LogPixelStreamingServers, Error, "Unsupported platform for Pixel Streaming scripts.");
		// empty process
		return TSharedPtr<FMonitoredProcess>();
#endif

		return LaunchChildProcess(OutScriptPath, Args, FString(TEXT("Download ps servers")), true /*bRunAsScript*/);
	}

	bool PopulateCirrusEndPoints(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndPoints)
	{
		FPixelStreamingServersModule& Module = FPixelStreamingServersModule::Get();

		// Query for ports, or set them if they don't exist
		int FallbackStreamerPort = UE::PixelStreaming::GetNextAvailablePort();
		int FallbackSFUPort = UE::PixelStreaming::GetNextAvailablePort();
		int FallbackHttpPort = UE::PixelStreaming::GetNextAvailablePort();
		int FallbackMatchmakerPort = UE::PixelStreaming::GetNextAvailablePort();
		if (FallbackStreamerPort == -1)
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Failed to find an available port for streamer connections");
			return false;
		}
		if (FallbackSFUPort == -1)
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Failed to find an available port for SFU connections");
			return false;
		}
		if (FallbackHttpPort == -1)
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Failed to find an available port for http connections");
			return false;
		}
		if (FallbackMatchmakerPort == -1)
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Failed to find an available port for matchmaker connections");
			return false;
		}

		FString StreamerPort = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--StreamerPort="), FString::FromInt(FallbackStreamerPort));
		FString SFUPort = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--SFUPort="), FString::FromInt(FallbackSFUPort));
		FString MatchmakerPort = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--MatchmakerPort="), FString::FromInt(FallbackMatchmakerPort));
		FString HttpPort = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--HttpPort="), FString::FromInt(FallbackHttpPort));

		// Construct endpoint urls

		// Streamer
		FURL SignallingStreamerUrl;
		SignallingStreamerUrl.Protocol = TEXT("ws");
		SignallingStreamerUrl.Host = TEXT("127.0.0.1");
		SignallingStreamerUrl.Port = FCString::Atoi(*StreamerPort); 
		SignallingStreamerUrl.Map = FString();

		// Players
		FURL PlayersUrl;
		PlayersUrl.Protocol = TEXT("ws");
		PlayersUrl.Host = TEXT("127.0.0.1");
		PlayersUrl.Port = FCString::Atoi(*HttpPort);
		PlayersUrl.Map = FString();

		// Webserver
		FURL WebServerUrl;
		WebServerUrl.Protocol = TEXT("http");
		WebServerUrl.Host = TEXT("127.0.0.1");
		WebServerUrl.Port = FCString::Atoi(*HttpPort);
		WebServerUrl.Map = FString();

		// SFU
		FURL SFUUrl;
		SFUUrl.Protocol = TEXT("ws");
		SFUUrl.Host = TEXT("127.0.0.1");
		SFUUrl.Port = FCString::Atoi(*SFUPort);
		SFUUrl.Map = FString();

		// Matchmaker
		FURL MatchmakerUrl;
		MatchmakerUrl.Protocol = TEXT("ws");
		MatchmakerUrl.Host = TEXT("127.0.0.1");
		MatchmakerUrl.Port = FCString::Atoi(*MatchmakerPort);
		MatchmakerUrl.Map = FString();

		OutEndPoints.Add(EEndpoint::Signalling_Streamer, SignallingStreamerUrl);
		OutEndPoints.Add(EEndpoint::Signalling_Players, PlayersUrl);
		OutEndPoints.Add(EEndpoint::Signalling_SFU, SFUUrl);
		OutEndPoints.Add(EEndpoint::Signalling_Matchmaker, MatchmakerUrl);
		OutEndPoints.Add(EEndpoint::Signalling_Webserver, WebServerUrl);

		return true;
	}

	FString ToString(FURL Url)
	{
		return FString::Printf(TEXT("%s://%s:%d"), *(Url.Protocol), *(Url.Host), Url.Port);
	}

	FString ToString(TArrayView<uint8> UTF8Bytes)
	{
		FUTF8ToTCHAR Converted((const ANSICHAR*)UTF8Bytes.GetData(), UTF8Bytes.Num());
		FString OutString = FString::ConstructFromPtrSize(Converted.Get(), Converted.Length());
		return OutString;
	}

	FString ToString(TSharedPtr<FJsonObject> JSONObj)
	{
		FString Res;
		auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Res);
		bool bSerialized = FJsonSerializer::Serialize(JSONObj.ToSharedRef(), JsonWriter);
		if(!bSerialized)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to stringify JSON object.");
		}
		return Res;
	}

	TSharedPtr<FJsonObject> ToJSON(const FString& InString)
	{
		TSharedPtr<FJsonObject> OutJSON = MakeShared<FJsonObject>();
		const auto JsonReader = TJsonReaderFactory<TCHAR>::Create(InString);
		if (FJsonSerializer::Deserialize(JsonReader, OutJSON))
		{
			return OutJSON;
		}
		return nullptr;
	}

} // UE::PixelStreamingServers::Utils