// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingServers.h"
#include "Templates/SharedPointer.h"
#include "Misc/MonitoredProcess.h"
#include "Dom/JsonObject.h"
#include "Engine/EngineBaseTypes.h"

namespace UE::PixelStreamingServers
{
	namespace Utils
	{
		/**
		 * Inspects the `LaunchArgs.ProcessArgs` for a string key matching `ArgKey`.
		 * We assume `LaunchArgs.ProcessArgs` are passed as `--key=value` in a String.
		 * @param LaunchArgs The launch args to get/set the ProcessArgs of.
		 * @param ArgKey The key to look for in the `LaunchArgs.ProcessArgs`
		 * @param FallbackArgValue If the key does not exist in the `LaunchArgs.ProcessArgs` this value to set for it in-place.
		 * @return The extracted/set process arg.
		 **/
		FString QueryOrSetProcessArgs(FLaunchArgs& LaunchArgs, FString ArgKey, FString FallbackArgValue);

		/**
		 * Launches an executable or script as a child process of Unreal Engine.
		 * Note: When running scripts they must be launchable through cmd.exe on Windows or bash on Linux.
		 * @param ExecutableAbsPath The absolute path to the executable or script we want to run.
		 * @param Args The argument to pass to the child process we are launching.
		 * @param LogPrefix If the child process has output the prefix to put on the UE log message for that output.
		 * @param bRunAsScript If true the `ExecutableAbsPath` will be passed to cmd/bash.
		 **/
		TSharedPtr<FMonitoredProcess> LaunchChildProcess(FString ExecutableAbsPath, FString Args, FString LogPrefix, bool bRunAsScript);

		/**
		 * Get the absolute path to the Pixel Streaming resources dir (changes based on whether called in editor or not.)
		 * In editor this PixelStreaming/Resources in a game this is Samples/PixelStreaming
		 * @param OutResourcesDir The absolute path to "resources" dir.
		 * @return True if the returned path exists.
		 **/
		bool GetResourcesDir(FString& OutResourcesDir);

		/**
		 * Get the absolute path to the webservers (changes based on whether called in editor or not.)
		 * @param OutWebServersAbsPath The absolute path to the webservers.
		 * @return True if the returned path exists.
		 **/
		bool GetWebServersDir(FString& OutWebServersAbsPath);

		/**
		 * Attempts to get the absolute path to a downloaded server residing in the /Resources directory
		 * of the Pixel Streaming plugin, if it exists.
		 * @param OutAbsPath The absolute path of the downloaded server.
		 * @param ServerDirectoryName The directory name of the server we should look for under /Resources
		 * @return True if the scripts to launch the server exist at this location.
		 **/
		bool GetDownloadedServer(FString& OutAbsPath, FString ServerDirectoryName);

		/**
		 * Download the Pixel Streaming servers using the `get_ps_servers` scripts.
		 * @param bSkipIfPresent Servers will not be downloaded if they are already present.
		 * @return The child process that is used to download the servers.
		 **/
		TSharedPtr<FMonitoredProcess> DownloadPixelStreamingServers(bool bSkipIfPresent);

		/**
		 * Populates the cirrus endpoints by extracting endpoint information from the `InLaunchArgs.ProcessArgs`, and if not found, using defaults.
		 * @param InLaunchArgs The launch args containing the process args that we will be inspecting.
		 * @param OutEndPoints The end points for the server if we launch it with these launch args.
		 * @param return True if the end point was added.
		 **/
		bool PopulateCirrusEndPoints(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndPoints);

		/**
		 * Convert a `FURL` into a `FString`. `FURL::ToString()` already exists; however, it appends a problematic trailing slash to the output string.
		 * @param Url The url to format into a string.
		 * @return The url formatted without a trailing slash (the trailing slash messes up the Websocket clients).
		 **/
		FString ToString(FURL Url);

		/**
		 * Convert a UTF8 byte array to an FString
		 * @param UTF8Bytes Byte array of UTF8 characters.
		 * @return A string representation of the byte array.
		 **/
		FString ToString(TArrayView<uint8> UTF8Bytes);

		/**
		 * Convert JSON object to FString.
		 * @param JSONObj A JSON object.
		 * @return The stringified JSON object.
		 **/
		FString ToString(TSharedPtr<FJsonObject> JSONObj);

		/** 
		 * Convert a JSON formatted string into a JSON object, if possible.
		 * @param InString The JSON object expressed as a string.
		 * @return The output json object or null on failure.
		 **/
		TSharedPtr<FJsonObject> ToJSON(const FString& InString);

	} // Utils
} // UE::PixelStreamingServers
