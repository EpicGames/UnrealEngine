// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignallingServer.h"
#include "ServerUtils.h"
#include "PixelStreamingServersLog.h"
#include "Misc/Paths.h"

namespace UE::PixelStreamingServers
{
	static const FString LEGACY_NAME = "_LEGACY_";

	FSignallingServer::FSignallingServer()
	{
		StreamerMessageHandlers.Add("endpointId").BindRaw(this, &FSignallingServer::OnStreamerIdMessage);
		StreamerMessageHandlers.Add("ping").BindRaw(this, &FSignallingServer::OnStreamerPingMessage);
		StreamerMessageHandlers.Add("disconnectPlayer").BindRaw(this, &FSignallingServer::OnStreamerDisconnectMessage);

		PlayerMessageHandlers.Add("listStreamers").BindRaw(this, &FSignallingServer::OnPlayerListStreamersMessage);
		PlayerMessageHandlers.Add("subscribe").BindRaw(this, &FSignallingServer::OnPlayerSubscribeMessage);
		PlayerMessageHandlers.Add("unsubscribe").BindRaw(this, &FSignallingServer::OnPlayerUnsubscribeMessage);
		PlayerMessageHandlers.Add("stats").BindRaw(this, &FSignallingServer::OnPlayerStatsMessage);
		PlayerMessageHandlers.Add("ping").BindRaw(this, &FSignallingServer::OnPlayerPingMessage);
	}

	void FSignallingServer::Stop()
	{
		if (StreamersWS)
		{
			StreamersWS->Stop();
			StreamersWS.Reset();
		}
		if (Probe)
		{
			Probe.Reset();
		}
	}

	bool FSignallingServer::TestConnection()
	{
		if (bIsReady)
		{
			return true;
		}
		else
		{
			bool bConnected = Probe && Probe->Probe();
			if (bConnected)
			{
				// Close the websocket connection so others can use it
				Probe->Close();
				Probe.Reset();
				// Note: even after closing the client WS of the probe above it will take another tick to remove the connection
				// from the ws server so if you query number of streamers during this time we do have to remove the probe from that count
				return true;
			}
			else
			{
				return false;
			}
		}
	}

	bool FSignallingServer::LaunchImpl(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndpoints)
	{
		Utils::PopulateCirrusEndPoints(InLaunchArgs, OutEndpoints);
		FURL PlayersURL = OutEndpoints[EEndpoint::Signalling_Players];
		FURL StreamerURL = OutEndpoints[EEndpoint::Signalling_Streamer];

		/*
		 * --------------- Streamers websocket server ---------------
		 */
		StreamersWS = MakeUnique<FWebSocketServerWrapper>();
		StreamersWS->OnMessage.AddRaw(this, &FSignallingServer::OnStreamerMessage);
		StreamersWS->OnOpenConnection.AddRaw(this, &FSignallingServer::OnStreamerConnected);
		StreamersWS->OnClosedConnection.AddRaw(this, &FSignallingServer::OnStreamerDisconnected);
		bool bLaunchedStreamerServer = StreamersWS->Launch(StreamerURL.Port);

		if (!bLaunchedStreamerServer)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to launch websocket server for streamers on port=%d", StreamerURL.Port);
			return false;
		}

		/*
		 * --------------- Players websocket server ---------------
		 */
		PlayersWS = MakeUnique<FWebSocketServerWrapper>();
		PlayersWS->EnableWebServer(GenerateDirectoriesToServe());
		PlayersWS->OnMessage.AddRaw(this, &FSignallingServer::OnPlayerMessage);
		PlayersWS->OnOpenConnection.AddRaw(this, &FSignallingServer::OnPlayerConnected);
		PlayersWS->OnClosedConnection.AddRaw(this, &FSignallingServer::OnPlayerDisconnected);
		bool bLaunchedPlayerServer = PlayersWS->Launch(PlayersURL.Port);

		if (!bLaunchedPlayerServer)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to launch websocket server for players on port=%d", PlayersURL.Port);
			return false;
		}

		/*
		 * --------------- Websocket probe ---------------
		 */

		if (bPollUntilReady)
		{
			TArray<FString> Protocols;
			Protocols.Add(FString(TEXT("binary")));
			Probe = MakeUnique<FWebSocketProbe>(StreamerURL, Protocols);
		}

		return true;
	}

	FString FSignallingServer::GetPathOnDisk()
	{
		return FString();
	}

	TArray<FWebSocketHttpMount> FSignallingServer::GenerateDirectoriesToServe() const
	{
		FString ServersDir;
		bool	bServersDirExists = Utils::GetWebServersDir(ServersDir);
		if (bServersDirExists)
		{
			ServersDir = ServersDir / TEXT("SignallingWebServer");
			bServersDirExists = FPaths::DirectoryExists(ServersDir);
		}

		TArray<FWebSocketHttpMount> MountsArr;

#if WITH_EDITOR
		// If server directory doesn't exist we will serve a known directory that gives the user a message
		// telling them to run the `get_ps_servers` script.
		if (!bServersDirExists)
		{
			FString OutResourcesDir;
			bool	bResourcesDirExists = Utils::GetResourcesDir(OutResourcesDir);
			FString NotFoundDir = OutResourcesDir / TEXT("NotFound");

			if (bResourcesDirExists && FPaths::DirectoryExists(NotFoundDir))
			{
				FWebSocketHttpMount Mount;
				Mount.SetPathOnDisk(NotFoundDir);
				Mount.SetWebPath(FString(TEXT("/")));
				Mount.SetDefaultFile(FString(TEXT("not_found.html")));
				MountsArr.Add(Mount);
				return MountsArr;
			}
		}
#endif // WITH_EDITOR

		// Add /Public
		FWebSocketHttpMount PublicMount;
		PublicMount.SetPathOnDisk(ServersDir / TEXT("www"));
		PublicMount.SetWebPath(FString(TEXT("/")));
		PublicMount.SetDefaultFile(FString(TEXT("player.html")));
		MountsArr.Add(PublicMount);

		// Todo (Luke.Bermingham): Expose way for user to specify what directories to serve.

		return MountsArr;
	}

	TSharedRef<FJsonObject> FSignallingServer::CreateConfigJSON() const
	{
		// Todo (Luke): Parse `iceServers` from the process args `--peerConnectionOptions`
		TArray<TSharedPtr<FJsonValue>> IceServersArr;

		TSharedPtr<FJsonObject> PeerConnectionOptionsJSON = MakeShared<FJsonObject>();
		PeerConnectionOptionsJSON->SetArrayField(FString(TEXT("iceServers")), IceServersArr);

		TSharedRef<FJsonObject> ConfigJSON = MakeShared<FJsonObject>();
		ConfigJSON->SetStringField(FString(TEXT("type")), FString(TEXT("config")));
		ConfigJSON->SetObjectField(FString(TEXT("peerConnectionOptions")), PeerConnectionOptionsJSON);
		return ConfigJSON;
	}

	TSharedPtr<FJsonObject> FSignallingServer::ParseMessage(const FString& InMessage, FString& OutMessageType) const
	{
		TSharedPtr<FJsonObject> JSONObj = Utils::ToJSON(InMessage);
		if (!JSONObj)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to parse message: %ls", *InMessage);
			return nullptr;
		}

		if (!JSONObj->TryGetStringField(TEXT("type"), OutMessageType))
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Incoming message did not contain a 'type' field: %ls", *InMessage);
			return nullptr;
		}

		return JSONObj;
	}

	void FSignallingServer::SubscribePlayer(uint16 PlayerConnectionId, const FString& StreamerName)
	{
		UE_LOGF(LogPixelStreamingServers, Log, "Subscribing player %d to streamer %ls", PlayerConnectionId, *StreamerName);

		uint16 StreamerConnectionId;
		if (!StreamersWS->GetNamedConnection(StreamerName, StreamerConnectionId))
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Streamer name %ls does not exist", *StreamerName);
			return;
		}

		if (!StreamersWS->GetConnections().Contains(StreamerConnectionId))
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Streamer %d does not exist", StreamerConnectionId);
			return;
		}

		if (PlayerSubscriptions.Contains(PlayerConnectionId))
		{
			// unsubscribe first
			UnsubscribePlayer(PlayerConnectionId);
		}

		// We don't want to make the connections shared to prevent someone accidentally holding on to it. So we use it raw here
		FWebSocketConnection* PlayerWS = (*PlayersWS->GetConnections().Find(PlayerConnectionId)).Get();
		bool				  bUESendsOffer = !PlayerWS->GetUrlArgs().Contains(TEXT("OfferToReceive=true"));

		// Send "playerConnected" message to streamer which kicks off making a new RTC connection
		TSharedRef<FJsonObject> OnPlayerConnectedJSON = MakeShared<FJsonObject>();
		OnPlayerConnectedJSON->SetStringField(TEXT("type"), "playerConnected");
		OnPlayerConnectedJSON->SetStringField(TEXT("playerId"), FString::FromInt(PlayerConnectionId));
		OnPlayerConnectedJSON->SetBoolField(TEXT("dataChannel"), true);
		OnPlayerConnectedJSON->SetBoolField(TEXT("sfu"), false);
		OnPlayerConnectedJSON->SetBoolField(TEXT("sendOffer"), bUESendsOffer);
		SendStreamerMessage(StreamerConnectionId, OnPlayerConnectedJSON);

		PlayerSubscriptions.Add(PlayerConnectionId, StreamerConnectionId);
	}

	void FSignallingServer::UnsubscribePlayer(uint16 PlayerConnectionId)
	{
		if (PlayerSubscriptions.Contains(PlayerConnectionId))
		{
			const uint16 StreamerConnectionId = PlayerSubscriptions[PlayerConnectionId];
			UE_LOGF(LogPixelStreamingServers, Log, "Unsubscribing player %d from streamer %d", PlayerConnectionId, StreamerConnectionId);

			// Send "playerDisconnected" message to streamer
			TSharedRef<FJsonObject> OnPlayerDisconnectedJSON = MakeShared<FJsonObject>();
			OnPlayerDisconnectedJSON->SetStringField(TEXT("type"), "playerDisconnected");
			OnPlayerDisconnectedJSON->SetStringField(TEXT("playerId"), FString::FromInt(PlayerConnectionId));
			SendStreamerMessage(StreamerConnectionId, OnPlayerDisconnectedJSON);

			PlayerSubscriptions.Remove(PlayerConnectionId);
		}
	}

	void FSignallingServer::SendPlayerMessage(uint16 PlayerId, TSharedPtr<FJsonObject> JSONObj)
	{
		const FString MessageString = Utils::ToString(JSONObj);
		UE_LOGF(LogPixelStreamingServers, Log, "Sending to player id=%d: %ls", PlayerId, *MessageString);
		PlayersWS->Send(PlayerId, MessageString);
	}

	void FSignallingServer::SendStreamerMessage(uint16 StreamerId, TSharedPtr<FJsonObject> JSONObj)
	{
		const FString MessageString = Utils::ToString(JSONObj);
		UE_LOGF(LogPixelStreamingServers, Log, "Sending to streamer id=%d: %ls", StreamerId, *MessageString);
		StreamersWS->Send(StreamerId, MessageString);
	}

	void FSignallingServer::OnStreamerConnected(uint16 ConnectionId)
	{
		UE_LOGF(LogPixelStreamingServers, Log, "Streamer websocket connected, id=%d", ConnectionId);

		// Send a config message to the streamer passing ICE servers to be used.
		TSharedPtr<FJsonObject> JSONObj = CreateConfigJSON();
		SendStreamerMessage(ConnectionId, JSONObj);

		// request the streamer id
		TSharedRef<FJsonObject> idJSON = MakeShared<FJsonObject>();
		idJSON->SetStringField(TEXT("type"), "identify");
		SendStreamerMessage(ConnectionId, idJSON);
	}

	void FSignallingServer::OnStreamerDisconnected(uint16 ConnectionId)
	{
		UE_LOGF(LogPixelStreamingServers, Log, "Streamer websocket disconnected, id=%d", ConnectionId);

		for (auto& ConnectionPair : PlayerSubscriptions)
		{
			const uint16 StreamerConnectionId = ConnectionPair.Key;
			const uint16 PlayerConnectionId = ConnectionPair.Value;
			if (StreamerConnectionId == ConnectionId)
			{
				UnsubscribePlayer(PlayerConnectionId);
			}
		}
	}

	void FSignallingServer::OnStreamerMessage(uint16 ConnectionId, TArrayView<uint8> Message)
	{
		const FString Msg = Utils::ToString(Message);
		UE_LOGF(LogPixelStreamingServers, Log, "From Streamer id=%d: %ls", ConnectionId, *Msg);

		FString					MsgType;
		TSharedPtr<FJsonObject> JSONObj = ParseMessage(Msg, MsgType);
		if (!JSONObj)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to parse incoming streamer message.");
			return;
		}

		if (auto* Handler = StreamerMessageHandlers.Find(MsgType))
		{
			Handler->Execute(ConnectionId, JSONObj);
		}
		else
		{
			// All other message types require a `playerId` field to be valid.
			uint16 PlayerConnectionId;
			if (!JSONObj->TryGetNumberField(TEXT("playerId"), PlayerConnectionId))
			{
				UE_LOGF(LogPixelStreamingServers, Warning, "Message did not contain a field called 'playerId' - message=%ls", *Msg);
				return;
			}

			// As message are going to the player they don't actually need the playerId field, the field exists only so we know who to send it to.
			JSONObj->RemoveField(TEXT("playerId"));

			SendPlayerMessage(PlayerConnectionId, JSONObj);
		}
	}

	void FSignallingServer::OnPlayerConnected(uint16 ConnectionId)
	{
		// Send config to newly connected player, which kicks off making a new RTC connection
		TSharedPtr<FJsonObject> ConfigJSON = CreateConfigJSON();
		SendPlayerMessage(ConnectionId, ConfigJSON);
	}

	void FSignallingServer::OnPlayerDisconnected(uint16 ConnectionId)
	{
		UnsubscribePlayer(ConnectionId);
	}

	void FSignallingServer::OnPlayerMessage(uint16 ConnectionId, TArrayView<uint8> Message)
	{
		const FString Msg = Utils::ToString(Message);
		UE_LOGF(LogPixelStreamingServers, Log, "From Player id=%d: %ls", ConnectionId, *Msg);

		FString					MsgType;
		TSharedPtr<FJsonObject> JSONObj = ParseMessage(Msg, MsgType);
		if (!JSONObj)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to parse incoming player message.");
			return;
		}

		if (auto* Handler = PlayerMessageHandlers.Find(MsgType))
		{
			Handler->Execute(ConnectionId, JSONObj);
		}
		else
		{
			if (!PlayerSubscriptions.Contains(ConnectionId))
			{
				TArray<FString> StreamerConnections = StreamersWS->GetConnectionNames();
				if (StreamerConnections.Num() == 0)
				{
					UE_LOGF(LogPixelStreamingServers, Error, "Player %d sent a message, but no streamers were connected", ConnectionId);
					return;
				}

				UE_LOGF(LogPixelStreamingServers, Log, "Player %d attempted to send an outgoing message without having subscribed first. Defaulting to %ls", ConnectionId, *StreamerConnections[0]);
				SubscribePlayer(ConnectionId, StreamerConnections[0]);
			}

			// Add player id to any messages going to streamer so streamer knows who sent it
			JSONObj->SetStringField(TEXT("playerId"), FString::FromInt(ConnectionId));
			SendStreamerMessage(PlayerSubscriptions[ConnectionId], JSONObj);
		}
	}

	void FSignallingServer::OnStreamerIdMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		FString StreamerName;
		if (JSONObj->TryGetStringField(TEXT("id"), StreamerName))
		{
			StreamersWS->NameConnection(ConnectionId, StreamerName);
			StreamersWS->RemoveName(LEGACY_NAME);
		}
	}

	void FSignallingServer::OnStreamerPingMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		const double			UnixTime = FDateTime::UtcNow().ToUnixTimestamp();
		TSharedRef<FJsonObject> PongJSON = MakeShared<FJsonObject>();
		PongJSON->SetStringField(TEXT("type"), "pong");
		PongJSON->SetNumberField(TEXT("time"), UnixTime);
		SendStreamerMessage(ConnectionId, PongJSON);
	}

	void FSignallingServer::OnStreamerDisconnectMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		uint16 PlayerConnectionId;
		if (!JSONObj->TryGetNumberField(TEXT("playerId"), PlayerConnectionId))
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Disconnect message did not contain a field called 'playerId'");
			return;
		}

		// TODO this might get called anyway from OnClosedConnection
		UnsubscribePlayer(PlayerConnectionId);
		PlayersWS->Close(PlayerConnectionId);
	}

	void FSignallingServer::OnPlayerListStreamersMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		TSharedRef<FJsonObject>		   listJSON = MakeShared<FJsonObject>();
		const TArray<FString>		   Names = StreamersWS->GetConnectionNames();
		TArray<TSharedPtr<FJsonValue>> JsonNames;
		for (const FString& Name : Names)
		{
			JsonNames.Add(MakeShared<FJsonValueString>(Name));
		}
		listJSON->SetStringField(TEXT("type"), TEXT("streamerList"));
		listJSON->SetArrayField(TEXT("ids"), JsonNames);
		SendPlayerMessage(ConnectionId, listJSON);
	}

	void FSignallingServer::OnPlayerSubscribeMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		FString StreamerName;
		if (!JSONObj->TryGetStringField(TEXT("streamerId"), StreamerName))
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Player %d subscribe message missing streamerId.", ConnectionId);
		}
		else
		{
			SubscribePlayer(ConnectionId, StreamerName);
		}
	}

	void FSignallingServer::OnPlayerUnsubscribeMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		UnsubscribePlayer(ConnectionId);
	}

	void FSignallingServer::OnPlayerStatsMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		UE_LOGF(LogPixelStreamingServers, Log, "Player %d stats = \n %ls", ConnectionId, *Utils::ToString(JSONObj.ToSharedRef()));
	}

	void FSignallingServer::OnPlayerPingMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		const double			UnixTime = FDateTime::UtcNow().ToUnixTimestamp();
		TSharedRef<FJsonObject> PongJSON = MakeShared<FJsonObject>();
		PongJSON->SetStringField(TEXT("type"), "pong");
		PongJSON->SetNumberField(TEXT("time"), UnixTime);
		SendPlayerMessage(ConnectionId, PongJSON);
	}

	void FSignallingServer::GetNumStreamers(TFunction<void(uint16)> OnNumStreamersReceived)
	{
		if (StreamersWS)
		{
			int NConnections = StreamersWS->Count();

			// If the probe is currently connected, it is not a streamer, so don't count it.
			if(Probe && Probe->IsConnected()) {
				NConnections = NConnections - 1;
			}

			OnNumStreamersReceived(NConnections);
		}
		else
		{
			// Streamers websocket server went out of scope, so we can assume no streamers are connected.
			OnNumStreamersReceived(0);
		}
	}

} // namespace UE::PixelStreamingServers
