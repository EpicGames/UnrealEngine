// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ServerBase.h"
#include "WebSocketProbe.h"
#include "WebSocketServerWrapper.h"
#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"
#include "Dom/JsonObject.h"

namespace UE::PixelStreamingServers
{
	/**
	 * A native C++ implementation of a Pixel Streaming signalling server (similar to cirrus.js).
	 * This implementation is missing some features such as SFU, matchmaker, and Webserver support.
	 * The purpose of this implementation is to have signalling server callable from UE, without bundling Cirrus itself.
	 **/
	class FSignallingServer : public FServerBase
	{

	public:
		FSignallingServer();
		virtual ~FSignallingServer() = default;

		// Begin FServerBase
		virtual void Stop() override;
		virtual bool TestConnection() override;
		virtual bool LaunchImpl(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndpoints) override;
		virtual FString GetPathOnDisk() override;
		virtual void GetNumStreamers(TFunction<void(uint16)> OnNumStreamersReceived) override;
		// End FServerBase

	protected:
		TArray<FWebSocketHttpMount> GenerateDirectoriesToServe() const;
		TSharedRef<FJsonObject> CreateConfigJSON() const;
		TSharedPtr<FJsonObject> ParseMessage(const FString& InMessage, FString& OutMessageType) const;

		void SubscribePlayer(uint16 PlayerConnectionId, const FString& StreamerName);
		void UnsubscribePlayer(uint16 PlayerConnectionId);

		virtual void SendPlayerMessage(uint16 PlayerId, TSharedPtr<FJsonObject> JSONObj);
		virtual void SendStreamerMessage(uint16 StreamerId, TSharedPtr<FJsonObject> JSONObj);

		// event handlers
		virtual void OnStreamerConnected(uint16 ConnectionId);
		virtual void OnStreamerDisconnected(uint16 ConnectionId);
		virtual void OnStreamerMessage(uint16 ConnectionId, TArrayView<uint8> Message);
		virtual void OnPlayerConnected(uint16 ConnectionId);
		virtual void OnPlayerDisconnected(uint16 ConnectionId);
		virtual void OnPlayerMessage(uint16 ConnectionId, TArrayView<uint8> Message);

		// message handlers
		void OnStreamerIdMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		void OnStreamerPingMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		void OnStreamerDisconnectMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		void OnPlayerListStreamersMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		void OnPlayerSubscribeMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		void OnPlayerUnsubscribeMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		void OnPlayerStatsMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		void OnPlayerPingMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj);
		
	protected:
		TUniquePtr<FWebSocketProbe> Probe;
		TUniquePtr<FWebSocketServerWrapper> StreamersWS;
		TUniquePtr<FWebSocketServerWrapper> PlayersWS;

		TMap<uint16, uint16> PlayerSubscriptions;

		DECLARE_DELEGATE_TwoParams(FMessageHandler, uint16, TSharedPtr<FJsonObject>);
		TMap<FString, FMessageHandler> StreamerMessageHandlers;
		TMap<FString, FMessageHandler> PlayerMessageHandlers;
	};

} // namespace UE::PixelStreamingServers