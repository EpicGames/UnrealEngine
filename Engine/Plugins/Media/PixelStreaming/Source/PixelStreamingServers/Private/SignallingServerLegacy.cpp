// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignallingServerLegacy.h"

namespace UE::PixelStreamingServers
{
	void FSignallingServerLegacy::SendPlayerMessage(uint16 PlayerId, TSharedPtr<FJsonObject> JSONObj)
	{
		// legacy supports: answer, disconnectPlayer, iceCandidate, config

		const FString MessageString = Utils::ToString(JSONObj);
		UE_LOGF(LogPixelStreamingServers, Log, "Sending to player id=%d: %ls", PlayerId, *MessageString);

		FString MessageType;
		if (!JSONObj->TryGetStringField(TEXT("type"), MessageType))
		{
			UE_LOGF(LogPixelStreamingServers, Error, "No message type on message sent to player %d", PlayerId);
			return;
		}

		static const TSet<FString> ForwardMessages = {
			"offer",
			"answer",
			"iceCandidate",
			"config",
		};

		if (ForwardMessages.Contains(MessageType))
		{
			PlayersWS->Send(PlayerId, MessageString);
			return;
		}
		else if (MessageType == FString(TEXT("disconnectPlayer")))
		{
			PlayersWS->Close(PlayerId);
			return;
		}
		else
		{
			// Unsupported message type
			UE_LOGF(LogPixelStreamingServers, Error, "Unsupported message type sent to player");
			return;
		}
	}

	void FSignallingServerLegacy::SendStreamerMessage(uint16 StreamerId, TSharedPtr<FJsonObject> JSONObj)
	{
		// legacy supports: offer, stats, kick, iceCandidate

		const FString MessageString = Utils::ToString(JSONObj);
		UE_LOGF(LogPixelStreamingServers, Log, "Sending to streamer id=%d: %ls", StreamerId, *MessageString);

		FString MessageType;
		if (!JSONObj->TryGetStringField(TEXT("type"), MessageType))
		{
			UE_LOGF(LogPixelStreamingServers, Error, "No message type on message sent to streamer %d", StreamerId);
			return;
		}

		static const TSet<FString> ForwardMessages = {
			"offer",
			"answer",
			"iceCandidate",
			"config",
			"identify",
			"playerConnected",
			"playerDisconnected"
		};

		if (ForwardMessages.Contains(MessageType))
		{
			StreamersWS->Send(StreamerId, MessageString);
			return;
		}
		else if (MessageType == FString(TEXT("stats")))
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Player stats = \n %ls", *MessageString);
			return;
		}
		else if (MessageType == FString(TEXT("kick")))
		{
			// do nothing, we removed kick from signalling
			return;
		}
		else
		{
			// Unsupported message type
			UE_LOGF(LogPixelStreamingServers, Error, "Unsupported message type sent to streamer");
			return;
		}
	}

	void FSignallingServerLegacy::OnStreamerConnected(uint16 ConnectionId)
	{
		if(StreamersWS->GetConnections().Num() > 1)
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Streamer (id=%d) attempted to connect to the signalling server, but we already have a streamer connected", ConnectionId);	
			return;
		}

		UE_LOGF(LogPixelStreamingServers, Log, "Streamer websocket connected, id=%d", ConnectionId);

		// Send a config message to the streamer passing ICE servers to be used.
		TSharedPtr<FJsonObject> JSONObj = CreateConfigJSON();
		SendStreamerMessage(ConnectionId, JSONObj);

		// request the streamer id
		TSharedRef<FJsonObject> idJSON = MakeShared<FJsonObject>();
		idJSON->SetStringField(TEXT("type"), "identify");
		SendStreamerMessage(ConnectionId, idJSON);

		StreamersWS->NameConnection(ConnectionId, TEXT("_LEGACY_"));
	}

	void FSignallingServerLegacy::OnPlayerMessage(uint16 ConnectionId, TArrayView<uint8> Message) 
	{
		const FString Msg = Utils::ToString(Message);
		UE_LOGF(LogPixelStreamingServers, Log, "From Player id=%d: %ls", ConnectionId, *Msg);

		FString MsgType;
		TSharedPtr<FJsonObject> JSONObj = ParseMessage(Msg, MsgType);
		if (!JSONObj)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to parse incoming player message.");
			return;
		}

		if (auto* Handler = PlayerMessageHandlers.Find(MsgType))
		{
			Handler->Execute(ConnectionId, JSONObj);
		}
		else
		{
			TArray<FString> StreamerConnections = StreamersWS->GetConnectionNames();
			if(StreamerConnections.Num() == 0)
			{
				UE_LOGF(LogPixelStreamingServers, Error, "Player %d sent a message, but no streamers were connected", ConnectionId);
				return;
			}

			uint16 StreamerConnectionId = INDEX_NONE;
			if (StreamersWS->GetNamedConnection(StreamerConnections[0], StreamerConnectionId))
			{
				// Add player id to any messages going to streamer so streamer knows who sent it
				JSONObj->SetStringField(TEXT("playerId"), FString::FromInt(ConnectionId));
				SendStreamerMessage(StreamerConnectionId, JSONObj);
			}
		}
	}
} // namespace UE::PixelStreamingServers