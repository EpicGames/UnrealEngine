// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SignallingServer.h"

namespace UE::PixelStreamingServers
{

	/*
	 * A signalling server that matches the signalling protocol and features of the 4.26/4.27 Cirrus signalling server.
	 * While we don't want people to use this it is useful to keep around for tests to prevent breaking changes while
	 * those servers are still being used by some users.
	 */
	class FSignallingServerLegacy : public FSignallingServer
	{
	protected:
		virtual void SendPlayerMessage(uint16 PlayerId, TSharedPtr<FJsonObject> JSONObj) override;
		virtual void SendStreamerMessage(uint16 StreamerId, TSharedPtr<FJsonObject> JSONObj) override;
		
		virtual void OnStreamerConnected(uint16 ConnectionId) override;
		virtual void OnPlayerMessage(uint16 ConnectionId, TArrayView<uint8> Message) override;
	};

} // namespace UE::PixelStreamingServers