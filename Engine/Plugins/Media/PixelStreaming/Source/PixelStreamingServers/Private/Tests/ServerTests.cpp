// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "PixelStreamingServers.h"
#include "Misc/Paths.h"
#include "PixelStreamingServersLog.h"
#include "ServerUtils.h"
#include "WebSocketProbe.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreamingServers
{
	static const int	 HttpPort = 85;
	static const FString ExpectedWebserverAddress = FString::Printf(TEXT("http://127.0.0.1:%d"), HttpPort);
	static const FString ExpectedPlayerWSAddress = FString::Printf(TEXT("ws://127.0.0.1:%d"), HttpPort);

	static const int	 SFUPort = 8889;
	static const FString ExpectedSFUAddress = FString::Printf(TEXT("ws://127.0.0.1:%d"), SFUPort);

	static const int	 StreamerPort = 8989;
	static const FString ExpectedStreamerAddress = FString::Printf(TEXT("ws://127.0.0.1:%d"), StreamerPort);

	static const int	 MatchmakerPort = 9999;
	static const FString ExpectedMatchmakerAddress = FString::Printf(TEXT("ws://127.0.0.1:%d"), MatchmakerPort);

	static const bool bTestServerBinary = false;

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitForServerOrTimeout, TSharedPtr<UE::PixelStreamingServers::IServer>, Server);
	bool FWaitForServerOrTimeout::Update()
	{
		if (Server)
		{
			return Server->IsTimedOut() || Server->IsReady();
		}
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCleanupServer, TSharedPtr<IServer>, Server);
	bool FCleanupServer::Update()
	{
		if (Server)
		{
			Server->Stop();
		}
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCheckNumStreamers, TSharedPtr<IServer>, Server, uint16, ExpectedNumStreamers);
	bool FCheckNumStreamers::Update()
	{
		if (Server && Server->IsReady())
		{
			uint16 ActualNumberOfStreamers = 0;

			Server->GetNumStreamers([&ActualNumberOfStreamers](uint16 NumStreamers) {
				ActualNumberOfStreamers = NumStreamers;
			});

			bool	bTestPassed = ActualNumberOfStreamers == ExpectedNumStreamers;
			FString LogString = FString::Printf(TEXT("Testing num ws connections. Actual=%d | Expected=%d"), ActualNumberOfStreamers, ExpectedNumStreamers);
			if (bTestPassed)
			{
				UE_LOGF(LogPixelStreamingServers, Log, "Success: %ls", *LogString);
				return true;
			}
			else if (GetCurrentRunTime() > 2.0f)
			{
				UE_LOGF(LogPixelStreamingServers, Error, "Failed (timed out after 2s): %ls", *LogString);
				return true;
			}
		}
		return false;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FStartWebsocketClient, TSharedPtr<IServer>, Server, TSharedPtr<FWebSocketProbe>, Probe, FURL, WSStreamerURL);
	bool FStartWebsocketClient::Update()
	{
		if (GetCurrentRunTime() > 2.0f)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Timed out after 2s of waiting for websocket to connect");
			return true;
		}

		// Do not proceed further into the test if server is not ready
		if (!Server || !Server->IsReady())
		{
			return false;
		}

		if (!Probe)
		{
			return false;
		}

		if (Probe->Probe())
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket client probe connected.");
			return true;
		}
		else
		{
			return false;
		}
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCloseWebsocketClient, TSharedPtr<IServer>, Server, TSharedPtr<FWebSocketProbe>, Probe);
	bool FCloseWebsocketClient::Update()
	{
		if (GetCurrentRunTime() > 2.0f)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Timed out after 2s of waiting for websocket to close");
			return true;
		}

		// Do not proceed further into the test if server is not ready
		if (!Server || !Server->IsReady())
		{
			return false;
		}

		// Do not proceed further into the test if probe is not setup
		if (!Probe)
		{
			return false;
		}

		if (Probe->IsConnected())
		{
			Probe->Close();
			UE_LOGF(LogPixelStreamingServers, Log, "Asked websocket client probe to close.");
			return false;
		}
		else
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket client probe closed.");
			return true;
		}
	}

	FString GetCirrusBinaryAbsPath()
	{

	#if PLATFORM_WINDOWS || PLATFORM_LINUX
		FString ServerPath =
			FPaths::EnginePluginsDir() / TEXT("Media") / TEXT("PixelStreaming") / TEXT("Resources") / TEXT("WebServers") / TEXT("SignallingWebServer");

	#if PLATFORM_WINDOWS
		ServerPath = ServerPath / TEXT("cirrus.exe");
	#else
		ServerPath = ServerPath / TEXT("cirrus");
	#endif
		ServerPath = FPaths::ConvertRelativePathToFull(ServerPath);

		return ServerPath;

	#elif PLATFORM_MAC
		UE_LOGF(LogPixelStreamingServers, Error, "No cirrus binaries exist for Mac!");
		return "";
	#else
		UE_LOGF(LogPixelStreamingServers, Error, "Unsupported platform for Pixel Streaming.");
		return "";
	#endif
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLaunchDownloadedCirrusTest, "System.Plugins.PixelStreaming.LaunchDownloadedCirrus", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FLaunchDownloadedCirrusTest::RunTest(const FString& Parameters)
	{
		UE_LOGF(LogPixelStreamingServers, Log, "----------- LaunchDownloadedCirrusTest -----------");

		TSharedPtr<IServer> SignallingServer = MakeCirrusServer();
		FLaunchArgs			LaunchArgs;
		LaunchArgs.bPollUntilReady = true;
		LaunchArgs.ReconnectionTimeoutSeconds = 30.0f;
		LaunchArgs.ReconnectionIntervalSeconds = 2.0f;
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--HttpPort=%d --SFUPort=%d --StreamerPort=%d --MatchmakerPort=%d"), HttpPort, SFUPort, StreamerPort, MatchmakerPort);

		if (bTestServerBinary)
		{
			LaunchArgs.ServerBinaryOverridePath = GetCirrusBinaryAbsPath();
		}

		bool bLaunched = SignallingServer->Launch(LaunchArgs);

		if (!bLaunched)
		{
			// If we were unable to launch this means some files were missing, we early exit here because this will always happen on Horde
			// and we don't want a permanently failing test on Horde.
			// Todo: Determine way to only disable test on Horde but not locally, or to make it actually download the required scripts.
			return true;
		}

		SignallingServer->OnReady.AddLambda([this](TMap<EEndpoint, FURL> Endpoints) {
			TestTrue("Got server OnReady.", true);

			FString ActualWebserverUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_Webserver]);
			UE_LOGF(LogPixelStreamingServers, Log, "Http address for webserver. Actual=%ls | Expected=%ls", *ActualWebserverUrl, *ExpectedWebserverAddress);
			TestTrue(FString::Printf(TEXT("Http address for webserver. Actual=%s | Expected=%s"), *ActualWebserverUrl, *ExpectedWebserverAddress),
				ActualWebserverUrl == ExpectedWebserverAddress);

			FString ActualStreamerUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_Streamer]);
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket address for streamer messages. Actual=%ls | Expected=%ls", *ActualStreamerUrl, *ExpectedStreamerAddress);
			TestTrue(FString::Printf(TEXT("Websocket address for streamer messages. Actual=%s | Expected=%s"), *ActualStreamerUrl, *ExpectedStreamerAddress),
				ActualStreamerUrl == ExpectedStreamerAddress);

			FString ActualPlayersUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_Players]);
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket address for player messages. Actual=%ls | Expected=%ls", *ActualPlayersUrl, *ExpectedPlayerWSAddress);
			TestTrue(FString::Printf(TEXT("Websocket address for player messages. Actual=%s | Expected=%s"), *ActualPlayersUrl, *ExpectedPlayerWSAddress),
				ActualPlayersUrl == ExpectedPlayerWSAddress);

			FString ActualSFUUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_SFU]);
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket address for SFU messages. Actual=%ls | Expected=%ls", *ActualSFUUrl, *ExpectedSFUAddress);
			TestTrue(FString::Printf(TEXT("Websocket address for SFU messages. Actual=%s | Expected=%s"), *ActualSFUUrl, *ExpectedSFUAddress),
				ActualSFUUrl == ExpectedSFUAddress);

			FString ActualMatchmakerUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_Matchmaker]);
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket address for matchmaker messages. Actual=%ls | Expected=%ls", *ActualMatchmakerUrl, *ExpectedMatchmakerAddress);
			TestTrue(FString::Printf(TEXT("Websocket address for matchmaker messages. Actual=%s | Expected=%s"), *ActualMatchmakerUrl, *ExpectedMatchmakerAddress),
				ActualMatchmakerUrl == ExpectedMatchmakerAddress);
		});

		SignallingServer->OnFailedToReady.AddLambda([this]() {
			TestTrue("Server was not ready.", false);
		});

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForServerOrTimeout(SignallingServer));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupServer(SignallingServer));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLaunchEmbeddedCirrusTest, "System.Plugins.PixelStreaming.LaunchEmbeddedCirrus", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FLaunchEmbeddedCirrusTest::RunTest(const FString& Parameters)
	{
		UE_LOGF(LogPixelStreamingServers, Log, "----------- LaunchEmbeddedCirrusTest -----------");

		TSharedPtr<IServer> SignallingServer = MakeSignallingServer();
		FLaunchArgs			LaunchArgs;
		LaunchArgs.bPollUntilReady = true;
		LaunchArgs.ReconnectionTimeoutSeconds = 30.0f;
		LaunchArgs.ReconnectionIntervalSeconds = 2.0f;
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--HttpPort=%d --StreamerPort=%d"), HttpPort, StreamerPort);

		bool bLaunched = SignallingServer->Launch(LaunchArgs);
		UE_LOGF(LogPixelStreamingServers, Log, "Embedded cirrus launched: %ls", bLaunched ? TEXT("true") : TEXT("false"));
		TestTrue("Embedded cirrus launched.", bLaunched);

		if (!bLaunched)
		{
			return false;
		}

		SignallingServer->OnReady.AddLambda([this](TMap<EEndpoint, FURL> Endpoints) {
			TestTrue("Got server OnReady.", true);

			FString ActualWebserverUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_Webserver]);
			UE_LOGF(LogPixelStreamingServers, Log, "Http address for webserver. Actual=%ls | Expected=%ls", *ActualWebserverUrl, *ExpectedWebserverAddress);
			TestTrue(FString::Printf(TEXT("Http address for webserver. Actual=%s | Expected=%s"), *ActualWebserverUrl, *ExpectedWebserverAddress),
				ActualWebserverUrl == ExpectedWebserverAddress);

			FString ActualStreamerUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_Streamer]);
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket address for streamer messages. Actual=%ls | Expected=%ls", *ActualStreamerUrl, *ExpectedStreamerAddress);
			TestTrue(FString::Printf(TEXT("Websocket address for streamer messages. Actual=%s | Expected=%s"), *ActualStreamerUrl, *ExpectedStreamerAddress),
				ActualStreamerUrl == ExpectedStreamerAddress);

			FString ActualPlayersUrl = Utils::ToString(Endpoints[EEndpoint::Signalling_Players]);
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket address for player messages. Actual=%ls | Expected=%ls", *ActualPlayersUrl, *ExpectedPlayerWSAddress);
			TestTrue(FString::Printf(TEXT("Websocket address for player messages. Actual=%s | Expected=%s"), *ActualPlayersUrl, *ExpectedPlayerWSAddress),
				ActualPlayersUrl == ExpectedPlayerWSAddress);
		});

		SignallingServer->OnFailedToReady.AddLambda([this]() {
			TestTrue("Server was not ready.", false);
		});

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForServerOrTimeout(SignallingServer));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupServer(SignallingServer));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTwoWebsocketToEmbeddedCirrusTest, "System.Plugins.PixelStreaming.TwoWebsocketToEmbeddedCirrus", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FTwoWebsocketToEmbeddedCirrusTest::RunTest(const FString& Parameters)
	{
		UE_LOGF(LogPixelStreamingServers, Log, "----------- TwoWebsocketToEmbeddedCirrus -----------");

		TSharedPtr<IServer> SignallingServer = MakeSignallingServer();
		FLaunchArgs			LaunchArgs;
		LaunchArgs.bPollUntilReady = true;
		LaunchArgs.ReconnectionTimeoutSeconds = 30.0f;
		LaunchArgs.ReconnectionIntervalSeconds = 2.0f;
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--HttpPort=%d --StreamerPort=%d"), HttpPort, StreamerPort);

		bool bLaunched = SignallingServer->Launch(LaunchArgs);
		UE_LOGF(LogPixelStreamingServers, Log, "Embedded cirrus launched: %ls", bLaunched ? TEXT("true") : TEXT("false"));
		TestTrue("Embedded cirrus launched.", bLaunched);

		if (!bLaunched)
		{
			return false;
		}

		FURL WSStreamerURL;
		WSStreamerURL.Protocol = TEXT("ws");
		WSStreamerURL.Host = TEXT("127.0.0.1");
		WSStreamerURL.Port = StreamerPort;
		WSStreamerURL.Map = FString();

		SignallingServer->OnReady.AddLambda([this](TMap<EEndpoint, FURL> Endpoints) {
			TestTrue("Got server OnReady.", true);
		});

		SignallingServer->OnFailedToReady.AddLambda([this]() {
			TestTrue("Server was not ready.", false);
		});

		// These websocket clients will be used to test number of connections
		TArray<FString> Protocols;
		Protocols.Add(FString(TEXT("binary")));
		TSharedPtr<FWebSocketProbe> Client1 = TSharedPtr<FWebSocketProbe>(new FWebSocketProbe(WSStreamerURL, Protocols));
		TSharedPtr<FWebSocketProbe> Client2 = TSharedPtr<FWebSocketProbe>(new FWebSocketProbe(WSStreamerURL, Protocols));

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForServerOrTimeout(SignallingServer));
		// Test that there should be zero streamers connected after the server is initially up
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumStreamers(SignallingServer, 0));
		// Start ws client 1
		ADD_LATENT_AUTOMATION_COMMAND(FStartWebsocketClient(SignallingServer, Client1, WSStreamerURL));
		// Check num streamers is 1
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumStreamers(SignallingServer, 1));
		// Start ws client 2
		ADD_LATENT_AUTOMATION_COMMAND(FStartWebsocketClient(SignallingServer, Client2, WSStreamerURL));
		// Check num streamers = 2
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumStreamers(SignallingServer, 2));
		// Close client 1
		ADD_LATENT_AUTOMATION_COMMAND(FCloseWebsocketClient(SignallingServer, Client1));
		// Check num streamers is 1
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumStreamers(SignallingServer, 1));
		// Close client 2
		ADD_LATENT_AUTOMATION_COMMAND(FCloseWebsocketClient(SignallingServer, Client2));
		// Check num streamers is 0
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumStreamers(SignallingServer, 0));
		// Shut down server
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupServer(SignallingServer));
		return true;
	}

	// Todo test where create and teardown signalling server 10 times in quick succession with probe to ensure ports are freed

} // namespace UE::PixelStreamingServers

#endif // WITH_DEV_AUTOMATION_TESTS