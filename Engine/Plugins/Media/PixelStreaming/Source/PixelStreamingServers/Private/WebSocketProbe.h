// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWebSocket.h"
#include "HAL/ThreadSafeCounter.h"
#include "WebSocketsModule.h"
#include "Templates/SharedPointer.h"
#include "PixelStreamingServersLog.h"
#include "Engine/EngineBaseTypes.h"
#include "ServerUtils.h"

namespace UE::PixelStreamingServers
{

	/*
	 * A utility class that tries to establish a websocket connection.
	 * Useful for testing whether servers have come online yet.
	 */
	class FWebSocketProbe
	{
	private:
		TSharedRef<IWebSocket> WebSocket;
		FThreadSafeBool		   bShouldAttemptReconnect;
		FThreadSafeBool		   bIsClosing = false;

	public:
		FWebSocketProbe(FURL Url, TArray<FString> Protocols = TArray<FString>())
			: WebSocket(FWebSocketsModule::Get().CreateWebSocket(Utils::ToString(Url), Protocols))
			, bShouldAttemptReconnect(true)
			, bIsClosing(false)
		{
			WebSocket->OnConnectionError().AddLambda([Url, &bShouldAttemptReconnect = bShouldAttemptReconnect](const FString& Error) {
				UE_LOGF(LogPixelStreamingServers, Log, "Probing websocket %ls | Msg= \"%ls\" | Retrying...", *Utils::ToString(Url), *Error);
				bShouldAttemptReconnect = true;
			});
		}

		void Close()
		{
			if (WebSocket->IsConnected() && !bIsClosing)
			{
				WebSocket->Close();
			}
			bIsClosing = true;
		}

		bool IsConnected() const
		{
			return WebSocket->IsConnected();
		}

		bool Probe()
		{
			bool bIsConnected = WebSocket->IsConnected();

			if (!bIsConnected && bShouldAttemptReconnect)
			{
				WebSocket->Connect();
				bShouldAttemptReconnect = false;
				bIsClosing = false;
			}

			return bIsConnected;
		}
	};

} // namespace UE::PixelStreamingServers