// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketServerWrapper.h"
#include "IWebSocketNetworkingModule.h"
#include "PixelStreamingServersLog.h"
#include "Modules/ModuleManager.h"

namespace UE::PixelStreamingServers
{

	/*
	* ------------- FWebSocketConnection -------------
	*/

	FThreadSafeCounter FWebSocketConnection::IdGenerator = FThreadSafeCounter(100);

	FWebSocketConnection::FWebSocketConnection(INetworkingWebSocket* InSocketConnection)
		: Id(IdGenerator.Increment())
		, SocketConnection(InSocketConnection)
	{
		UrlArgs = SocketConnection->GetUrlArgs();
	}

	FWebSocketConnection::~FWebSocketConnection()
	{
		if (SocketConnection)
		{
			delete SocketConnection;
			SocketConnection = nullptr;
		}
	}

	uint16 FWebSocketConnection::GetId() const
	{
		return Id;
	}

	TArray<FString> FWebSocketConnection::GetUrlArgs() const
	{
		return UrlArgs;
	}

	bool FWebSocketConnection::Send(FString Message) const
	{
		// Convert FString into uint8 array.
		FTCHARToUTF8 UTF8String(*Message);
		
		// Send the uint8 buffer
		// Note: Due to how this socket connection is implemented, only binary messages are supported
		return SocketConnection->Send((const uint8*) UTF8String.Get(), UTF8String.Length(), false);
	}

	void FWebSocketConnection::SetCallbacks()
	{
		if(SocketConnection)
		{
			SocketConnection->SetReceiveCallBack(OnPacketReceivedCallback);
			SocketConnection->SetSocketClosedCallBack(OnClosedCallback);
		}
	}

	/*
	* ------------- FWebSocketServerWrapper -------------
	*/

	FWebSocketServerWrapper::FWebSocketServerWrapper()
		: FTickableGameObject(ETickableTickType::Never)
		, bLaunched(false)
	{

	}

	FWebSocketServerWrapper::~FWebSocketServerWrapper()
	{
		Stop();
	}

	void FWebSocketServerWrapper::EnableWebServer(TArray<FWebSocketHttpMount> InDirectoriesToServe)
	{
		bEnableWebServer = true;
		DirectoriesToServe = InDirectoriesToServe;
	}

	bool FWebSocketServerWrapper::Launch(uint16 Port)
	{
		SetTickableTickType(ETickableTickType::Always);

		WSServer = FModuleManager::Get().LoadModuleChecked<IWebSocketNetworkingModule>(TEXT("WebSocketNetworking")).CreateServer();
		
		if(bEnableWebServer)
		{
			WSServer->EnableHTTPServer(DirectoriesToServe);
		}

		OnClientConnectedCallback.BindRaw(this, &FWebSocketServerWrapper::OnConnectionOpened);
		bLaunched = WSServer->Init(Port, OnClientConnectedCallback);
		if(!bLaunched)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Failed to launch Websocket server at ws://127.0.0.1:%d", Port);
			WSServer.Reset();
			OnClientConnectedCallback.Unbind();
		}
		else
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Started Websocket server at ws://127.0.0.1:%d", Port);
		}
		return bLaunched;
	}

	bool FWebSocketServerWrapper::IsLaunched() const
	{
		return bLaunched;
	}

	bool FWebSocketServerWrapper::HasConnections() const
	{
		return Connections.Num() > 0;
	}

	bool FWebSocketServerWrapper::GetFirstConnection(uint16& OutConnectionId) const
	{
		if(Connections.Num() > 0)
		{
			OutConnectionId = Connections.CreateConstIterator().Key();
			return true;
		}
		return false;
	}

	void FWebSocketServerWrapper::NameConnection(uint16 ConnectionId, const FString& Name)
	{
		if (auto* Connection = Connections.Find(ConnectionId))
		{
			NamedConnections.FindOrAdd(Name) = ConnectionId;
		}
	}

	void FWebSocketServerWrapper::RemoveName(const FString& Name)
	{
		NamedConnections.Remove(Name);
	}

	bool FWebSocketServerWrapper::GetNamedConnection(const FString& Name, uint16& OutConnectionId) const
	{
		if (auto* ConnectionId = NamedConnections.Find(Name))
		{
			OutConnectionId = *ConnectionId;
			return true;
		}
		return false;
	}

	TArray<FString> FWebSocketServerWrapper::GetConnectionNames() const
	{
		TArray<FString> Names;
		for (auto [Name, ConnectionId] : NamedConnections)
		{
			Names.Add(Name);
		}
		return Names;
	}

	void FWebSocketServerWrapper::Stop()
	{
		SetTickableTickType(ETickableTickType::Never);
		
		bLaunched = false;
		Connections.Empty();
		OnClientConnectedCallback.Unbind();
		WSServer.Reset();
	}

	bool FWebSocketServerWrapper::Close(uint16 ConnectionId)
	{
		for (const FString* Key = NamedConnections.FindKey(ConnectionId); Key; Key = NamedConnections.FindKey(ConnectionId))
		{
			NamedConnections.Remove(*Key);
		}

		if(Connections.Contains(ConnectionId))
		{
			return Connections.Remove(ConnectionId) > 0;
		}
		else
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Could not close websocket connection because there was no connection=%d.", ConnectionId);
			return false;
		}
	}

	bool FWebSocketServerWrapper::Send(uint16 ConnectionId, FString Message) const
	{
		if(Connections.Contains(ConnectionId))
		{
			return Connections[ConnectionId]->Send(Message);
		}
		else
		{
			UE_LOGF(LogPixelStreamingServers, Warning, "Did not send websocket message because there was no connection=%d.", ConnectionId);
			return false;
		}
	}

	bool FWebSocketServerWrapper::Send(const FString& ConnectionName, FString Message) const
	{
		if (auto* ConnectionId = NamedConnections.Find(ConnectionName))
		{
			return Send(*ConnectionId, Message);
		}
		return false;
	}

	void FWebSocketServerWrapper::OnConnectionOpened(INetworkingWebSocket* Socket)
	{
		if(!Socket)
		{
			UE_LOGF(LogPixelStreamingServers, Error, "Websocket client connected with a null socket.");
			return;
		}
		else
		{
			UE_LOGF(LogPixelStreamingServers, Log, "Websocket client connected. Remote=%ls | Local=%ls", *Socket->RemoteEndPoint(true), *Socket->LocalEndPoint(true));
		}

		// Had a new client connect over websocket, store the connection with a unique ID.
		TUniquePtr<FWebSocketConnection> Connection = MakeUnique<FWebSocketConnection>(Socket);
		const uint16 Id = Connection->GetId();

		// Bind to socket callbacks for messages/closed.
		Connection->OnPacketReceivedCallback.BindRaw(this, &FWebSocketServerWrapper::OnPacketReceived, Id);
		Connection->OnClosedCallback.BindRaw(this, &FWebSocketServerWrapper::OnConnectionClosed, Id);
		Connection->SetCallbacks();

		Connections.Add(Id, MoveTemp(Connection));

		// Broadcast that we got a new websocket connection
		OnOpenConnection.Broadcast(Id);
	}

	void FWebSocketServerWrapper::OnPacketReceived(void* Data, int32 Size, uint16 ConnectionId)
	{
		if(Size > 0)
		{
			TArrayView<uint8> DataView = MakeArrayView(static_cast<uint8*>(Data), Size);
			OnMessage.Broadcast(ConnectionId, DataView);
		}
	}

	void FWebSocketServerWrapper::OnConnectionClosed(uint16 ConnectionId)
	{
		for (const FString* Key = NamedConnections.FindKey(ConnectionId); Key; Key = NamedConnections.FindKey(ConnectionId))
		{
			NamedConnections.Remove(*Key);
		}

		Connections.Remove(ConnectionId);
		OnClosedConnection.Broadcast(ConnectionId);
	}

	void FWebSocketServerWrapper::Tick(float DeltaTime)
	{
		if(!IsLaunched())
		{
			return;
		}

		if(WSServer)
		{
			// Tick the websocket server
			WSServer->Tick();
		}
	}

} // UE::PixelStreamingServers
